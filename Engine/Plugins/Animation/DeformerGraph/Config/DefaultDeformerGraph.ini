[CoreRedirects]
; Uncomment these if you have old Deformer Graph assets that won't load properly.
;+StructRedirects=(OldName="OptimusType_ShaderText",NewName="/Script/OptimusCore.OptimusShaderText")

;+ClassRedirects=(OldName="/Script/OptimusDeveloper....",NewName="/Script/OptimusCore.", MatchWildcard=true)
;+StructRedirects=(OldName="/Script/OptimusDeveloper....",NewName="/Script/OptimusCore.", MatchWildcard=true)
;+EnumRedirects=(OldName="/Script/OptimusDeveloper....",NewName="/Script/OptimusCore.", MatchWildcard=true)

+ClassRedirects=(OldName="/Script/OptimusCore.ClothDataInterface",NewName="/Script/OptimusCore.OptimusClothDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.ClothDataProvider",NewName="/Script/OptimusCore.OptimusClothDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.DataInterface",NewName="/Script/OptimusCore.OptimusGraphDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.GraphDataProvider",NewName="/Script/OptimusCore.OptimusGraphDataProvider")
+StructRedirects=(OldName="/Script/OptimusCore.GraphVariableDescription",NewName="/Script/OptimusCore.OptimusGraphVariableDescription")

+ClassRedirects=(OldName="/Script/OptimusCore.MorphTargetDataInterface",NewName="/Script/OptimusCore.OptimusMorphTargetDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.MorphTargetDataProvider",NewName="/Script/OptimusCore.OptimusMorphTargetDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.TransientBufferDataInterface",NewName="/Script/OptimusCore.OptimusTransientBufferDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.PersistentBufferDataInterface",NewName="/Script/OptimusCore.OptimusPersistentBufferDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.TransientBufferDataProvider",NewName="/Script/OptimusCore.OptimusTransientBufferDataProvider")
+ClassRedirects=(OldName="/Script/OptimusCore.PersistentBufferDataProvider",NewName="/Script/OptimusCore.OptimusPersistentBufferDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.SceneDataInterface",NewName="/Script/OptimusCore.OptimusSceneDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SceneDataProvider",NewName="/Script/OptimusCore.OptimusSceneDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.SkeletalMeshReadDataInterface",NewName="/Script/OptimusCore.OptimusSkinnedMeshDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SkeletalMeshReadDataProvider",NewName="/Script/OptimusCore.OptimusSkinnedMeshDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.SkeletonDataInterface",NewName="/Script/OptimusCore.OptimusSkeletonDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SkeletonDataProvider",NewName="/Script/OptimusCore.OptimusSkeletonDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshDataInterface",NewName="/Script/OptimusCore.OptimusSkinnedMeshDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshDataProvider",NewName="/Script/OptimusCore.OptimusSkinnedMeshDataProvider")

+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshExecDataInterface",NewName="/Script/OptimusCore.OptimusSkinnedMeshExecDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshExecDataProvider",NewName="/Script/OptimusCore.OptimusSkinnedMeshExecDataProvider")
+EnumRedirects=(OldName="/Script/OptimusCore.ESkinnedMeshExecDomain",NewName="/Script/OptimusCore.EOptimusSkinnedMeshExecDomain")

+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshWriteDataInterface",NewName="/Script/OptimusCore.OptimusSkinnedMeshWriteDataInterface")
+ClassRedirects=(OldName="/Script/OptimusCore.SkinnedMeshWriteDataProvider",NewName="/Script/OptimusCore.OptimusSkinnedMeshWriteDataProvider")

+StructRedirects=(OldName="/Script/OptimusCore.OptimusMultiLevelDataDomain", NewName="/Script/OptimusCore.OptimusDataDomain")

[/Script/OptimusSettings.OptimusSettings]
+DefaultDeformer=/DeformerGraph/Deformers/DG_LinearBlendSkin_Morph_Cloth.DG_LinearBlendSkin_Morph_Cloth
+DefaultRecomputeTangentDeformer=/DeformerGraph/Deformers/DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals.DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals
