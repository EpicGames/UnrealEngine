{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.9",
	"FriendlyName" : "Deformer Graph",
	"Description" : "Editor for creating GPU mesh deformation graphs",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "OptimusSettings",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OptimusCore",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "OptimusDeveloper",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
			
		},
		{
			"Name": "OptimusEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "ComputeFramework",
			"Enabled": true
		},
		{
			"Name": "ControlRig",
			"Enabled": true
		}
	]
}
