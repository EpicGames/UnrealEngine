// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/Optimus/Private/DataInterfaceSkeletonCommon.ush"

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumBoneInfluences;
uint {DataInterfaceName}_InputWeightStride;
uint {DataInterfaceName}_InputWeightIndexSize;
FCompressedBoneTransformBuffer {DataInterfaceName}_BoneMatrices;
Buffer<uint> {DataInterfaceName}_InputWeightStream;
Buffer<uint> {DataInterfaceName}_InputWeightLookupStream;

StructuredBuffer<float3x4> {DataInterfaceName}_LayeredBoneMatrices; 

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumBones_{DataInterfaceName}(uint VertexIndex)
{
	return ReadNumBones(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_NumBoneInfluences,
		VertexIndex
		);
}

float3x4 ReadBoneMatrix_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
	return ReadBoneMatrix(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_BoneMatrices,
		VertexIndex,
		BoneIndex
		);
}

float ReadBoneWeight_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
	return ReadBoneWeight(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_NumBoneInfluences,
		VertexIndex,
		BoneIndex
		);
}

float3x4 ReadWeightedBoneMatrix_{DataInterfaceName}(uint VertexIndex)
{
	return ReadWeightedBoneMatrix(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_NumBoneInfluences,
		{DataInterfaceName}_BoneMatrices,
		VertexIndex
		);
}


float3x4 ReadLayeredBoneMatrix_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
#if !ENABLE_DEFORMER_BONES

	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex 		= VertexIndex;
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[BufferIndex];
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = {DataInterfaceName}_InputWeightIndexSize & 0xff;
	int BoneIndexOffset = StreamOffset + (BoneIndexSize * BoneIndex);
	int BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset];
	if (BoneIndexSize > 1)
	{
		BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset + 1] << 8 | BoneGlobalIndex;
	}
	
	return {DataInterfaceName}_LayeredBoneMatrices[BoneGlobalIndex];

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = VertexIndex * ({DataInterfaceName}_InputWeightStride / 4);
	int BlendIndex = GetBlendIndices({DataInterfaceName}_InputWeightStream, StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return {DataInterfaceName}_LayeredBoneMatrices[BlendIndex];

#endif
}

float3x4 ReadWeightedLayeredBoneMatrix_{DataInterfaceName}(uint VertexIndex)
{
#if !ENABLE_DEFORMER_BONES
	
	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);
	
#else
	uint NumBoneInfluences = ReadNumBones_{DataInterfaceName}(VertexIndex);
	float3x4 Result = 0;
	
	for (uint InfluenceIdx = 0; InfluenceIdx < NumBoneInfluences; InfluenceIdx++)
	{
		float Weight = ReadBoneWeight_{DataInterfaceName}(VertexIndex, InfluenceIdx);
		float3x4 BoneMatrix = ReadLayeredBoneMatrix_{DataInterfaceName}(VertexIndex, InfluenceIdx);
		Result += Weight * BoneMatrix;
	}

	return Result;
#endif
}