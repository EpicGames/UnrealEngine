// Copyright Epic Games, Inc. All Rights Reserved.

StructuredBuffer<{TypeName}> {DataInterfaceName}_{AttributeName};


{TypeName} Read{AttributeName}_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
#if !ENABLE_DEFORMER_BONES

	return 0;

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex 		= VertexIndex;
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[BufferIndex];
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = {DataInterfaceName}_InputWeightIndexSize & 0xff;
	int BoneIndexOffset = StreamOffset + (BoneIndexSize * BoneIndex);
	int BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset];
	if (BoneIndexSize > 1)
	{
		BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset + 1] << 8 | BoneGlobalIndex;
	}
	
	return {DataInterfaceName}_{AttributeName}[BoneGlobalIndex];

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));
	int BlendIndex = GetBlendIndices({DataInterfaceName}_InputWeightStream, StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return {DataInterfaceName}_{AttributeName}[BlendIndex];

#endif
}