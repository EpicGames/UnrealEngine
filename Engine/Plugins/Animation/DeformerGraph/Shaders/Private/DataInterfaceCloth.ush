// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef DATA_INTERFACE_CLOTH_ONCE
#define DATA_INTERFACE_CLOTH_ONCE 1

struct FVertexTriangleInfluence
{
	float4 PositionBaryCoordsAndDist;
	float4 NormalBaryCoordsAndDist;
	float4 TangentBaryCoordsAndDist;
	uint4 SourceMeshVertIndices;
	float Weight;
};

struct FClothResult
{
	float ClothWeight;
	float3 Position;
	float3 TangentX;
	float3 TangentZ;
};

#endif // DATA_INTERFACE_CLOTH_ONCE

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_BaseVertexIndex;
uint {DataInterfaceName}_InputStreamStart;
uint {DataInterfaceName}_NumInfluencesPerVertex;
float {DataInterfaceName}_ClothBlendWeight;
float3 {DataInterfaceName}_MeshScale;
float4x4 {DataInterfaceName}_ClothToLocal;
Buffer<float4> {DataInterfaceName}_ClothBuffer;
Buffer<float2> {DataInterfaceName}_ClothPositionsAndNormalsBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float4x4 ReadClothToLocal_{DataInterfaceName}()
{
#if ENABLE_DEFORMER_CLOTH
	return {DataInterfaceName}_ClothToLocal;
#else
	return float4x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1);
#endif
}

FVertexTriangleInfluence GetClothInfluence_{DataInterfaceName}(uint VertexIndex, uint InfluenceIndex)
{
	const uint VertexOffset = {DataInterfaceName}_NumInfluencesPerVertex * (VertexIndex - {DataInterfaceName}_BaseVertexIndex)  + {DataInterfaceName}_InputStreamStart;

	const uint NUM_FLOAT4S_PER_VERTEX_INFLUENCE = 4; // Stride of GPUSkinApexCloth
	const uint Offset = (VertexOffset + InfluenceIndex) * NUM_FLOAT4S_PER_VERTEX_INFLUENCE;

	FVertexTriangleInfluence Influence;

	Influence.PositionBaryCoordsAndDist = {DataInterfaceName}_ClothBuffer[Offset];
	Influence.NormalBaryCoordsAndDist = {DataInterfaceName}_ClothBuffer[Offset + 1];
	Influence.TangentBaryCoordsAndDist = {DataInterfaceName}_ClothBuffer[Offset + 2];

	uint4 PackedIndices = asuint({DataInterfaceName}_ClothBuffer[Offset + 3]);
	Influence.SourceMeshVertIndices.yw = (PackedIndices.xy >> 16) & 0xffff;
	Influence.SourceMeshVertIndices.xz = PackedIndices.xy & 0xffff;
	
	Influence.Weight = asfloat(PackedIndices[2]);

	return Influence;
}

FClothResult GetClothResult_{DataInterfaceName}(uint VertexIndex)
{
	float ClothWeight = 0;
	float3 SimulatedPosition = float3(0, 0, 0);
	float3 NormalPosition = float3(0, 0, 0);
	float3 TangentPosition = float3(0, 0, 0);

	int NumInfluences = 0;
	float SumWeights = 0;

	for (uint i = 0; i < {DataInterfaceName}_NumInfluencesPerVertex; ++i)
	{
		const FVertexTriangleInfluence Influence = GetClothInfluence_{DataInterfaceName}(VertexIndex, i);

		if (Influence.SourceMeshVertIndices.w < 0xFFFF)
		{
			++NumInfluences;

			float3 A = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3], {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 1].x);
			float3 B = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3], {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 1].x);
			float3 C = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3], {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 1].x);

			float3 NA = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 1].y, {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.x * 3 + 2]);
			float3 NB = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 1].y, {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.y * 3 + 2]);
			float3 NC = float3({DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 1].y, {DataInterfaceName}_ClothPositionsAndNormalsBuffer[Influence.SourceMeshVertIndices.z * 3 + 2]);

			ClothWeight += {DataInterfaceName}_ClothBlendWeight * (1.0f - (Influence.SourceMeshVertIndices.w / 65535.0f));

			float Weight = 1.0f;
			if ({DataInterfaceName}_NumInfluencesPerVertex > 1)
			{
				// Weight is packed in the last coordinate
				Weight = Influence.Weight;
				SumWeights += Weight;
			}
			else
			{
				// Single influence, weight is 1.0
				Weight = 1.0f;
				SumWeights = 1.0f;
			}
			
			float3 Scale = {DataInterfaceName}_MeshScale;

			NormalPosition += Weight * (Influence.NormalBaryCoordsAndDist.x * (A + NA * Influence.NormalBaryCoordsAndDist.w * Scale.x)
				+ Influence.NormalBaryCoordsAndDist.y * (B + NB * Influence.NormalBaryCoordsAndDist.w * Scale.y)
				+ Influence.NormalBaryCoordsAndDist.z * (C + NC * Influence.NormalBaryCoordsAndDist.w * Scale.z));

			TangentPosition += Weight * (Influence.TangentBaryCoordsAndDist.x * (A + NA * Influence.TangentBaryCoordsAndDist.w * Scale.x)
				+ Influence.TangentBaryCoordsAndDist.y * (B + NB * Influence.TangentBaryCoordsAndDist.w * Scale.y)
				+ Influence.TangentBaryCoordsAndDist.z * (C + NC * Influence.TangentBaryCoordsAndDist.w * Scale.z));

			float3 TriangleBary = float3(Influence.PositionBaryCoordsAndDist.x,
				Influence.PositionBaryCoordsAndDist.y,
				1.0f - Influence.PositionBaryCoordsAndDist.x - Influence.PositionBaryCoordsAndDist.y);

			float3 SimPosition = TriangleBary.x * (A + NA * Influence.PositionBaryCoordsAndDist.w * Scale.x)
				+ TriangleBary.y * (B + NB * Influence.PositionBaryCoordsAndDist.w * Scale.y)
				+ TriangleBary.z * (C + NC * Influence.PositionBaryCoordsAndDist.w * Scale.z);

			SimulatedPosition += Weight * SimPosition;
		}
	}

	FClothResult Result;
	Result.ClothWeight = 0;
	Result.Position = float3(0, 0, 0);
	Result.TangentX = float3(0, 0, 0);
	Result.TangentZ = float3(0, 0, 0);

	if (NumInfluences > 0 && SumWeights > 1e-4f)
	{
		float InvWeight = 1.0f / SumWeights;
		SimulatedPosition *= InvWeight;
		TangentPosition *= InvWeight;
		NormalPosition *= InvWeight;

		Result.ClothWeight = ClothWeight / (float) {DataInterfaceName}_NumInfluencesPerVertex;

		Result.Position = SimulatedPosition;
		
		Result.TangentX = normalize(TangentPosition - SimulatedPosition);
		Result.TangentZ = normalize(NormalPosition - SimulatedPosition);

		// Cloth data are all in world space so need to change into local space
		Result.TangentX = mul(Result.TangentX, (half3x3) {DataInterfaceName}_ClothToLocal);
		Result.TangentZ = mul(Result.TangentZ, (half3x3) {DataInterfaceName}_ClothToLocal);
	}

	return Result;
}

// Individual functions all call the same (expensive) code.
// There is the hope that the shader compiler may optimize here.
// But really we will need to work out a way for Compute Kernels to do some shared work once and carry the results of that in an opaque context struct.

float ReadClothWeight_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_CLOTH
	FClothResult Result = GetClothResult_{DataInterfaceName}(VertexIndex);
	return Result.ClothWeight;
#else
	return 0;
#endif
}

float3 ReadClothPosition_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_CLOTH
	FClothResult Result = GetClothResult_{DataInterfaceName}(VertexIndex);
	return Result.Position;
#else
	return float3(0, 0, 0);
#endif
}

float3 ReadClothTangentX_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_CLOTH
	FClothResult Result = GetClothResult_{DataInterfaceName}(VertexIndex);
	return Result.TangentX;
#else
	return float3(1, 0, 0);
#endif
}

float3 ReadClothTangentZ_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_CLOTH
	FClothResult Result = GetClothResult_{DataInterfaceName}(VertexIndex);
	return Result.TangentZ;
#else
	return float3(0, 0, 1);
#endif
}
