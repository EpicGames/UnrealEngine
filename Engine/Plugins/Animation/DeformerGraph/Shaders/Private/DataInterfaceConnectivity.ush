// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_MaxConnectedVertexCount;
StructuredBuffer<uint> {DataInterfaceName}_ConnectivityBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumConnectedVertices_{DataInterfaceName}(uint VertexIndex)
{
	return {DataInterfaceName}_MaxConnectedVertexCount;
}

uint ReadConnectedVertex_{DataInterfaceName}(uint VertexIndex, uint ConnectedVertexSubIndex)
{
	uint BaseIndex = (VertexIndex) * {DataInterfaceName}_MaxConnectedVertexCount;
	uint ConnectedVertexIndex = {DataInterfaceName}_ConnectivityBuffer[BaseIndex + ConnectedVertexSubIndex];
	return ConnectedVertexIndex;
}
