// Copyright Epic Games, Inc. All Rights Reserved.

uint3 {DataInterfaceName}_NumThreads;
uint {DataInterfaceName}_NumThreadsPerInvocation;
uint {DataInterfaceName}_ThreadIndexOffset;

uint3 ReadNumThreads_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumThreads;
}

uint ReadNumThreadsPerInvocation_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumThreadsPerInvocation;
}

uint ReadThreadIndexOffset_{DataInterfaceName}()
{
	return {DataInterfaceName}_ThreadIndexOffset;
}