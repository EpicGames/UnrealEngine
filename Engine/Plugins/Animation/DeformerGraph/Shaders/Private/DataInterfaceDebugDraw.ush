// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/ShaderPrintCommon.ush"

#ifndef DATA_INTERFACE_DEBUGDRAW_ONCE
#define DATA_INTERFACE_DEBUGDRAW_ONCE 1

struct FDebugDraw
{
	FShaderPrintContext Context;
	float4x4 LocalToWorld;

	float3 TransformLocalToWorld(float3 P)
	{
		return mul(float4(P, 1), LocalToWorld).xyz;
	}

	void AddLine(float3 Pos0, float3 Pos1, float4 Color)
	{
		AddLineTWS(Context, TransformLocalToWorld(Pos0), TransformLocalToWorld(Pos1), Color);
	}

	void AddCross(float3 Pos, float Size, float4 Color)
	{
		AddCrossTWS(Context, TransformLocalToWorld(Pos), Size, Color);
	}

	void AddQuad(float3 Pos0, float3 Pos1, float3 Pos2, float3 Pos3, float4 Color)
	{
		AddQuadTWS(
			Context,
			TransformLocalToWorld(Pos0),
			TransformLocalToWorld(Pos1),
			TransformLocalToWorld(Pos2),
			TransformLocalToWorld(Pos3),
			Color);
	}

	void AddAxis(float3 Pos, float3x3 InM, float Scale)
	{
		AddReferentialWS(Context, TransformLocalToWorld(Pos), mul(InM, (float3x3)LocalToWorld), Scale);
	}
};

#endif // DATA_INTERFACE_DEBUGDRAW_ONCE

float4x4 {DataInterfaceName}_LocalToWorld;
int2 {DataInterfaceName}_Resolution;
float2 {DataInterfaceName}_FontSize;
float2 {DataInterfaceName}_FontSpacing;
uint {DataInterfaceName}_MaxCharacterCount;
uint {DataInterfaceName}_MaxSymbolCount;
uint {DataInterfaceName}_MaxStateCount;
uint {DataInterfaceName}_MaxLineCount;
uint {DataInterfaceName}_MaxTriangleCount;
Buffer<uint> {DataInterfaceName}_StateBuffer;
RWBuffer<uint> {DataInterfaceName}_RWEntryBuffer;

FDebugDraw ReadDebugDraw_{DataInterfaceName}()
{
	FDebugDraw DebugDraw;
	DebugDraw.Context.bIsActive = true;
	DebugDraw.Context.StartPos = 0;
	DebugDraw.Context.Pos = 0;
	DebugDraw.Context.Config.Resolution = {DataInterfaceName}_Resolution;
	DebugDraw.Context.Config.CursorCoord = 0;
	DebugDraw.Context.Config.TranslatedWorldOffset = 0;
	DebugDraw.Context.Config.FontSize = {DataInterfaceName}_FontSize;
	DebugDraw.Context.Config.FontSpacing = {DataInterfaceName}_FontSpacing;
	DebugDraw.Context.Config.MaxCharacterCount = {DataInterfaceName}_MaxCharacterCount;
	DebugDraw.Context.Config.MaxSymbolCount = {DataInterfaceName}_MaxSymbolCount;
	DebugDraw.Context.Config.MaxStateCount = {DataInterfaceName}_MaxStateCount;
	DebugDraw.Context.Config.MaxLineCount = {DataInterfaceName}_MaxLineCount;
	DebugDraw.Context.Config.MaxTriangleCount = {DataInterfaceName}_MaxTriangleCount;
	DebugDraw.Context.Buffers.StateBuffer = {DataInterfaceName}_StateBuffer;
	DebugDraw.Context.Buffers.RWEntryBuffer = {DataInterfaceName}_RWEntryBuffer;
	DebugDraw.LocalToWorld = {DataInterfaceName}_LocalToWorld;
	return DebugDraw;
}
