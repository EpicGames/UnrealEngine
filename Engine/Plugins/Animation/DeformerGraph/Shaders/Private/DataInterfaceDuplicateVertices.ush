// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumDuplicateVertices;
uint {DataInterfaceName}_BaseVertexIndex;
Buffer<uint> {DataInterfaceName}_DuplicateVertexStartAndLength;
Buffer<uint> {DataInterfaceName}_DuplicateVertices;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumDuplicateVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumDuplicateVertices;
}

int2 ReadDuplicateVerticesStartAndLength_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DUPLICATED_VERTICES
	uint BufferIndex = (VertexIndex - {DataInterfaceName}_BaseVertexIndex) * 2;
	return int2({DataInterfaceName}_DuplicateVertexStartAndLength[BufferIndex  + 1], {DataInterfaceName}_DuplicateVertexStartAndLength[BufferIndex]);
#else	
	return 0;
#endif
}

uint ReadDuplicateVertex_{DataInterfaceName}(uint Index)
{
#if ENABLE_DUPLICATED_VERTICES
	return {DataInterfaceName}_DuplicateVertices[Index] + {DataInterfaceName}_BaseVertexIndex;
#else	
	return 0;
#endif
}
