// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumTriangles;
StructuredBuffer<int> {DataInterfaceName}_VertexToEdgeBuffer;
StructuredBuffer<int> {DataInterfaceName}_EdgeToTwinEdgeBuffer;

uint {DataInterfaceName}_bUseBufferFromRenderData;
StructuredBuffer<int> {DataInterfaceName}_RenderDataVertexToEdgeBuffer;
StructuredBuffer<int> {DataInterfaceName}_RenderDataEdgeToTwinEdgeBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumTriangles_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumTriangles;
}

int ReadEdge_{DataInterfaceName}(uint VertexIndex)
{
	return {DataInterfaceName}_bUseBufferFromRenderData == 0  ?
		{DataInterfaceName}_VertexToEdgeBuffer[VertexIndex] :
		{DataInterfaceName}_RenderDataVertexToEdgeBuffer[VertexIndex];
}

int ReadTwinEdge_{DataInterfaceName}(uint EdgeIndex)
{
	int TwinEdge = {DataInterfaceName}_bUseBufferFromRenderData == 0 ?
		{DataInterfaceName}_EdgeToTwinEdgeBuffer[EdgeIndex] :
		{DataInterfaceName}_RenderDataEdgeToTwinEdgeBuffer[EdgeIndex];
	
	return TwinEdge == -1 ? -1 : TwinEdge;
}
