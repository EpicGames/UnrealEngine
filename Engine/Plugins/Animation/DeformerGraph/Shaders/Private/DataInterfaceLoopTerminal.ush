// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_Index;
uint {DataInterfaceName}_Count;

uint ReadIndex_{DataInterfaceName}()
{
	return {DataInterfaceName}_Index;
}

uint ReadCount_{DataInterfaceName}()
{
	return {DataInterfaceName}_Count;
}

