// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
Buffer<float> {DataInterfaceName}_MorphBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float3 ReadDeltaPosition_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_MORPHTARGET
	uint BufferIndex = (VertexIndex) * 6;
	return float3({DataInterfaceName}_MorphBuffer[BufferIndex], {DataInterfaceName}_MorphBuffer[BufferIndex + 1], {DataInterfaceName}_MorphBuffer[BufferIndex + 2]);
#else
	return float3(0, 0, 0);
#endif
}

float3 ReadDeltaNormal_{DataInterfaceName}(uint VertexIndex)
{
#if ENABLE_DEFORMER_MORPHTARGET
	uint BufferIndex = (VertexIndex) * 6;
	return float3({DataInterfaceName}_MorphBuffer[BufferIndex + 3], {DataInterfaceName}_MorphBuffer[BufferIndex + 4], {DataInterfaceName}_MorphBuffer[BufferIndex + 5]);
#else
	return float3(0, 0, 0);
#endif
}
