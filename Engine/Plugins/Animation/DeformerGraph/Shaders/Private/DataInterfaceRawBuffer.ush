// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/ComputeFramework/Private/BufferAlias.ush"

// BUFFER_TYPE is type of underlying structure.
// PUBLIC_TYPE is the exposed type that calling code expects to use.
// Where BUFFER_TYPE is different to PUBLIC_TYPE we set BUFFER_TYPE_RAW and expect ReadRawBuffer() and WriteRawBuffer() to be implemented somewhere.
#define PUBLIC_TYPE {PublicType}
#define BUFFER_TYPE {BufferType}
#define BUFFER_TYPE_RAW {BufferTypeRaw}
// BUFFER_TYPE_SUPPORTS_ATOMIC is set if we can enable atomic operations on the PUBLIC_TYPE.
#define BUFFER_TYPE_SUPPORTS_ATOMIC {SupportAtomic}
// BUFFER_SPLIT_READ_WRITE is set if we can use an SRV to read the buffer. It is unset if we might need to read and write the buffer in the same kernel.
#define BUFFER_SPLIT_READ_WRITE {SplitReadWrite}

uint {DataInterfaceName}_BufferSize;
#if BUFFER_SPLIT_READ_WRITE
StructuredBuffer<BUFFER_TYPE> {DataInterfaceName}_BufferSRV;
#endif
RWStructuredBuffer<BUFFER_TYPE> {DataInterfaceName}_BufferUAV;

uint ReadNumValues_{DataInterfaceName}()
{
	return {DataInterfaceName}_BufferSize;
}

PUBLIC_TYPE ReadValue_{DataInterfaceName}(uint Index)
{
	uint BufferIndex = Index;
	PUBLIC_TYPE ReturnValue;
#if BUFFER_SPLIT_READ_WRITE
#if BUFFER_TYPE_RAW
	ReadRawBuffer({DataInterfaceName}_BufferSRV, BufferIndex, ReturnValue);
#else
	ReturnValue = {DataInterfaceName}_BufferSRV[BufferIndex];
#endif
#else
#if BUFFER_TYPE_RAW
	ReadRawBuffer({DataInterfaceName}_BufferUAV, BufferIndex, ReturnValue);
#else
	ReturnValue = {DataInterfaceName}_BufferUAV[BufferIndex];
#endif
#endif
	return ReturnValue;
}

PUBLIC_TYPE ReadValueUAV_{DataInterfaceName}(uint Index)
{
	uint BufferIndex = Index;
	PUBLIC_TYPE ReturnValue;

#if BUFFER_TYPE_RAW
	ReadRawBuffer({DataInterfaceName}_BufferUAV, BufferIndex, ReturnValue);
#else
	ReturnValue = {DataInterfaceName}_BufferUAV[BufferIndex];
#endif
	
	return ReturnValue;
}

void WriteValue_{DataInterfaceName}(uint Index, PUBLIC_TYPE Value)
{
	uint BufferIndex = Index;
#if BUFFER_TYPE_RAW
	WriteRawBuffer({DataInterfaceName}_BufferUAV, BufferIndex, Value);
#else
	{DataInterfaceName}_BufferUAV[BufferIndex] = Value;
#endif
}

PUBLIC_TYPE WriteAtomicAdd_{DataInterfaceName}(uint Index, PUBLIC_TYPE Value)
{
#if BUFFER_TYPE_SUPPORTS_ATOMIC
	uint BufferIndex = Index;
	PUBLIC_TYPE OldValue;
	InterlockedAdd({DataInterfaceName}_BufferUAV[BufferIndex], Value, OldValue);
	return OldValue;
#endif
	return 0;
}

PUBLIC_TYPE WriteAtomicMin_{DataInterfaceName}(uint Index, PUBLIC_TYPE Value)
{
#if BUFFER_TYPE_SUPPORTS_ATOMIC
	uint BufferIndex = Index;
	PUBLIC_TYPE OldValue;
	InterlockedMin({DataInterfaceName}_BufferUAV[BufferIndex], Value, OldValue);
	return OldValue;
#endif
	return 0;
}

PUBLIC_TYPE WriteAtomicMax_{DataInterfaceName}(uint Index, PUBLIC_TYPE Value)
{
#if BUFFER_TYPE_SUPPORTS_ATOMIC
	uint BufferIndex = Index;
	PUBLIC_TYPE OldValue;
	InterlockedMax({DataInterfaceName}_BufferUAV[BufferIndex], Value, OldValue);
	return OldValue;
#endif
	return 0;
}

#undef PUBLIC_TYPE
#undef BUFFER_TYPE
#undef BUFFER_TYPE_RAW
#undef BUFFER_TYPE_SUPPORTS_ATOMIC
#undef BUFFER_SPLIT_READ_WRITE
