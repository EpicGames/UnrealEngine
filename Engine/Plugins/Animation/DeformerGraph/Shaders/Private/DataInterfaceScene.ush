// Copyright Epic Games, Inc. All Rights Reserved.

float {DataInterfaceName}_GameTime;
float {DataInterfaceName}_GameTimeDelta;
uint {DataInterfaceName}_FrameNumber;

float ReadGameTime_{DataInterfaceName}()
{
	return {DataInterfaceName}_GameTime;
}

float ReadGameTimeDelta_{DataInterfaceName}()
{
	return {DataInterfaceName}_GameTimeDelta;
}

float ReadFrameNumber_{DataInterfaceName}()
{
	return {DataInterfaceName}_FrameNumber;
}
