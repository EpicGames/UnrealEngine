// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/Optimus/Private/DataInterfaceSkeletonCommon.ush"

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumBoneInfluences;
uint {DataInterfaceName}_InputWeightStride;
uint {DataInterfaceName}_InputWeightIndexSize;
FCompressedBoneTransformBuffer {DataInterfaceName}_BoneMatrices;
Buffer<uint> {DataInterfaceName}_InputWeightStream;
Buffer<uint> {DataInterfaceName}_InputWeightLookupStream;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumBones_{DataInterfaceName}(uint VertexIndex)
{
	return ReadNumBones(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_NumBoneInfluences,
		VertexIndex
		);
}

float3x4 ReadBoneMatrix_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
	return ReadBoneMatrix(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_BoneMatrices,
		VertexIndex,
		BoneIndex
		);
}

float ReadBoneWeight_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
	return ReadBoneWeight(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_NumBoneInfluences,	
		VertexIndex,
		BoneIndex
		);
}

float3x4 ReadWeightedBoneMatrix_{DataInterfaceName}(uint VertexIndex)
{
	return ReadWeightedBoneMatrix(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_NumBoneInfluences,
		{DataInterfaceName}_BoneMatrices,
		VertexIndex
		);
}
