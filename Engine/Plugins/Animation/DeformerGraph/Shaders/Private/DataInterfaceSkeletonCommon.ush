// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "/Engine/Private/GpuSkinCommon.ush"

uint ReadNumBones(
	Buffer<uint> InputWeightLookupStream,
	uint NumBoneInfluences,
	uint VertexIndex
	)
{
#if !ENABLE_DEFORMER_BONES
	return 1;
#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex = VertexIndex;
	uint BlendOffsetCount = InputWeightLookupStream[BufferIndex];
	return BlendOffsetCount & 0xff;
#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	return NumBoneInfluences;
#endif
}

float ReadBoneWeight(
	Buffer<uint> InputWeightLookupStream,
	uint InputWeightIndexSize,
	Buffer<uint> InputWeightStream,
	uint InputWeightStride,
	uint NumBoneInfluences,
	uint VertexIndex,
	uint BoneIndex
	)
{
#if !ENABLE_DEFORMER_BONES

	return 1;

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex 		= VertexIndex;	
	uint BlendOffsetCount = InputWeightLookupStream[BufferIndex];
	int NumBoneInfluencesLocal = BlendOffsetCount & 0xff;
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = InputWeightIndexSize & 0xff;
	int BoneWeightSize = InputWeightIndexSize >> 8;
	int WeightsOffset = StreamOffset + (BoneIndexSize * NumBoneInfluencesLocal);
	int BoneWeightOffset = WeightsOffset + BoneIndex * BoneWeightSize;
	
	if (BoneWeightSize > 1)
	{
		return float(InputWeightStream[BoneWeightOffset + 1] << 8 | InputWeightStream[BoneWeightOffset]) / 65535.0;
	}
	else
	{
		return float(InputWeightStream[BoneWeightOffset]) / 255.0f;
	}

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * (InputWeightStride / 4));
	float BlendWeight = GetBlendWeights(InputWeightStream, StreamOffset, BoneIndex / 4, NumBoneInfluences)[BoneIndex & 0x3];
	return BlendWeight;

#endif
}

float3x4 GetBoneMatrix(FCompressedBoneTransformBuffer BoneMatrices, uint BoneIndex)
{
	return LoadCompressedBoneTransformTransposed(BoneMatrices, 0, BoneIndex);
}

float3x4 ReadBoneMatrix(
	Buffer<uint> InputWeightLookupStream,
	uint InputWeightIndexSize,
	Buffer<uint> InputWeightStream,
	uint InputWeightStride,
	FCompressedBoneTransformBuffer BoneMatrices,	
	uint VertexIndex,
	uint BoneIndex
	)
{
#if !ENABLE_DEFORMER_BONES

	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex 		= VertexIndex;
	uint BlendOffsetCount = InputWeightLookupStream[BufferIndex];
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = InputWeightIndexSize & 0xff;
	int BoneIndexOffset = StreamOffset + (BoneIndexSize * BoneIndex);
	int BoneGlobalIndex = InputWeightStream[BoneIndexOffset];
	if (BoneIndexSize > 1)
	{
		BoneGlobalIndex = InputWeightStream[BoneIndexOffset + 1] << 8 | BoneGlobalIndex;
	}
	
	return GetBoneMatrix(BoneMatrices, BoneGlobalIndex);

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * (InputWeightStride / 4));
	int BlendIndex = GetBlendIndices(InputWeightStream ,StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return GetBoneMatrix(BoneMatrices,BlendIndex);

#endif
}

float3x4 ReadWeightedBoneMatrix(
	Buffer<uint> InputWeightLookupStream,
	uint InputWeightIndexSize,
	Buffer<uint> InputWeightStream,
	uint InputWeightStride,
	uint NumBoneInfluences,
	FCompressedBoneTransformBuffer BoneMatrices,
	uint VertexIndex
	)
{
#if !ENABLE_DEFORMER_BONES

	return float3x4(1, 0, 0, 0, 0, 1, 0, 0, 0, 0, 1, 0);

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE

	const uint BufferIndex 		= VertexIndex;
	const uint BlendOffsetCount = InputWeightLookupStream[BufferIndex];
	return ComputeBoneMatrixWithUnlimitedInfluences(BoneMatrices, InputWeightStream, InputWeightIndexSize, BlendOffsetCount);

#else

	const uint StreamOffset = (VertexIndex * (InputWeightStride / 4));
	const uint NumInfluences = NumBoneInfluences;

	FGPUSkinIndexAndWeight IndicesAndWeights = (FGPUSkinIndexAndWeight)0;
	if (NumInfluences > 0)
	{
		IndicesAndWeights.BlendIndices  = GetBlendIndices(InputWeightStream, StreamOffset, 0);
		IndicesAndWeights.BlendWeights  = GetBlendWeights(InputWeightStream, StreamOffset, 0, NumInfluences);
	}
	if (NumInfluences > 4)
	{
		IndicesAndWeights.BlendIndices2 = GetBlendIndices(InputWeightStream, StreamOffset, 1);
		IndicesAndWeights.BlendWeights2 = GetBlendWeights(InputWeightStream, StreamOffset, 1, NumInfluences);
	}
	return ComputeBoneMatrixWithLimitedInfluences(BoneMatrices, IndicesAndWeights, NumInfluences > 4);

#endif
}
