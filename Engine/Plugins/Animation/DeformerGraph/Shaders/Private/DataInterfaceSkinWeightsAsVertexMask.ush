// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/Optimus/Private/DataInterfaceSkeletonCommon.ush"

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumBoneInfluences;
uint {DataInterfaceName}_InputWeightStride;
uint {DataInterfaceName}_InputWeightIndexSize;
Buffer<uint> {DataInterfaceName}_InputWeightStream;
Buffer<uint> {DataInterfaceName}_InputWeightLookupStream;

StructuredBuffer<uint> {DataInterfaceName}_BoneIsSelected;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumBones_{DataInterfaceName}(uint VertexIndex)
{
	return ReadNumBones(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_NumBoneInfluences,
		VertexIndex
		);
}

float ReadBoneWeight_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
	return ReadBoneWeight(
		{DataInterfaceName}_InputWeightLookupStream,
		{DataInterfaceName}_InputWeightIndexSize,
		{DataInterfaceName}_InputWeightStream,
		{DataInterfaceName}_InputWeightStride,
		{DataInterfaceName}_NumBoneInfluences,	
		VertexIndex,
		BoneIndex
		);
}

uint ReadBoneIsSelected_{DataInterfaceName}(uint VertexIndex, uint BoneIndex)
{
#if !ENABLE_DEFORMER_BONES

	return 0;

#elif GPUSKIN_UNLIMITED_BONE_INFLUENCE
	const uint BufferIndex 		= VertexIndex;
	uint BlendOffsetCount = {DataInterfaceName}_InputWeightLookupStream[BufferIndex];
	int StreamOffset = BlendOffsetCount >> 8;
	int BoneIndexSize = {DataInterfaceName}_InputWeightIndexSize & 0xff;
	int BoneIndexOffset = StreamOffset + (BoneIndexSize * BoneIndex);
	int BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset];
	if (BoneIndexSize > 1)
	{
		BoneGlobalIndex = {DataInterfaceName}_InputWeightStream[BoneIndexOffset + 1] << 8 | BoneGlobalIndex;
	}
	
	return {DataInterfaceName}_BoneIsSelected[BoneGlobalIndex];

#else // !GPUSKIN_UNLIMITED_BONE_INFLUENCE
	
	uint StreamOffset = (VertexIndex * ({DataInterfaceName}_InputWeightStride / 4));
	int BlendIndex = GetBlendIndices({DataInterfaceName}_InputWeightStream, StreamOffset, BoneIndex / 4)[BoneIndex & 0x3];
	return {DataInterfaceName}_BoneIsSelected[BlendIndex];

#endif
}


float ReadMask_{DataInterfaceName}(uint VertexIndex)
{
#if !ENABLE_DEFORMER_BONES

	return 0;

#else

	float MaskValue = 0;
	
	uint NumBones = ReadNumBones_{DataInterfaceName}(VertexIndex);
	
	uint BoneIndex = 0;
	for (BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		float BoneWeight = ReadBoneWeight_{DataInterfaceName}(VertexIndex, BoneIndex);
		uint BoneIsSelected = ReadBoneIsSelected_{DataInterfaceName}(VertexIndex, BoneIndex);
		MaskValue += BoneIsSelected ? BoneWeight : 0;		
	}

	return MaskValue;
#endif
}
