// Copyright Epic Games, Inc. All Rights Reserved.

// Include required for platform TANGENT_RWBUFFER_FORMAT.
#include "/Engine/Private/GpuSkinCommon.ush"

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_NumTriangles;
uint {DataInterfaceName}_NumUVChannels;
Buffer<uint> {DataInterfaceName}_IndexBuffer;
Buffer<float> {DataInterfaceName}_PositionInputBuffer;
Buffer<SNORM float4> {DataInterfaceName}_TangentInputBuffer;
Buffer<float4> {DataInterfaceName}_ColorInputBuffer;
uint {DataInterfaceName}_ColorIndexMask;
Buffer<float2> {DataInterfaceName}_UVInputBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

uint ReadNumTriangles_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumTriangles;
}

uint ReadNumUVChannels_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumUVChannels;
}

uint ReadIndexBuffer_{DataInterfaceName}(uint Index)
{
	uint BufferIndex = Index;
	return {DataInterfaceName}_IndexBuffer[BufferIndex];
}

float3 ReadPosition_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = ( VertexIndex) * 3;
	return float3({DataInterfaceName}_PositionInputBuffer[BufferIndex], {DataInterfaceName}_PositionInputBuffer[BufferIndex + 1], {DataInterfaceName}_PositionInputBuffer[BufferIndex + 2]);
}

float4 ReadTangentX_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = (VertexIndex) * 2;
	return TangentBias_SkinCache({DataInterfaceName}_TangentInputBuffer[BufferIndex]);
}

float4 ReadTangentZ_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = (VertexIndex) * 2;
	return TangentBias_SkinCache({DataInterfaceName}_TangentInputBuffer[BufferIndex + 1]);
}

float4 ReadColor_{DataInterfaceName}(uint VertexIndex)
{
	// Index Mask is used here in case ColorStaticBuffer is bound to the GWhiteVertexBufferWithSRV that is only a few bytes
	uint BufferIndex = (VertexIndex) & ({DataInterfaceName}_ColorIndexMask);
	return {DataInterfaceName}_ColorInputBuffer[BufferIndex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
}

float2 ReadUV_{DataInterfaceName}(uint VertexIndex, uint UVChannelIndex)
{
	uint BufferIndex = (VertexIndex) * {DataInterfaceName}_NumUVChannels + UVChannelIndex;
	return {DataInterfaceName}_UVInputBuffer[BufferIndex];
}
