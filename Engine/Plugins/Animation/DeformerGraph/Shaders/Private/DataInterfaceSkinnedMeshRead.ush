// Copyright Epic Games, Inc. All Rights Reserved.

// Include required for platform TANGENT_RWBUFFER_FORMAT.
#include "/Engine/Private/GpuSkinCommon.ush"

#if OPENGL_PROFILE
// On OpenGL, deformer/skin cache tangents are stored as SINT [-32767, 32767], instead of SNORM [-1.0,1.0], see TangentUnbias_SkinCache
#define TangentRead_SkinCache(X)	(X / 32767.0)
#else
#define TangentRead_SkinCache(X)	(X)
#endif // OPENGL_PROFILE

uint {DataInterfaceName}_NumVertices;

Buffer<float> {DataInterfaceName}_PositionBufferSRV;
Buffer<TANGENT_RWBUFFER_FORMAT> {DataInterfaceName}_TangentBufferSRV;
Buffer<UNORM float4> {DataInterfaceName}_ColorBufferSRV;

Buffer<float> {DataInterfaceName}_PositionStaticBuffer;
Buffer<SNORM float4> {DataInterfaceName}_TangentStaticBuffer;
Buffer<float4> {DataInterfaceName}_ColorStaticBuffer;
uint {DataInterfaceName}_ColorIndexMask;
uint {DataInterfaceName}_ReadableOutputBuffers;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float3 ReadPosition_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = (VertexIndex) * 3;

	if ({DataInterfaceName}_ReadableOutputBuffers & OPTIMUS_SKINNED_MESH_READ_POSITION)
	{
		return float3(
			{DataInterfaceName}_PositionBufferSRV[BufferIndex + 0], 
			{DataInterfaceName}_PositionBufferSRV[BufferIndex + 1],
			{DataInterfaceName}_PositionBufferSRV[BufferIndex + 2]);
	}
	else
	{
		return float3(
			{DataInterfaceName}_PositionStaticBuffer[BufferIndex + 0], 
			{DataInterfaceName}_PositionStaticBuffer[BufferIndex + 1],
			{DataInterfaceName}_PositionStaticBuffer[BufferIndex + 2]);
	}
}

float4 ReadTangentX_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = (VertexIndex) * 2;
	
	if ({DataInterfaceName}_ReadableOutputBuffers & OPTIMUS_SKINNED_MESH_READ_TANGENTS)
	{
		return TangentRead_SkinCache({DataInterfaceName}_TangentBufferSRV[BufferIndex]) ;
	}
	else
	{
		return TangentBias_SkinCache({DataInterfaceName}_TangentStaticBuffer[BufferIndex]) ;
	}
}

float4 ReadTangentZ_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = (VertexIndex) * 2;

	if ({DataInterfaceName}_ReadableOutputBuffers & OPTIMUS_SKINNED_MESH_READ_TANGENTS)
	{
		return TangentRead_SkinCache({DataInterfaceName}_TangentBufferSRV[BufferIndex + 1]) ;
	}
	else
	{
		return TangentBias_SkinCache({DataInterfaceName}_TangentStaticBuffer[BufferIndex + 1]) ;
	}
}

float4 ReadColor_{DataInterfaceName}(uint VertexIndex)
{
	// Index Mask is used here in case ColorStaticBuffer is bound to the GWhiteVertexBufferWithSRV that is only a few bytes
	uint BufferIndex = (VertexIndex) & ({DataInterfaceName}_ColorIndexMask);

	if ({DataInterfaceName}_ReadableOutputBuffers & OPTIMUS_SKINNED_MESH_READ_COLOR)
	{
		return {DataInterfaceName}_ColorBufferSRV[BufferIndex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	}
	else
	{
		return {DataInterfaceName}_ColorStaticBuffer[BufferIndex] FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
	}
}


