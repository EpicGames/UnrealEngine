// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_bIsValid;
float {DataInterfaceName}_DefaultValue;
Buffer<float> {DataInterfaceName}_ValueBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float ReadValue_{DataInterfaceName}(uint VertexIndex)
{
	if ({DataInterfaceName}_bIsValid)
	{
		uint BufferIndex = (VertexIndex);
		return {DataInterfaceName}_ValueBuffer[BufferIndex];
	}
	else
	{
		return {DataInterfaceName}_DefaultValue;
	}
}
