// Copyright Epic Games, Inc. All Rights Reserved.

// Include required for platform TANGENT_RWBUFFER_FORMAT.
#include "/Engine/Private/GpuSkinCommon.ush"

uint {DataInterfaceName}_NumVertices;
RWBuffer<float> {DataInterfaceName}_PositionBufferUAV;
RWBuffer<TANGENT_RWBUFFER_FORMAT> {DataInterfaceName}_TangentBufferUAV;
RWBuffer<UNORM float4> {DataInterfaceName}_ColorBufferUAV;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

void WritePosition_{DataInterfaceName}(uint VertexIndex, float3 Position)
{
	uint BufferIndex = (VertexIndex) * 3;
	{DataInterfaceName}_PositionBufferUAV[BufferIndex + 0] = Position.x;
	{DataInterfaceName}_PositionBufferUAV[BufferIndex + 1] = Position.y;
	{DataInterfaceName}_PositionBufferUAV[BufferIndex + 2] = Position.z;
}

void WriteTangentX_{DataInterfaceName}(uint VertexIndex, float4 TangentX)
{
	uint BufferIndex = (VertexIndex) * 2;
	{DataInterfaceName}_TangentBufferUAV[BufferIndex + 0] = TangentUnbias_SkinCache(TangentX);
}

void WriteTangentZ_{DataInterfaceName}(uint VertexIndex, float4 TangentZ)
{
	uint BufferIndex = (VertexIndex) * 2;
	{DataInterfaceName}_TangentBufferUAV[BufferIndex + 1] = TangentUnbias_SkinCache(TangentZ);
}

void WriteColor_{DataInterfaceName}(uint VertexIndex, float4 Color)
{
	uint BufferIndex = VertexIndex;
	{DataInterfaceName}_ColorBufferUAV[BufferIndex] = Color FMANUALFETCH_COLOR_COMPONENT_SWIZZLE;
}
