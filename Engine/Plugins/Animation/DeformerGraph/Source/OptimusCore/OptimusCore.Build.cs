// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
namespace UnrealBuildTool.Rules
{
    public class OptimusCore : ModuleRules
    {
        public OptimusCore(ReadOnlyTargetRules Target) : base(Target)
		{
			NumIncludedBytesPerUnityCPPOverride = 688128; // best unity size found from using UBT ProfileUnitySizes mode

			PrivateIncludePaths.Add(Path.Combine(EngineDirectory, "Source/Runtime/Renderer/Private"));

            PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"SlateCore"
				}
			);

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"ComputeFramework",
					"Core",
					"CoreUObject",
					"Engine",
					"OptimusSettings",
					"Projects",
					"RenderCore",
					"Renderer",
					"RHI",
					"RigVM",
					"ControlRig",
					"StaticMeshDescription",
					"MeshDescription"
				}
			);

            if (Target.bBuildEditor == true)
            {
	            PrivateDependencyModuleNames.AddRange(
		            new string[]
		            {
			            "RigVMDeveloper",
		            }
	            );
            }
        }
    }
}
