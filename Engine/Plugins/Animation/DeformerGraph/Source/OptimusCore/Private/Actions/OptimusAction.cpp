// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusAction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusAction)


FOptimusAction::FOptimusAction(const FString& InTitle /*= {}*/) :
	Title(InTitle)
{

}


FOptimusAction::~FOptimusAction()
{

}



TWeakPtr<FOptimusAction> FOptimusCompoundAction::AddSubAction(FOptimusAction* InAction)
{
	AddSubAction(TSharedPtr<FOptimusAction>(InAction));

	return SubActions.Last().ToWeakPtr();
	
}


bool FOptimusCompoundAction::Do(IOptimusPathResolver* InRoot)
{
	for (int32 ActionIndex = 0; ActionIndex < SubActions.Num(); ActionIndex++)
	{
		if (!SubActions[ActionIndex]->Do(InRoot))
		{
			// If the action failed, then walk backwards and try to recover our state so we're
			// not in some sort of a half-constructed scenario.
			for (; ActionIndex-- > 0; /**/)
			{
				// We don't care if they fail at this point. We're stepping on the emergency
				// brakes at this point.
				SubActions[ActionIndex]->Undo(InRoot);
			}

			return false;
		}
	}
	return true;
}


bool FOptimusCompoundAction::Undo(IOptimusPathResolver* InRoot)
{
	for (int32 ActionIndex = SubActions.Num(); ActionIndex-- > 0; /**/)
	{
		if (!SubActions[ActionIndex]->Undo(InRoot))
		{
			// Try to recover from failure as above.
			for (ActionIndex++; ActionIndex < SubActions.Num(); ActionIndex++)
			{
				SubActions[ActionIndex]->Do(InRoot);
			}

			return false;
		}
	}
	return true;
}
