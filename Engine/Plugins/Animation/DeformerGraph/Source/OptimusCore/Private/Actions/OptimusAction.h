// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#include "OptimusAction.generated.h"

class IOptimusPathResolver;

// Base action class. This is a UStruct so that we can use UE's RAII to check for type
// similarity.
USTRUCT()
struct FOptimusAction
{
	GENERATED_BODY()

	FOptimusAction(const FString& InTitle = {});
	virtual ~FOptimusAction();

	const FString& GetTitle() const 
	{
		return Title;
	}

	void SetTitle(const FString& InTitle)
	{
		Title = InTitle;
	}

	template <typename... ArgTypes>
	void SetTitlef(::UE::Core::TCheckedFormatString<FString::FmtCharType, ArgTypes...> Fmt, ArgTypes... Args)
	{
		Title = FString::Printf(Fmt, Args...);
	}

protected:
	friend class UOptimusActionStack;
	friend struct FOptimusCompoundAction;

	/// Performs the action as set by the action's constructor.
	virtual bool Do(IOptimusPathResolver* InRoot) PURE_VIRTUAL(FOptimusAction::Do, return false; );

	/// Reverts the action performed by the action's Do function.
	virtual bool Undo(IOptimusPathResolver* InRoot) PURE_VIRTUAL(FOptimusAction::Undo, return false; );

private:
	/// The title of the action. Should be set by the constructor of the derived objects.
	FString Title;
};

// Mark FOptimusAction as pure virtual, so that the UObject machinery won't attempt to instantiate it.
template<>
struct TStructOpsTypeTraits<FOptimusAction> :
TStructOpsTypeTraitsBase2<FOptimusAction>
{
	enum
	{
		WithPureVirtual = true,
    };
};


USTRUCT()
struct FOptimusCompoundAction :
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusCompoundAction(const FString& InTitle = {}) :
		FOptimusAction(InTitle)
	{}

	// Copy nothing.
	FOptimusCompoundAction(const FOptimusCompoundAction &) {}
	FOptimusCompoundAction &operator=(const FOptimusCompoundAction &) { return *this; }

	FOptimusCompoundAction(FOptimusCompoundAction&&) = default;
	FOptimusCompoundAction& operator=(FOptimusCompoundAction&&) = default;

	template <typename... ArgTypes>
	FOptimusCompoundAction(::UE::Core::TCheckedFormatString<FString::FmtCharType, ArgTypes...> Fmt, ArgTypes ...Args) :
		FOptimusAction(FString::Printf(Fmt, Args...))
	{ }

	/// Add a sub-action from a heap-constructed action. This takes ownership of the pointer.
	/// @param InAction The action to add.
	/// Returns a weakptr for subsequent actions reference the result of this action
	TWeakPtr<FOptimusAction> AddSubAction(FOptimusAction* InAction);

	template<typename T, typename... ArgTypes>
	typename TEnableIf<TPointerIsConvertibleFromTo<T, FOptimusAction>::Value, TWeakPtr<T>>::Type 
	AddSubAction(ArgTypes&& ...Args)
	{
		TSharedPtr<T> Action = MakeShared<T>(Forward<ArgTypes>(Args)...);
		SubActions.Add(Action);
		return Action.ToWeakPtr();
	}

	bool HasSubActions() const
	{
		return SubActions.Num() != 0;
	}

protected:
	friend class UOptimusActionStack;
	void AddSubAction(TSharedPtr<FOptimusAction> InAction)
	{
		SubActions.Add(InAction);
	}

	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	TArray<TSharedPtr<FOptimusAction>> SubActions;
};
