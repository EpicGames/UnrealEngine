// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusComponentBindingActions.h"

#include "IOptimusPathResolver.h"
#include "OptimusComponentSource.h"
#include "OptimusDeformer.h"
#include "OptimusHelpers.h"

#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h" 

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusComponentBindingActions)


FOptimusComponentBindingAction_AddBinding::FOptimusComponentBindingAction_AddBinding(
	const UOptimusComponentSource* InComponentSource,
	FName InName
	)
{
	if (ensure(InComponentSource))
	{
		ComponentSourceClassPath = InComponentSource->GetClass()->GetStructPathName();
		ComponentBindingName = InName;

		SetTitlef(TEXT("Add component binding '%s'"), *ComponentBindingName.ToString());
	}
}


UOptimusComponentSourceBinding* FOptimusComponentBindingAction_AddBinding::GetComponentBinding(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveComponentBinding(ComponentBindingName);
}


bool FOptimusComponentBindingAction_AddBinding::Do(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);

	const UClass* ComponentSourceClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), ComponentSourceClassPath, EFindObjectFlags::None));
	if (!ComponentSourceClass)
	{
		return false;
	}
	UOptimusComponentSource* ComponentSource = ComponentSourceClass->GetDefaultObject<UOptimusComponentSource>();
	if (!ComponentSource)
	{
		return false;
	}
	
	UOptimusComponentSourceBinding* Binding = Deformer->CreateComponentBindingDirect(ComponentSource, ComponentBindingName);
	if (!Binding)
	{
		return false;
	}

	if (!Deformer->AddComponentBindingDirect(Binding, INDEX_NONE))
	{
		Optimus::RemoveObject(Binding);
		return false;
	}

	return true;
}

bool FOptimusComponentBindingAction_AddBinding::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusComponentSourceBinding* Binding = GetComponentBinding(InRoot);
	if (!Binding)
	{
		return false;
	}

	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	return Deformer->RemoveComponentBindingDirect(Binding);
}



FOptimusComponentBindingAction_RemoveBinding::FOptimusComponentBindingAction_RemoveBinding(
	UOptimusComponentSourceBinding* InBinding
	)
{
	if (ensure(InBinding))
	{
		ComponentSourceClassPath = InBinding->GetComponentSource()->GetClass()->GetStructPathName();
		ComponentBindingName = InBinding->BindingName;
		ComponentTags = InBinding->ComponentTags;
		ComponentBindingIndex = InBinding->GetIndex();

		SetTitlef(TEXT("Remove component binding '%s'"), *ComponentBindingName.ToString());
	}
}


bool FOptimusComponentBindingAction_RemoveBinding::Do(IOptimusPathResolver* InRoot)
{
	UOptimusComponentSourceBinding* Binding = InRoot->ResolveComponentBinding(ComponentBindingName);
	if (!Binding)
	{
		return false;
	}

	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	return Deformer->RemoveComponentBindingDirect(Binding);
}


bool FOptimusComponentBindingAction_RemoveBinding::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);

	const UClass* ComponentSourceClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), ComponentSourceClassPath, EFindObjectFlags::None));
	if (!ComponentSourceClass)
	{
		return false;
	}
	UOptimusComponentSource* ComponentSource = ComponentSourceClass->GetDefaultObject<UOptimusComponentSource>();
	if (!ComponentSource)
	{
		return false;
	}
	
	UOptimusComponentSourceBinding* Binding = Deformer->CreateComponentBindingDirect(ComponentSource, ComponentBindingName);
	if (!Binding)
	{
		return false;
	}

	Binding->ComponentTags = ComponentTags;

	if (!Deformer->AddComponentBindingDirect(Binding, ComponentBindingIndex))
	{
		Optimus::RemoveObject(Binding);
		return false;
	}

	return true;
}


FOptimusComponentBindingAction_RenameBinding::FOptimusComponentBindingAction_RenameBinding(
	UOptimusComponentSourceBinding* InBinding,
	FName InNewName
	)
{
	if (ensure(InBinding))
	{
		OldName = InBinding->GetFName();
		NewName = InNewName;

		SetTitlef(TEXT("Rename component binding to '%s'"), *NewName.ToString());
	}
}


bool FOptimusComponentBindingAction_RenameBinding::Do(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	UOptimusComponentSourceBinding* Binding = Deformer->ResolveComponentBinding(OldName);

	return Binding && Deformer->RenameComponentBindingDirect(Binding, NewName);
}


bool FOptimusComponentBindingAction_RenameBinding::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	UOptimusComponentSourceBinding* Binding = Deformer->ResolveComponentBinding(NewName);

	return Binding && Deformer->RenameComponentBindingDirect(Binding, OldName);
}



FOptimusComponentBindingAction_SetComponentSource::FOptimusComponentBindingAction_SetComponentSource(
	UOptimusComponentSourceBinding* InComponentBinding, 
	const UOptimusComponentSource* InComponentSource
	)
{
	if (ensure(InComponentBinding) && ensure(InComponentSource))
	{
		BindingName = InComponentBinding->BindingName;
		NewComponentSourceClassPath = InComponentSource->GetClass()->GetStructPathName();
		OldComponentSourceClassPath = InComponentBinding->ComponentType->GetStructPathName();

		SetTitlef(TEXT("Set Component source to '%s'"), *InComponentSource->GetBindingName().ToString());
	}
}


bool FOptimusComponentBindingAction_SetComponentSource::Do(IOptimusPathResolver* InRoot)
{
	return SetComponentSource(InRoot, NewComponentSourceClassPath);
}


bool FOptimusComponentBindingAction_SetComponentSource::Undo(IOptimusPathResolver* InRoot)
{
	return SetComponentSource(InRoot, OldComponentSourceClassPath);
}


bool FOptimusComponentBindingAction_SetComponentSource::SetComponentSource(
	IOptimusPathResolver* InRoot,
	FTopLevelAssetPath InComponentSourceClassPath
	) const
{
	const UClass* ComponentSourceClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), InComponentSourceClassPath, EFindObjectFlags::None));
	if (!ComponentSourceClass)
	{
		return false;
	}
	const UOptimusComponentSource* ComponentSource = ComponentSourceClass->GetDefaultObject<UOptimusComponentSource>();
	if (!ComponentSource)
	{
		return false;
	}

	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	UOptimusComponentSourceBinding* Binding = Deformer->ResolveComponentBinding(BindingName);
	return Binding && Deformer->SetComponentBindingSourceDirect(Binding, ComponentSource);
}

