// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusAction.h"

#include "OptimusComponentBindingActions.generated.h"


class UOptimusComponentSource;
class UOptimusComponentSourceBinding;


USTRUCT()
struct FOptimusComponentBindingAction_AddBinding : 
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusComponentBindingAction_AddBinding() = default;

	FOptimusComponentBindingAction_AddBinding(
		const UOptimusComponentSource *InComponentSource,
		FName InName
		);

	UOptimusComponentSourceBinding* GetComponentBinding(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the component binding to create.
	FName ComponentBindingName;

	// The class path of the component source to use.
	FTopLevelAssetPath ComponentSourceClassPath;
};


USTRUCT()
struct FOptimusComponentBindingAction_RemoveBinding : 
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusComponentBindingAction_RemoveBinding() = default;

	FOptimusComponentBindingAction_RemoveBinding(
	    UOptimusComponentSourceBinding* InBinding
	    );

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the component binding to remove.
	FName ComponentBindingName;
	
	// The class path of the component source to recreate on undo.
	FTopLevelAssetPath ComponentSourceClassPath;

	// Stored component tags
	TArray<FName> ComponentTags;

	// The index of the binding in the list.
	int32 ComponentBindingIndex = 0;
};


USTRUCT()
struct FOptimusComponentBindingAction_RenameBinding : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusComponentBindingAction_RenameBinding() = default;

	FOptimusComponentBindingAction_RenameBinding(
		UOptimusComponentSourceBinding* InBinding,
		FName InNewName);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The new name to give the component binding.
	FName NewName;

	// The old name of the component binding.
	FName OldName;
};


USTRUCT()
struct FOptimusComponentBindingAction_SetComponentSource : 
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusComponentBindingAction_SetComponentSource() = default;

	FOptimusComponentBindingAction_SetComponentSource(
		UOptimusComponentSourceBinding* InComponentBinding,
		const UOptimusComponentSource* InComponentSource
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	bool SetComponentSource(
		IOptimusPathResolver* InRoot, 
		FTopLevelAssetPath InComponentSourceClassPath
		) const;

	// The name of the variable to update.
	FName BindingName;

	// The class path of the component source to set
	FTopLevelAssetPath NewComponentSourceClassPath;

	// The class path of the old component source to set on undo.
	FTopLevelAssetPath OldComponentSourceClassPath;
};
