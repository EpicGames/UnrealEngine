// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodeActions.h"

#include "IOptimusPathResolver.h"
#include "OptimusHelpers.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "IOptimusNodeAdderPinProvider.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusNode_Comment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodeActions)


FOptimusNodeAction_RenameNode::FOptimusNodeAction_RenameNode(
	UOptimusNode* InNode, 
	FText InNewName
	)
{
	NodePath = InNode->GetNodePath();
	NewName = InNewName;
	OldName = InNode->GetDisplayName();

	SetTitlef(TEXT("Rename %s"), *InNode->GetDisplayName().ToString());
}


bool FOptimusNodeAction_RenameNode::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNode *Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	Node->SetDisplayName(NewName);

	return true;
}


bool FOptimusNodeAction_RenameNode::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	Node->SetDisplayName(OldName);

	return true;
}


FOptimusNodeAction_MoveNode::FOptimusNodeAction_MoveNode(
	UOptimusNode* InNode,
	const FVector2D& InPosition
)
{
	NodePath = InNode->GetNodePath();
	NewPosition = InPosition;
	OldPosition = InNode->GetGraphPosition();
}

bool FOptimusNodeAction_MoveNode::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	return Node->SetGraphPositionDirect(NewPosition);
}

bool FOptimusNodeAction_MoveNode::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	return Node->SetGraphPositionDirect(OldPosition);
}



FOptimusCommentNodeAction_ResizeNode::FOptimusCommentNodeAction_ResizeNode(
	UOptimusNode_Comment* InNode,
	const UE::Slate::FDeprecateVector2DParameter& InNewSize)
{
	NodePath = InNode->GetNodePath();
	OldSize = InNode->GetSize();	
	NewSize = InNewSize;
	
	SetTitlef(TEXT("Resize Comment Node"));
}

bool FOptimusCommentNodeAction_ResizeNode::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNode_Comment* Node = Cast<UOptimusNode_Comment>(InRoot->ResolveNodePath(NodePath));
	if (!Node)
	{
		return false;
	}

	return Node->SetSizeDirect(NewSize);
}

bool FOptimusCommentNodeAction_ResizeNode::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusNode_Comment* Node = Cast<UOptimusNode_Comment>(InRoot->ResolveNodePath(NodePath));
	if (!Node)
	{
		return false;
	}

	return Node->SetSizeDirect(OldSize);
}


FOptimusNodeAction_SetPinValue::FOptimusNodeAction_SetPinValue(
	UOptimusNodePin* InPin, 
	const FString& InNewValue
	)
{
	if (ensure(InPin) && InPin->GetSubPins().IsEmpty())
	{
		PinPath = InPin->GetPinPath();
		OldValue = InPin->GetValueAsString();
		NewValue = InNewValue;

		SetTitlef(TEXT("Set Value %s"), *InPin->GetPinPath());
	}
}


bool FOptimusNodeAction_SetPinValue::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNodePin* Pin = InRoot->ResolvePinPath(PinPath);
	if (!Pin)
	{
		return false;
	}

	return Pin->SetValueFromStringDirect(NewValue);
}


bool FOptimusNodeAction_SetPinValue::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusNodePin* Pin = InRoot->ResolvePinPath(PinPath);
	if (!Pin)
	{
		return false;
	}

	return Pin->SetValueFromStringDirect(OldValue);
}


FOptimusNodeAction_SetPinName::FOptimusNodeAction_SetPinName(
	UOptimusNodePin* InPin,
	FName InPinName
	)
{
	if (ensure(InPin))
	{
		PinPath = InPin->GetPinPath();
		NewPinName = InPinName;
		OldPinName = InPin->GetFName();
	}
}


bool FOptimusNodeAction_SetPinName::Do(IOptimusPathResolver* InRoot)
{
	return SetPinName(InRoot, NewPinName);
}


bool FOptimusNodeAction_SetPinName::Undo(IOptimusPathResolver* InRoot)
{
	return SetPinName(InRoot, OldPinName);
}


bool FOptimusNodeAction_SetPinName::SetPinName(
	IOptimusPathResolver* InRoot, 
	FName InName
	)
{
	UOptimusNodePin *Pin = InRoot->ResolvePinPath(PinPath);

	if (!Pin)
	{
		return false;
	}
	
	if (!Pin->GetOwningNode()->SetPinNameDirect(Pin, InName))
	{
		return false;
	}

	PinPath = Pin->GetPinPath();
	return true;
}


FOptimusNodeAction_SetPinType::FOptimusNodeAction_SetPinType(
	UOptimusNodePin* InPin,
	FOptimusDataTypeRef InDataType
	)
{
	if (ensure(InPin))
	{
		PinPath = InPin->GetPinPath();
		NewDataTypeName = InDataType.TypeName;
		OldDataTypeName = InPin->GetDataType()->TypeName;
	}
}


bool FOptimusNodeAction_SetPinType::Do(IOptimusPathResolver* InRoot)
{
	return SetPinType(InRoot, NewDataTypeName);
}


bool FOptimusNodeAction_SetPinType::Undo(IOptimusPathResolver* InRoot)
{
	return SetPinType(InRoot, OldDataTypeName);
}


bool FOptimusNodeAction_SetPinType::SetPinType(
	IOptimusPathResolver* InRoot,
	FName InDataType
	) const
{
	UOptimusNodePin *Pin = InRoot->ResolvePinPath(PinPath);

	if (!Pin)
	{
		return false;
	}

	FOptimusDataTypeRef DataType(FOptimusDataTypeRegistry::Get().FindType(InDataType));
	
	return Pin->GetOwningNode()->SetPinDataTypeDirect(Pin, DataType);
}


FOptimusNodeAction_SetPinDataDomain::FOptimusNodeAction_SetPinDataDomain(
	const UOptimusNodePin* InPin,
	const FOptimusDataDomain& InDataDomain
	)
{
	if (ensure(InPin))
	{
		PinPath = InPin->GetPinPath();
		NewDataDomain = InDataDomain;
		OldDataDomain = InPin->GetDataDomain();
	}
}


bool FOptimusNodeAction_SetPinDataDomain::Do(IOptimusPathResolver* InRoot)
{
	return SetPinDataDomain(InRoot, NewDataDomain);
}


bool FOptimusNodeAction_SetPinDataDomain::Undo(IOptimusPathResolver* InRoot)
{
	return SetPinDataDomain(InRoot, OldDataDomain);
}


bool FOptimusNodeAction_SetPinDataDomain::SetPinDataDomain(
	IOptimusPathResolver* InRoot,
	const FOptimusDataDomain& InDataDomain
	) const
{
	UOptimusNodePin *Pin = InRoot->ResolvePinPath(PinPath);

	if (!Pin)
	{
		return false;
	}

	return Pin->GetOwningNode()->SetPinDataDomainDirect(Pin, InDataDomain);
}


FOptimusNodeAction_AddRemovePin::FOptimusNodeAction_AddRemovePin(
	UOptimusNode* InNode,
	FName InName,
	EOptimusNodePinDirection InDirection,
	const FOptimusDataDomain& InDataDomain,
	FOptimusDataTypeRef InDataType,
	const UOptimusNodePin* InBeforePin,
	const UOptimusNodePin* InParentPin
	)
{
	if (ensure(InNode) &&
		ensure(!InBeforePin || InBeforePin->GetOwningNode() == InNode) &&
		ensure(!InBeforePin || InBeforePin->GetParentPin() == InParentPin) &&
		ensure(!InParentPin || InParentPin->GetOwningNode() == InNode) &&
		ensure(!InParentPin || InParentPin->IsGroupingPin()))
	{
		NodePath = InNode->GetNodePath();
		PinName = InName;
		Direction = InDirection;
		DataDomain = InDataDomain;
		DataType = InDataType.TypeName;

		// New pins are always created in a non-expanded state.
		bExpanded = false;

		if (InBeforePin)
		{
			BeforePinPath = InBeforePin->GetPinPath();
		}
		if (InParentPin)
		{
			ParentPinPath = InParentPin->GetPinPath();
		}
	}
}

FOptimusNodeAction_AddRemovePin::FOptimusNodeAction_AddRemovePin(
	UOptimusNode* InNode,
	FName InName,
	EOptimusNodePinDirection InDirection,
	const UOptimusNodePin* InBeforePin
	)
{
	if (ensure(InNode) &&
		ensure(!InBeforePin || InBeforePin->GetOwningNode() == InNode))
	{
		NodePath = InNode->GetNodePath();
		PinName = InName;
		Direction = InDirection;
		bIsGroupingPin = true;

		// New pins are always created in a non-expanded state.
		bExpanded = false;

		if (InBeforePin)
		{
			BeforePinPath = InBeforePin->GetPinPath();
		}
	}
}


FOptimusNodeAction_AddRemovePin::FOptimusNodeAction_AddRemovePin(UOptimusNodePin* InPin)
{
	if (ensure(InPin))
	{
		NodePath = InPin->GetOwningNode()->GetNodePath();
		PinPath = InPin->GetPinPath();
		PinName = InPin->GetFName();
		Direction = InPin->GetDirection();
		bIsGroupingPin = InPin->IsGroupingPin();

		if (!bIsGroupingPin)
		{
			DataDomain = InPin->GetDataDomain();
			DataType = InPin->GetDataType()->TypeName;
		}

		// Store the expansion info.
		bExpanded = InPin->GetIsExpanded();

		// Capture the before and parent pin (if any).
		TArrayView<UOptimusNodePin* const> Pins;
		if (InPin->GetParentPin())
		{
			Pins = InPin->GetParentPin()->GetSubPins();

			ParentPinPath = InPin->GetParentPin()->GetPinPath();
		}
		else
		{
			Pins = InPin->GetOwningNode()->GetPins();
		}
		const int32 PinIndex = Pins.IndexOfByKey(InPin);
		if (ensure(Pins.IsValidIndex(PinIndex)))
		{
			// If it's not the last, then grab the path of the next pin.
			if (PinIndex < (Pins.Num() - 1))
			{
				BeforePinPath = Pins[PinIndex + 1]->GetPinPath();
			}
		}
	}
}


bool FOptimusNodeAction_AddRemovePin::AddPin(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	UOptimusNodePin *BeforePin = nullptr;
	if (!BeforePinPath.IsEmpty())
	{
		BeforePin = InRoot->ResolvePinPath(BeforePinPath);
		if (!BeforePin)
		{
			return false;
		}
	}
	UOptimusNodePin *ParentPin = nullptr;
	if (!ParentPinPath.IsEmpty())
	{
		ParentPin = InRoot->ResolvePinPath(ParentPinPath);
		if (!ParentPin)
		{
			return false;
		}
	}

	UOptimusNodePin *Pin;
	if (bIsGroupingPin)
	{
		Pin = Node->AddGroupingPinDirect(PinName, Direction, BeforePin);
	}
	else
	{
		const FOptimusDataTypeRef TypeRef = FOptimusDataTypeRegistry::Get().FindType(DataType);
		if (!TypeRef.IsValid())
		{
			return false;
		}
		
		Pin = Node->AddPinDirect(PinName, Direction, DataDomain, TypeRef, BeforePin, ParentPin);
	}
	
	if (!Pin)
	{
		return false;
	}

	Pin->SetIsExpanded(bExpanded);

	PinName = Pin->GetFName();
	PinPath = Pin->GetPinPath();

	return true;
}


bool FOptimusNodeAction_AddRemovePin::RemovePin(IOptimusPathResolver* InRoot) const
{
	UOptimusNodePin *Pin = InRoot->ResolvePinPath(PinPath);
	if (!Pin)
	{
		return false;
	}
	UOptimusNode *Node = Pin->GetOwningNode();

	return Node->RemovePinDirect(Pin);
}


UOptimusNodePin* FOptimusNodeAction_AddPin::GetPin(IOptimusPathResolver* InRoot) const
{
	return InRoot->ResolvePinPath(PinPath);
}

UOptimusNodePin* FOptimusNodeAction_AddGroupingPin::GetPin(IOptimusPathResolver* InRoot) const
{
	return InRoot->ResolvePinPath(PinPath);
}



FOptimusNodeAction_MovePin::FOptimusNodeAction_MovePin(
	UOptimusNodePin* InPinToMove,
	const UOptimusNodePin* InPinBefore
	)
{
	if (ensure(InPinToMove))
	{
		PinPath = InPinToMove->GetPinPath();

		const UOptimusNodePin* NextPin = InPinToMove->GetNextPin();
		if (NextPin)
		{
			OldBeforePinPath = NextPin->GetPinPath(); 
		}

		if (InPinBefore)
		{
			NewBeforePinPath = InPinBefore->GetPinPath(); 
		}
	}
}

bool FOptimusNodeAction_MovePin::MovePin(
	IOptimusPathResolver* InRoot,
	const FString& InBeforePinPath
	)
{
	UOptimusNodePin* Pin = InRoot->ResolvePinPath(PinPath);
	if (!Pin)
	{
		return false;
	}

	const UOptimusNodePin* BeforePin = nullptr; 
	if (!InBeforePinPath.IsEmpty())
	{
		BeforePin = InRoot->ResolvePinPath(InBeforePinPath);
		if (!BeforePin)
		{
			return false;
		}
	}

	return Pin->GetOwningNode()->MovePinDirect(Pin, BeforePin);
}
