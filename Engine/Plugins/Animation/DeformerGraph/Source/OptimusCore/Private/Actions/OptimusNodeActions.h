// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusAction.h"
#include "OptimusNodePin.h"
#include "Types/SlateVector2.h"

#include "OptimusNodeActions.generated.h"

class UOptimusNode;
class UOptimusNodePin;
class IOptimusNodeAdderPinProvider;
class UOptimusNode_Comment;

USTRUCT()
struct FOptimusNodeAction_RenameNode : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_RenameNode() = default;

	FOptimusNodeAction_RenameNode(
		UOptimusNode* InNode,
		FText InNewName
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the node to be renamed.
	FString NodePath;

	// The node's new name
	FText NewName;

	// The node's old name
	FText OldName;
};


USTRUCT()
struct FOptimusNodeAction_MoveNode :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_MoveNode() = default;

	FOptimusNodeAction_MoveNode(
		UOptimusNode* InNode,
		const FVector2D &InPosition
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the node to be moved.
	FString NodePath;

	// The node's new position
	FDeprecateSlateVector2D NewPosition;

	// The node's old position
	FDeprecateSlateVector2D OldPosition;
};

USTRUCT()
struct FOptimusCommentNodeAction_ResizeNode :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusCommentNodeAction_ResizeNode() = default;

	FOptimusCommentNodeAction_ResizeNode(
		UOptimusNode_Comment* InNode,
		const UE::Slate::FDeprecateVector2DParameter& InNewSize
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the node to be resized
	FString NodePath;

	FDeprecateSlateVector2D OldSize;

	FDeprecateSlateVector2D NewSize;
};

USTRUCT()
struct FOptimusNodeAction_SetPinValue :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_SetPinValue() = default;

	FOptimusNodeAction_SetPinValue(
		UOptimusNodePin *InPin,
		const FString &InNewValue
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;
	
private:
	// The path of the pin to set the value on
	FString PinPath;

	// The new value to set
	FString NewValue;

	// The old value
	FString OldValue;
};


USTRUCT()
struct FOptimusNodeAction_SetPinName :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_SetPinName() = default;

	FOptimusNodeAction_SetPinName(
		UOptimusNodePin *InPin,
		FName InPinName
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;
	
private:
	bool SetPinName(
		IOptimusPathResolver* InRoot,
		FName InName);
	
	// The path of the pin to set the value on
	FString PinPath;

	// The new name to set
	FName NewPinName;

	// The old name
	FName OldPinName;
};


USTRUCT()
struct FOptimusNodeAction_SetPinType :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_SetPinType() = default;

	FOptimusNodeAction_SetPinType(
		UOptimusNodePin *InPin,
		FOptimusDataTypeRef InDataType
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;
	
private:
	bool SetPinType(
		IOptimusPathResolver* InRoot,
		FName InDataType) const;
	
	// The path of the pin to set the value on
	FString PinPath;

	// The new type to set
	FName NewDataTypeName;

	// The old data type
	FName OldDataTypeName;
};


USTRUCT()
struct FOptimusNodeAction_SetPinDataDomain :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeAction_SetPinDataDomain() = default;

	FOptimusNodeAction_SetPinDataDomain(
		const UOptimusNodePin *InPin,
		const FOptimusDataDomain& InDataDomain
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;
	
private:
	bool SetPinDataDomain(
		IOptimusPathResolver* InRoot,
		const FOptimusDataDomain& InDataDomain
		) const;
	
	// The path of the pin to set the value on
	FString PinPath;

	// The new data domain to set
	FOptimusDataDomain NewDataDomain;

	// The old data domain
	FOptimusDataDomain OldDataDomain;
};

USTRUCT()
struct FOptimusNodeAction_AddRemovePin :
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusNodeAction_AddRemovePin() = default;

	FOptimusNodeAction_AddRemovePin(
		UOptimusNode* InNode,
		FName InName,
		EOptimusNodePinDirection InDirection,
		const FOptimusDataDomain& InDataDomain,
		FOptimusDataTypeRef InDataType,
		const UOptimusNodePin* InBeforePin = nullptr,
		const UOptimusNodePin* InParentPin = nullptr
	);

	FOptimusNodeAction_AddRemovePin(
		UOptimusNode* InNode,
		FName InName,
		EOptimusNodePinDirection InDirection,
		const UOptimusNodePin* InBeforePin = nullptr
	);
	
	FOptimusNodeAction_AddRemovePin(
		UOptimusNodePin *InPin
		);

protected:
	bool AddPin(IOptimusPathResolver* InRoot);
	bool RemovePin(IOptimusPathResolver* InRoot) const;
	
	// The path of the node to have the pin added/removed from.
	FString NodePath;

	// Name of the new pin. After Do is called, this will be changed to the actual pin name
	// that got constructed.
	FName PinName = NAME_None;

	// The pin direction (input or output)
	EOptimusNodePinDirection Direction = EOptimusNodePinDirection::Unknown;

	// Is it a grouping pin?
	bool bIsGroupingPin = false;

	// The the data domain assumed for this pin.
	FOptimusDataDomain DataDomain;

	// The data type of the pin to create
	FName DataType = NAME_None;

	// (Optional) The pin that will be located right after this new pin.  
	FString BeforePinPath;

	// (Optional) The pin that will be the parent of this new new pin.  
	FString ParentPinPath;
	
	// The path of the newly created pin
	FString PinPath;

	// Expanded state of the pin being removed. 
	bool bExpanded = false;
};

template<>
struct TStructOpsTypeTraits<FOptimusNodeAction_AddRemovePin> :
	TStructOpsTypeTraitsBase2<FOptimusNodeAction_AddRemovePin>
{
	enum
	{
		WithPureVirtual = true,
	};
};


USTRUCT()
struct FOptimusNodeAction_AddPin :
	public FOptimusNodeAction_AddRemovePin
{
	GENERATED_BODY()

	FOptimusNodeAction_AddPin() = default;

	FOptimusNodeAction_AddPin(
		UOptimusNode* InNode,
		FName InName,
		EOptimusNodePinDirection InDirection,
		FOptimusDataDomain InDataDomain,
		FOptimusDataTypeRef InDataType,
		const UOptimusNodePin* InBeforePin = nullptr,
		const UOptimusNodePin* InParentPin = nullptr
	) :
		FOptimusNodeAction_AddRemovePin(InNode, InName, InDirection, InDataDomain, InDataType, InBeforePin, InParentPin)
	{
	}

	// Called to retrieve the pin that was created by Do after it has been called.
	UOptimusNodePin* GetPin(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return AddPin(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return RemovePin(InRoot); }
};


USTRUCT()
struct FOptimusNodeAction_AddGroupingPin :
	public FOptimusNodeAction_AddRemovePin
{
	GENERATED_BODY()

	FOptimusNodeAction_AddGroupingPin() = default;

	FOptimusNodeAction_AddGroupingPin(
		UOptimusNode* InNode,
		FName InName,
		EOptimusNodePinDirection InDirection,
		const UOptimusNodePin* InBeforePin = nullptr
	) :
		FOptimusNodeAction_AddRemovePin(InNode, InName, InDirection, InBeforePin)
	{
	}

	// Called to retrieve the pin that was created by Do after it has been called.
	UOptimusNodePin* GetPin(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return AddPin(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return RemovePin(InRoot); }
};


USTRUCT()
struct FOptimusNodeAction_RemovePin :
	public FOptimusNodeAction_AddRemovePin
{
	GENERATED_BODY()
	
	FOptimusNodeAction_RemovePin() = default;

	FOptimusNodeAction_RemovePin(
		UOptimusNodePin *InPinToRemove
		) : FOptimusNodeAction_AddRemovePin(InPinToRemove)
	{
	}

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return RemovePin(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return AddPin(InRoot); }
};


USTRUCT()
struct FOptimusNodeAction_MovePin :
	public FOptimusAction
{
	GENERATED_BODY()
	
	FOptimusNodeAction_MovePin() = default;

	FOptimusNodeAction_MovePin(
		UOptimusNodePin *InPinToMove,
		const UOptimusNodePin* InPinBefore
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return MovePin(InRoot, NewBeforePinPath); }
	bool Undo(IOptimusPathResolver* InRoot) override { return MovePin(InRoot, OldBeforePinPath); }

private:
	bool MovePin(
		IOptimusPathResolver* InRoot,
		const FString& InBeforePinPath
		);
	
	// The path of the pin to move
	FString PinPath;
	
	// The pin that this pin was next before, before the move. Empty string if it was the last pin.  
	FString OldBeforePinPath;

	// The pin that this pin will be next before, after the move. Empty string if it will be the last pin.
	FString NewBeforePinPath;
};
