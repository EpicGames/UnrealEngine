// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodeGraphActions.h"

#include "IOptimusCoreModule.h"
#include "IOptimusPathResolver.h"
#include "IOptimusUnnamedNodePinProvider.h"
#include "Nodes/OptimusNode_ComputeKernelFunction.h"
#include "Nodes/OptimusNode_CustomComputeKernel.h"
#include "Nodes/OptimusNode_ConstantValue.h"
#include "OptimusHelpers.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodeLink.h"
#include "OptimusNodePair.h"
#include "OptimusNodePin.h"


#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodeGraphActions)


// ---- Add graph

FOptimusNodeGraphAction_AddGraph::FOptimusNodeGraphAction_AddGraph(
	const FString& InGraphOwnerPath, 
	EOptimusNodeGraphType InGraphType, 
	FName InGraphName, 
	int32 InGraphIndex,
	TFunction<bool(UOptimusNodeGraph*)> InConfigureGraphFunc
	)
{
	GraphOwnerPath = InGraphOwnerPath;
	GraphType = InGraphType;
	GraphName = InGraphName;
	GraphIndex = InGraphIndex;
	ConfigureGraphFunc = InConfigureGraphFunc;

	SetTitlef(TEXT("Add graph"));

}



UOptimusNodeGraph* FOptimusNodeGraphAction_AddGraph::GetGraph(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveGraphPath(GraphPath);
}


bool FOptimusNodeGraphAction_AddGraph::Do(
	IOptimusPathResolver* InRoot
	)
{
	IOptimusNodeGraphCollectionOwner* GraphOwner = InRoot->ResolveCollectionPath(GraphOwnerPath);
	if (!GraphOwner)
	{
		return false;
	}
	
	UOptimusNodeGraph* Graph = GraphOwner->CreateGraphDirect(GraphType, GraphName, {});
	if (!Graph)
	{
		return false;
	}
	
	if (ConfigureGraphFunc && !ConfigureGraphFunc(Graph))
	{
		Optimus::RemoveObject(Graph);
		return false;
	}

	// Add the graph to the collection
	if (!GraphOwner->AddGraphDirect(Graph, GraphIndex))
	{
		Optimus::RemoveObject(Graph);
		return false;
	}
	
	if (GraphName == NAME_None)
	{
		GraphName = Graph->GetFName();
	}

	GraphPath = Graph->GetCollectionPath();
	return true;
}


bool FOptimusNodeGraphAction_AddGraph::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (Graph == nullptr)
	{
		return false;
	}

	return Graph->GetCollectionOwner()->RemoveGraphDirect(Graph);
}


// ---- Remove graph

FOptimusNodeGraphAction_RemoveGraph::FOptimusNodeGraphAction_RemoveGraph(
	UOptimusNodeGraph* InGraph
	)
{
	if (ensure(InGraph))
	{
		GraphPath = InGraph->GetCollectionPath();
		GraphOwnerPath = InGraph->GetCollectionOwner()->GetCollectionPath();
		GraphType = InGraph->GetGraphType();
		GraphName = InGraph->GetFName();
		GraphIndex = InGraph->GetGraphIndex();

		SetTitlef(TEXT("Remove graph"));
	}
}

bool FOptimusNodeGraphAction_RemoveGraph::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}

	IOptimusNodeGraphCollectionOwner* GraphOwner = InRoot->ResolveCollectionPath(GraphOwnerPath);
	if (!GraphOwner)
	{
		return false;
	}

	// Serialize all stored properties and referenced object 
	{
		Optimus::FBinaryObjectWriter GraphArchive(Graph, GraphData);
	}
	
	return GraphOwner->RemoveGraphDirect(Graph);
}


bool FOptimusNodeGraphAction_RemoveGraph::Undo(
	IOptimusPathResolver* InRoot
	)
{
	IOptimusNodeGraphCollectionOwner* GraphOwner = InRoot->ResolveCollectionPath(GraphOwnerPath);
	if (!GraphOwner)
	{
		return false;
	}
	
	// Create a graph, but don't add it to the list of used graphs. Otherwise interested parties
	// will be notified with a partially constructed graph.
	UOptimusNodeGraph* Graph = GraphOwner->CreateGraphDirect(GraphType, GraphName, TOptional<int32>());
	if (Graph == nullptr)
	{
		return false;
	}

	// Deserialize all the stored properties (and sub-objects) back onto the new graph.
	{
		Optimus::FBinaryObjectReader GraphArchive(Graph, GraphData);
	}
	
	// Now add the graph such that interested parties get notified.
	if (!GraphOwner->AddGraphDirect(Graph, GraphIndex))
	{
		Optimus::RemoveObject(Graph);
		return false;
	}
	
	return true;
}


// ---- Rename graph

FOptimusNodeGraphAction_RenameGraph::FOptimusNodeGraphAction_RenameGraph(
	UOptimusNodeGraph* InGraph,
	FName InNewName
	)
{
	if (ensure(InGraph) && InGraph->GetFName() != InNewName)
	{
		GraphPath = InGraph->GetCollectionPath();
		GraphOwnerPath = InGraph->GetCollectionOwner()->GetCollectionPath();

		// Ensure the name is unique within our namespace.
		if (StaticFindObject(UOptimusNodeGraph::StaticClass(), InGraph->GetOuter(), *InNewName.ToString()) != nullptr)
		{
			InNewName = MakeUniqueObjectName(InGraph->GetOuter(), UOptimusNodeGraph::StaticClass(), InNewName);
		}

		NewGraphName = InNewName;
		OldGraphName = InGraph->GetFName();

		SetTitlef(TEXT("Rename graph"));
	}
}


bool FOptimusNodeGraphAction_RenameGraph::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}

	IOptimusNodeGraphCollectionOwner* GraphOwner = InRoot->ResolveCollectionPath(GraphOwnerPath);
	if (!GraphOwner)
	{
		return false;
	}

	if (GraphOwner->RenameGraphDirect(Graph, NewGraphName.ToString()))
	{
		GraphPath = Graph->GetCollectionPath();
		return true;
	}

	return false;
	
}

bool FOptimusNodeGraphAction_RenameGraph::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}

	IOptimusNodeGraphCollectionOwner* GraphOwner = InRoot->ResolveCollectionPath(GraphOwnerPath);
	if (!GraphOwner)
	{
		return false;
	}

	if (GraphOwner->RenameGraphDirect(Graph, OldGraphName.ToString()))
	{
		GraphPath = Graph->GetCollectionPath();
		return true;
	}

	return false;	
}	


// ---- Add node

FOptimusNodeGraphAction_AddNode::FOptimusNodeGraphAction_AddNode(
	const FString& InGraphPath,
    const UClass* InNodeClass,
	FName InNodeName,
    TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
    )
{
	if (ensure(InNodeClass != nullptr))
	{
		GraphPath = InGraphPath;
		NodeClassPath = InNodeClass->GetPathName();
		NodeName = InNodeName;
		ConfigureNodeFunc = InConfigureNodeFunc;

		// FIXME: Prettier name.
		SetTitlef(TEXT("Add Node"));
	}
}


UOptimusNode* FOptimusNodeGraphAction_AddNode::GetNode(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveNodePath(NodePath);
}


bool FOptimusNodeGraphAction_AddNode::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}
	UClass* NodeClass = Optimus::FindObjectInPackageOrGlobal<UClass>(NodeClassPath);
	if (!NodeClass)
	{
		return false;
	}

	UOptimusNode* Node = Graph->CreateNodeDirect(NodeClass, NodeName, ConfigureNodeFunc);
	if (!Node)
	{
		return false;
	}

	NodePath = Node->GetNodePath();

	return true;
}


bool FOptimusNodeGraphAction_AddNode::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	UOptimusNodeGraph* Graph = Node->GetOwningGraph();
	if (!Graph)
	{
		return false;
	}

	// Save the assigned node name for when Do gets called again.
	NodeName = Node->GetFName();

	return Graph->RemoveNodeDirect(Node);
}


// ---- Duplicate node


FOptimusNodeGraphAction_DuplicateNode::FOptimusNodeGraphAction_DuplicateNode(
	const FString& InTargetGraphPath,
	UOptimusNode* InSourceNode,
	FName InNodeName,
	TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
	)
{
	if (ensure(InSourceNode != nullptr))
	{
		SourceNodePath = InSourceNode->GetNodePath();
		GraphPath = InTargetGraphPath;
		NodeName = InNodeName;
		NodeClassPath = InSourceNode->GetClass()->GetPathName();
		ConfigureNodeFunc = InConfigureNodeFunc;

		FMemoryWriter NodeArchive(CachedNodeData);
		InSourceNode->ExportState(NodeArchive);
	}
}


UOptimusNode* FOptimusNodeGraphAction_DuplicateNode::GetNode(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveNodePath(NodePath);
}


bool FOptimusNodeGraphAction_DuplicateNode::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNode* SourceNode = InRoot->ResolveNodePath(SourceNodePath);

	TArray<uint8> NodeData;
	if (!SourceNode)
	{
		// Used the cached data if we are duplicating from a clipboard graph
		NodeData = CachedNodeData;
	}
	else
	{
		FMemoryWriter NodeArchive(NodeData);
		SourceNode->ExportState(NodeArchive);
	}
	
	
	auto BootstrapNodeFunc = [NodeData, this](UOptimusNode* InNode) -> bool
	{
		FMemoryReader NodeArchive(NodeData);
		InNode->ImportState(NodeArchive);
		
		if (!ConfigureNodeFunc)
		{
			return true;
		}
		return ConfigureNodeFunc(InNode);
	};

	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}

	UClass* NodeClass = Optimus::FindObjectInPackageOrGlobal<UClass>(NodeClassPath);
	if (!NodeClass)
	{
		return false;
	}

	if (UOptimusNode_ConstantValueGeneratorClass* GeneratorClass = Cast<UOptimusNode_ConstantValueGeneratorClass>(NodeClass))
	{
		// Make sure a node class from the current package is used for constant nodes
		NodeClass = UOptimusNode_ConstantValueGeneratorClass::GetClassForType(Graph->GetPackage(),GeneratorClass->DataType);
	}

	UOptimusNode* Node = Graph->CreateNodeDirect(NodeClass, NodeName, BootstrapNodeFunc);
	if (!Node)
	{
		return false;
	}

	// Inform the node that it has been photocopied, and so it can do any fix-ups related to it.
	Node->PostDuplicate(EDuplicateMode::Normal);

	NodeName = Node->GetFName();
	NodePath = Node->GetNodePath();

	return true;
}


bool FOptimusNodeGraphAction_DuplicateNode::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	UOptimusNodeGraph* Graph = Node->GetOwningGraph();
	if (!ensure(Graph))
	{
		return false;
	}

	return Graph->RemoveNodeDirect(Node);
}


// ---- Remove node

FOptimusNodeGraphAction_RemoveNode::FOptimusNodeGraphAction_RemoveNode(
	UOptimusNode* InNode
	)
{
	if (ensure(InNode != nullptr))
	{
		NodePath = InNode->GetNodePath();

		GraphPath = InNode->GetOwningGraph()->GetCollectionPath();
		NodeName = InNode->GetFName();
		NodeClassPath = InNode->GetClass()->GetPathName();

		// FIXME: Prettier name.
		SetTitlef(TEXT("Remove Node"));
	}
}


bool FOptimusNodeGraphAction_RemoveNode::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	if (!Node)
	{
		return false;
	}

	UOptimusNodeGraph* Graph = Node->GetOwningGraph();
	if (!ensure(Graph))
	{
		return false;
	}

	FMemoryWriter NodeArchive(NodeData);
	Node->ExportState(NodeArchive);

	return Graph->RemoveNodeDirect(Node);
}


bool FOptimusNodeGraphAction_RemoveNode::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}
	UClass* NodeClass = Optimus::FindObjectInPackageOrGlobal<UClass>(NodeClassPath);
	if (!NodeClass)
	{
		return false;
	}

	UOptimusNode* Node = Graph->CreateNodeDirect(NodeClass, NodeName, [this](UOptimusNode* InNode)
	{
		FMemoryReader NodeArchive(NodeData);
		InNode->ImportState(NodeArchive);
		return true;
	});

	if (!Node)
	{
		return false;
	}
	
	return true;
}


FOptimusNodeGraphAction_AddRemoveNodePair::FOptimusNodeGraphAction_AddRemoveNodePair(const FString& InFirstNodePath, const FString& InSecondNodePath)
{
	FirstNodePath = InFirstNodePath;
	SecondNodePath = InSecondNodePath;
}

bool FOptimusNodeGraphAction_AddRemoveNodePair::AddNodePair(IOptimusPathResolver* InRoot)
{
	UOptimusNode* FirstNode = InRoot->ResolveNodePath(FirstNodePath);
	UOptimusNode* SecondNode = InRoot->ResolveNodePath(SecondNodePath);

	UOptimusNodeGraph* Graph = FirstNode->GetOwningGraph();
	return Graph->AddNodePairDirect(FirstNode, SecondNode);
}

bool FOptimusNodeGraphAction_AddRemoveNodePair::RemoveNodePair(IOptimusPathResolver* InRoot)
{
	UOptimusNode* FirstNode = InRoot->ResolveNodePath(FirstNodePath);
	UOptimusNode* SecondNode = InRoot->ResolveNodePath(SecondNodePath);

	UOptimusNodeGraph* Graph = FirstNode->GetOwningGraph();
	return Graph->RemoveNodePairDirect(FirstNode, SecondNode);
}

FOptimusNodeGraphAction_AddNodePair::FOptimusNodeGraphAction_AddNodePair(const FString& InFirstNodePath, const FString& InSecondNodePath) :
	FOptimusNodeGraphAction_AddRemoveNodePair(InFirstNodePath, InSecondNodePath)
{
	SetTitlef(TEXT("Add Node Pair"));
}

FOptimusNodeGraphAction_AddNodePair::FOptimusNodeGraphAction_AddNodePair(TWeakPtr<FOptimusNodeGraphAction_AddNode> InAddFirstNodeAction, TWeakPtr<FOptimusNodeGraphAction_AddNode> InAddSecondNodeAction)
{
	AddFirstNodeAction = InAddFirstNodeAction;
	AddSecondNodeAction = InAddSecondNodeAction;

	SetTitlef(TEXT("Add Node Pair"));
}

bool FOptimusNodeGraphAction_AddNodePair::Do(IOptimusPathResolver* InRoot)
{
	if (!FirstNodePath.IsEmpty() && !SecondNodePath.IsEmpty())
	{
		return AddNodePair(InRoot);	
	}

	// In case the node paths are not assigned, try to extract them from previous add node actions
	if (!AddFirstNodeAction.IsValid() || !AddSecondNodeAction.IsValid())
	{
		return false;
	}

	const UOptimusNode* FirstNode = AddFirstNodeAction.Pin()->GetNode(InRoot);
	const UOptimusNode* SecondNode = AddSecondNodeAction.Pin()->GetNode(InRoot);

	if (!FirstNode || !SecondNode)
	{
		return false;	
	}

	FirstNodePath = FirstNode->GetNodePath();
	SecondNodePath = SecondNode->GetNodePath();
	
	return AddNodePair(InRoot);	
}

bool FOptimusNodeGraphAction_AddNodePair::Undo(IOptimusPathResolver* InRoot)
{
	// Upon undo, the node paths should have been assigned during Do()
	if (ensure(!FirstNodePath.IsEmpty() && !SecondNodePath.IsEmpty()))
	{
		return RemoveNodePair(InRoot);	
	}

	return false;
}

FOptimusNodeGraphAction_RemoveNodePair::FOptimusNodeGraphAction_RemoveNodePair(const UOptimusNodePair* InNodePair) :
	FOptimusNodeGraphAction_AddRemoveNodePair(InNodePair->GetFirst()->GetNodePath(), InNodePair->GetSecond()->GetNodePath())
{
	SetTitlef(TEXT("Remove Node Pair"));
}

// ---- Add/remove link base

FOptimusNodeGraphAction_AddRemoveLink::FOptimusNodeGraphAction_AddRemoveLink(
	UOptimusNodePin* InNodeOutputPin, 
	UOptimusNodePin* InNodeInputPin,
	bool bInCanFail
	)
{
	if (ensure(InNodeOutputPin != nullptr) && ensure(InNodeInputPin != nullptr) &&
		ensure(InNodeOutputPin->GetDirection() == EOptimusNodePinDirection::Output) && 
		ensure(InNodeInputPin->GetDirection() == EOptimusNodePinDirection::Input) && 
		ensure(InNodeOutputPin->GetOwningNode() != InNodeInputPin->GetOwningNode()) &&
		ensure(InNodeOutputPin->GetOwningNode()->GetOwningGraph() == InNodeInputPin->GetOwningNode()->GetOwningGraph())
		)
	{
		NodeOutputPinPath = InNodeOutputPin->GetPinPath();
		NodeInputPinPath = InNodeInputPin->GetPinPath();
		bCanFail = bInCanFail;
	}
}


FOptimusNodeGraphAction_AddRemoveLink::FOptimusNodeGraphAction_AddRemoveLink(
	const FString& InNodeOutputPinPath,
	const FString& InNodeInputPinPath,
	bool bInCanFail
	)
{
	if (ensure(!InNodeOutputPinPath.IsEmpty()) && ensure(!InNodeInputPinPath.IsEmpty()))
	{
		NodeOutputPinPath = InNodeOutputPinPath;
		NodeInputPinPath = InNodeInputPinPath;
		bCanFail = bInCanFail;
	}
}


bool FOptimusNodeGraphAction_AddRemoveLink::AddLink(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodePin* InNodeOutputPin = InRoot->ResolvePinPath(NodeOutputPinPath);
	if (InNodeOutputPin == nullptr)
	{
		return bCanFail;
	}

	UOptimusNodePin* InNodeInputPin = InRoot->ResolvePinPath(NodeInputPinPath);
	if (InNodeInputPin == nullptr)
	{
		return bCanFail;
	}

	UOptimusNodeGraph* Graph = InNodeOutputPin->GetOwningNode()->GetOwningGraph();
	return Graph->AddLinkDirect(InNodeOutputPin, InNodeInputPin);
}


bool FOptimusNodeGraphAction_AddRemoveLink::RemoveLink(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodePin* InNodeOutputPin = InRoot->ResolvePinPath(NodeOutputPinPath);
	if (InNodeOutputPin == nullptr)
	{
		return bCanFail;
	}

	UOptimusNodePin* InNodeInputPin = InRoot->ResolvePinPath(NodeInputPinPath);
	if (InNodeInputPin == nullptr)
	{
		return bCanFail;
	}

	UOptimusNodeGraph* Graph = InNodeOutputPin->GetOwningNode()->GetOwningGraph();
	return Graph->RemoveLinkDirect(InNodeOutputPin, InNodeInputPin);
}


// ---- Add link

FOptimusNodeGraphAction_AddLink::FOptimusNodeGraphAction_AddLink(
	UOptimusNodePin* InNodeOutputPin, 
	UOptimusNodePin* InNodeInputPin,
	bool bInCanFail
	) :
	FOptimusNodeGraphAction_AddRemoveLink(InNodeOutputPin, InNodeInputPin, bInCanFail)
{
	// FIXME: Prettier name.
	SetTitlef(TEXT("Add Link"));
}


FOptimusNodeGraphAction_AddLink::FOptimusNodeGraphAction_AddLink(
	const FString& InNodeOutputPinPath,
	const FString& InNodeInputPinPath,
	bool bInCanFail
	) :
	FOptimusNodeGraphAction_AddRemoveLink(InNodeOutputPinPath, InNodeInputPinPath, bInCanFail)
{
	SetTitlef(TEXT("Add Link"));
}


// ---- Remove link

FOptimusNodeGraphAction_RemoveLink::FOptimusNodeGraphAction_RemoveLink(
	const UOptimusNodeLink* InLink
	) :
	FOptimusNodeGraphAction_AddRemoveLink(InLink->GetNodeOutputPin(), InLink->GetNodeInputPin())
{
	SetTitlef(TEXT("Remove Link"));
}


FOptimusNodeGraphAction_RemoveLink::FOptimusNodeGraphAction_RemoveLink(
	UOptimusNodePin* InNodeOutputPin,
	UOptimusNodePin* InNodeInputPin
	) :
	FOptimusNodeGraphAction_AddRemoveLink(InNodeOutputPin, InNodeInputPin)
{
	SetTitlef(TEXT("Remove Link"));
}

FOptimusNodeGraphAction_ConnectAdderPin::FOptimusNodeGraphAction_ConnectAdderPin(
	IOptimusNodeAdderPinProvider* InAdderPinProvider,
	const IOptimusNodeAdderPinProvider::FAdderPinAction& InAction,
	UOptimusNodePin* InSourcePin
	) :
	Action(InAction)
{
	if (InAction.bCanAutoLink)
	{
		SetTitlef(TEXT("Add Pin & Link"));
	}
	else
	{
		SetTitlef(TEXT("Add Pin From Pin"));
	}
	
	UOptimusNode* Node = Cast<UOptimusNode>(InAdderPinProvider);
	
	if (ensure(Node))
	{
		NodePath = Node->GetNodePath();
	}

	SourcePinPath = InSourcePin->GetPinPath();
}

bool FOptimusNodeGraphAction_ConnectAdderPin::Do(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	IOptimusNodeAdderPinProvider* AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(Node);
	
	if (!ensure(AdderPinProvider))
	{
		return false;
	}

	UOptimusNodePin* SourcePin = InRoot->ResolvePinPath(SourcePinPath);

	FName Name = FName(SourcePin->GetDisplayName().ToString());
	if (IOptimusUnnamedNodePinProvider* UnnamedPinProvider = Cast<IOptimusUnnamedNodePinProvider>(SourcePin->GetOwningNode()))
	{
		Name = UnnamedPinProvider->GetNameForAdderPin(SourcePin);
	}
	
	TArray<UOptimusNodePin*> AddedPins = AdderPinProvider->TryAddPinFromPin(Action, SourcePin, Name);

	if (AddedPins.Num() == 0)
	{
		return false;
	}

	AddedPinPaths.Reset();
	Algo::Transform(AddedPins, AddedPinPaths,
		[](UOptimusNodePin* InPin)
		{
			return InPin->GetPinPath();
		});

	UOptimusNodePin* AddedPin = AddedPins.Last();
	
	if (!ensure(!AddedPin->IsGroupingPin()))
	{
		AdderPinProvider->RemoveAddedPins(AddedPins);
		return false;
	}

	if (Action.bCanAutoLink)
	{
		if (SourcePin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			NodeOutputPinPath = SourcePin->GetPinPath();
			NodeInputPinPath = AddedPin->GetPinPath();
		}
		else
		{
			
			NodeOutputPinPath = AddedPin->GetPinPath();
			NodeInputPinPath = SourcePin->GetPinPath();
		}
		
		if (!ensure(AddLink(InRoot)))
		{
			AdderPinProvider->RemoveAddedPins(AddedPins);
			return false;
		}
	}
	
	return true;
}

bool FOptimusNodeGraphAction_ConnectAdderPin::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusNode* Node = InRoot->ResolveNodePath(NodePath);
	IOptimusNodeAdderPinProvider* AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(Node);
	
	if (!ensure(AdderPinProvider))
	{
		return false;
	}

	if (Action.bCanAutoLink)
	{
		if (!ensure(RemoveLink(InRoot)))
		{
			return false;
		}
	}

	TArray<UOptimusNodePin*> AddedPins;

	Algo::Transform(AddedPinPaths, AddedPins,
		[InRoot](const FString& InPinPath)
		{
			return InRoot->ResolvePinPath(InPinPath);
		});

	return AdderPinProvider->RemoveAddedPins(AddedPins);
}


FOptimusNodeGraphAction_PackageKernelFunction::FOptimusNodeGraphAction_PackageKernelFunction(
	UOptimusNode_CustomComputeKernel* InKernelNode,
	FName InNodeName
	)
{
	if (ensure(InKernelNode))
	{
		GraphPath = InKernelNode->GetOwningGraph()->GetCollectionPath();

		NodeName = InNodeName;
		NodePosition = InKernelNode->GetGraphPosition();
		Category = InKernelNode->Category;
		KernelName = InKernelNode->KernelName;
		GroupSize = InKernelNode->GroupSize;
		InputBindings = InKernelNode->InputBindingArray.InnerArray;
		OutputBindings = InKernelNode->OutputBindingArray.InnerArray;
		ShaderSource = InKernelNode->ShaderSource.ShaderText;
	}
}


UOptimusNode* FOptimusNodeGraphAction_PackageKernelFunction::GetNode(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveNodePath(NodePath);
}


bool FOptimusNodeGraphAction_PackageKernelFunction::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph *Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}
	
	UClass *PackagedNodeClass = UOptimusNode_ComputeKernelFunctionGeneratorClass::CreateNodeClass(
		Graph->GetPackage(), Category, KernelName, GroupSize,
		InputBindings, OutputBindings, ShaderSource);
	if (!PackagedNodeClass)
	{
		return false;
	}
	
	// Notify the world that we've added a new node class. This updates the node palette, among
	// other things.
	Graph->GlobalNotify(EOptimusGlobalNotifyType::NodeTypeAdded, PackagedNodeClass);
	
	// FIXME: This packaging action should only create the class. We need action chaining with
	// argument piping.
	NodeClassName = PackagedNodeClass->GetName();

	UOptimusNode *Node = Graph->CreateNodeDirect(PackagedNodeClass, NodeName, 
		[NodePosition=NodePosition](UOptimusNode *InNode) {
			return InNode->SetGraphPositionDirect(NodePosition);
		});
	if (!Node)
	{
		return false;
	}
		
	NodePath = Node->GetNodePath();
	
	return true;
}


bool FOptimusNodeGraphAction_PackageKernelFunction::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph *Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}
	
	UOptimusNode *Node =  InRoot->ResolveNodePath(NodePath);
	if (!Graph)
	{
		return false;
	}

	UClass* NodeClass = FindObject<UClass>(Graph->GetPackage(), *NodeClassName);
	if (!NodeClass)
	{
		return false;
	}
	
	if (!Graph->RemoveNodeDirect(Node))
	{
		return false;
	}

	if (!NodeClass->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional))
	{
		return false;
	}

	// Notify the world that we've removed the class. We do this after shuffling it into the transient
	// package so that it can be filtered out by UOptimusNode::GetAllNodeClasses.
	Graph->GlobalNotify(EOptimusGlobalNotifyType::NodeTypeRemoved, NodeClass);

	return true;
}


FOptimusNodeGraphAction_UnpackageKernelFunction::FOptimusNodeGraphAction_UnpackageKernelFunction(
	UOptimusNode_ComputeKernelFunction* InKernelFunction,
	FName InNodeName
	)
{
	if (ensure(InKernelFunction))
	{
		GraphPath = InKernelFunction->GetOwningGraph()->GetCollectionPath();
		ClassPath = InKernelFunction->GetClass()->GetPathName();
		NodeName = InNodeName;
		NodePosition = InKernelFunction->GetGraphPosition();
	}
}


UOptimusNode* FOptimusNodeGraphAction_UnpackageKernelFunction::GetNode(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveNodePath(NodePath);
}


bool FOptimusNodeGraphAction_UnpackageKernelFunction::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph* Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}

	UOptimusNode_ComputeKernelFunctionGeneratorClass* Class = Optimus::FindObjectInPackageOrGlobal<UOptimusNode_ComputeKernelFunctionGeneratorClass>(ClassPath);
	if (!Class)
	{
		return false;
	}

	UOptimusNode *Node = Graph->CreateNodeDirect(UOptimusNode_CustomComputeKernel::StaticClass(), NodeName, 
		[Class, NodePosition=NodePosition](UOptimusNode *InNode) {
			UOptimusNode_CustomComputeKernel* KernelNode = Cast<UOptimusNode_CustomComputeKernel>(InNode);
			KernelNode->Category = Class->Category;
			KernelNode->KernelName = Class->KernelName;
			KernelNode->GroupSize = Class->GroupSize;
			KernelNode->InputBindingArray = Class->InputBindings;
			KernelNode->OutputBindingArray = Class->OutputBindings;
			KernelNode->ShaderSource.ShaderText = Class->ShaderSource;
			return InNode->SetGraphPositionDirect(NodePosition);
		});
	if (!Node)
	{
		return false;
	}

	NodePath = Node->GetNodePath();
	return true;
}


bool FOptimusNodeGraphAction_UnpackageKernelFunction::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusNodeGraph *Graph = InRoot->ResolveGraphPath(GraphPath);
	if (!Graph)
	{
		return false;
	}
	
	UOptimusNode *Node =  InRoot->ResolveNodePath(NodePath);
	if (!Graph)
	{
		return false;
	}

	return Graph->RemoveNodeDirect(Node);
}
