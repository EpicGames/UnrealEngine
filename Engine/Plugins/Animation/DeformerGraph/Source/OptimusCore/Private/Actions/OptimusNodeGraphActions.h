// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusAction.h"
#include "OptimusDataDomain.h"
#include "OptimusBindingTypes.h"
#include "Nodes/OptimusNode_ComputeKernelBase.h"
#include "IOptimusNodeAdderPinProvider.h"

#include "UObject/UnrealNames.h"

#include "OptimusNodeGraphActions.generated.h"

class IOptimusNodeAdderPinProvider;
class UOptimusNode_ComputeKernelFunction;
class UOptimusNode_CustomComputeKernel;
enum class EOptimusNodeGraphType;
class IOptimusNodeGraphCollectionOwner;
class UOptimusNode;
class UOptimusNodePair;
class UOptimusNodeGraph;
class UOptimusNodeLink;
class UOptimusNodePin;


USTRUCT()
struct FOptimusNodeGraphAction_AddGraph :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_AddGraph() = default;

	FOptimusNodeGraphAction_AddGraph(
		const FString& InGraphOwnerPath,
		EOptimusNodeGraphType InGraphType,
		FName InGraphName,
		int32 InGraphIndex,
		TFunction<bool(UOptimusNodeGraph*)> InConfigureGraphFunc = {}
		);

	UOptimusNodeGraph* GetGraph(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

	// The type of graph to create
	EOptimusNodeGraphType GraphType;

	// The path to the graph owner
	FString GraphOwnerPath;

	// The name of the graph being created.
	FName GraphName;

	// The index of this new graph in the graph stack.
	int32 GraphIndex;

	// An optional function called to configure the graph after it gets created, but before it
	// gets added to the graph collection. This can only be used to configure properties of the
	// graph itself and objects owned by the graph, otherwise those modification are unlikely
	// to survive an undo/redo pass.
	// The function should return false if the config fails. In that case the rest of the action
	// gets unwound, leaving the asset in a consistent state.
	TFunction<bool(UOptimusNodeGraph*)> ConfigureGraphFunc;
	
	// The path of the freshly created graph after the first call to Do.
	FString GraphPath;
};


USTRUCT()
struct FOptimusNodeGraphAction_RemoveGraph : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_RemoveGraph() = default;

	FOptimusNodeGraphAction_RemoveGraph(
	    UOptimusNodeGraph* InGraph);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the graph to remove.
	FString GraphPath;

	// The path of the graph's owner.
	FString GraphOwnerPath;
	
	// The type of graph to reconstruct back to.
	EOptimusNodeGraphType GraphType;

	// The name to reconstruct the node as.
	FName GraphName;
	
	// The absolute evaluation order the graph was in.
	int32 GraphIndex;

	// The stored graph data.
	TArray<uint8> GraphData;
};


USTRUCT()
struct FOptimusNodeGraphAction_RenameGraph : public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_RenameGraph() = default;

	FOptimusNodeGraphAction_RenameGraph(
	    UOptimusNodeGraph* InGraph,
		FName InNewName);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:

	// The path of the graph owner that will actually perform the rename 
	FString GraphOwnerPath;
	
	// The path of the graph to rename. This value will change after each rename.
	FString GraphPath;

	// The new name for this graph. This name may be modified to retain namespace unicity.
	FName NewGraphName;

	// The previous name of the graph
	FName OldGraphName;
};


USTRUCT()
struct FOptimusNodeGraphAction_AddNode : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_AddNode() = default;

	FOptimusNodeGraphAction_AddNode(
		const FString& InGraphPath,
		const UClass* InNodeClass,
		FName InNodeName,
		TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
		);

	/// Called to retrieve the node that was created by DoImpl after it has been called.
	UOptimusNode* GetNode(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the graph the node should be added to.
	FString GraphPath;

	// The class path of the node to add.
	FString NodeClassPath;

	// An optional function called to configure the node after it gets created, but before it
	// gets added to the graph.
	TFunction<bool(UOptimusNode*)> ConfigureNodeFunc;

	// The path of the newly added node or the node to remove.
	FString NodePath;

	// The name of the newly added node. Used if we undo and then redo the action to
	// ensure we reconstruct the node with the same name.
	FName NodeName = NAME_None;
};


USTRUCT()
struct FOptimusNodeGraphAction_DuplicateNode : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_DuplicateNode() = default;

	FOptimusNodeGraphAction_DuplicateNode(
		const FString& InTargetGraphPath,
		UOptimusNode* InSourceNode,
		FName InNodeName,
		TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
		);

	/// Called to retrieve the node that was created by DoImpl after it has been called.
	UOptimusNode* GetNode(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	
	// The path of the node to duplicate
	FString SourceNodePath;
	
	// The path of the graph the node should be added to.
	FString GraphPath;
	
	// The name of the node to create. The original name of the node is used as a template but
	// will be updated to avoid namespace collisions when first created.
	FName NodeName;
	
	// The class path of the node to add.
	FString NodeClassPath;

	// An optional function called to configure the node after it gets created, but before it
	// gets added to the graph.
	TFunction<bool(UOptimusNode*)> ConfigureNodeFunc;

	// State of the node when this action was created, useful when moving nodes from clipboard graph to a real graph
	// not used if we can resolve the source node path during Do(), for regular duplicate usage
	TArray<uint8> CachedNodeData;

	// Path to the node that gets created through duplication.
	FString NodePath;
};


USTRUCT()
struct FOptimusNodeGraphAction_RemoveNode :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusNodeGraphAction_RemoveNode() = default;

	FOptimusNodeGraphAction_RemoveNode(
		UOptimusNode* InNode
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// Path to the node to remove.
	FString NodePath;

	// The path of the graph the node should be added to.
	FString GraphPath;

	// The class path of the node to reconstruct.
	FString NodeClassPath;

	// The name to reconstruct the node as.
	FName NodeName;

	// The stored node data.
	TArray<uint8> NodeData;
};

// A base action for adding/removing node pairs.
USTRUCT()
struct FOptimusNodeGraphAction_AddRemoveNodePair:
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_AddRemoveNodePair() = default;

	FOptimusNodeGraphAction_AddRemoveNodePair(
		const FString& InFirstNodePath,
		const FString& InSecondNodePath
		);

protected:
	bool AddNodePair(IOptimusPathResolver* InRoot);
	bool RemoveNodePair(IOptimusPathResolver* InRoot);

	FString FirstNodePath;

	FString SecondNodePath;
};

// Mark FOptimusNodeGraphAction_AddRemoveNodePair as pure virtual, so that the UObject machinery
// won't attempt to instantiate it.
template<>
struct TStructOpsTypeTraits<FOptimusNodeGraphAction_AddRemoveNodePair> :
	TStructOpsTypeTraitsBase2<FOptimusNodeGraphAction_AddRemoveNodePair>
{
	enum
	{
		WithPureVirtual = true,
    };
};



USTRUCT()
struct FOptimusNodeGraphAction_AddNodePair :
	public FOptimusNodeGraphAction_AddRemoveNodePair
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_AddNodePair() = default;

	// Use when the node paths are unknown after add node actions are performed
	FOptimusNodeGraphAction_AddNodePair(
		TWeakPtr<FOptimusNodeGraphAction_AddNode> InAddFirstNodeAction,
		TWeakPtr<FOptimusNodeGraphAction_AddNode> InAddSecondNodeAction
		);
	
	// Use when the node paths are available already
	FOptimusNodeGraphAction_AddNodePair(
		const FString& InFirstNodePath,
		const FString& InSecondNodePath
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override; 
	bool Undo(IOptimusPathResolver* InRoot) override;

	
	TWeakPtr<FOptimusNodeGraphAction_AddNode> AddFirstNodeAction;
	TWeakPtr<FOptimusNodeGraphAction_AddNode> AddSecondNodeAction;
};

USTRUCT()
struct FOptimusNodeGraphAction_RemoveNodePair :
	public FOptimusNodeGraphAction_AddRemoveNodePair
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_RemoveNodePair() = default;

	FOptimusNodeGraphAction_RemoveNodePair(
		const UOptimusNodePair* InNodePair
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return RemoveNodePair(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return AddNodePair(InRoot); }
};


// A base action for adding/removing nodes.
USTRUCT()
struct FOptimusNodeGraphAction_AddRemoveLink :
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_AddRemoveLink() = default;

	FOptimusNodeGraphAction_AddRemoveLink(
		UOptimusNodePin* InNodeOutputPin, 
		UOptimusNodePin* InNodeInputPin,
		bool bInCanFail = false
		);

	FOptimusNodeGraphAction_AddRemoveLink(
		const FString& InNodeOutputPinPath,
		const FString& InNodeInputPinPath,
		bool bInCanFail = false
		);

protected:
	bool AddLink(IOptimusPathResolver* InRoot);
	bool RemoveLink(IOptimusPathResolver* InRoot);

	// The path of the output pin on the node to connect/disconnect to/from.
	FString NodeOutputPinPath;

	// The path of the output input on the node to connect/disconnect to/from.
	FString NodeInputPinPath;

	// The operation is allowed to fail gracefully without terminating the
	// rest of the actions being performed.
	bool bCanFail = false;
};

// Mark FOptimusNodeGraphAction_AddRemoveLink as pure virtual, so that the UObject machinery
// won't attempt to instantiate it.
template<>
struct TStructOpsTypeTraits<FOptimusNodeGraphAction_AddRemoveLink> :
	TStructOpsTypeTraitsBase2<FOptimusNodeGraphAction_AddRemoveLink>
{
	enum
	{
		WithPureVirtual = true,
    };
};



USTRUCT()
struct FOptimusNodeGraphAction_AddLink :
	public FOptimusNodeGraphAction_AddRemoveLink
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_AddLink() = default;

	FOptimusNodeGraphAction_AddLink(
		UOptimusNodePin* InNodeOutputPin,
		UOptimusNodePin* InNodeInputPin,
		bool bInCanFail = false
	);

	FOptimusNodeGraphAction_AddLink(
		const FString& InNodeOutputPinPath,
		const FString& InNodeInputPinPath,
		bool bInCanFail = false
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return AddLink(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return RemoveLink(InRoot); }
};

USTRUCT()
struct FOptimusNodeGraphAction_RemoveLink :
	public FOptimusNodeGraphAction_AddRemoveLink
{
	GENERATED_BODY()

	FOptimusNodeGraphAction_RemoveLink() = default;

	FOptimusNodeGraphAction_RemoveLink(
		const UOptimusNodeLink *InLink
	);

	FOptimusNodeGraphAction_RemoveLink(
		UOptimusNodePin* InNodeOutputPin,
		UOptimusNodePin* InNodeInputPin
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override { return RemoveLink(InRoot); }
	bool Undo(IOptimusPathResolver* InRoot) override { return AddLink(InRoot); }
};

USTRUCT()
struct FOptimusNodeGraphAction_ConnectAdderPin
	: public FOptimusNodeGraphAction_AddRemoveLink
{
	GENERATED_BODY()
	
	FOptimusNodeGraphAction_ConnectAdderPin() = default;

	FOptimusNodeGraphAction_ConnectAdderPin(
		IOptimusNodeAdderPinProvider* InAdderPinProvider,
		const IOptimusNodeAdderPinProvider::FAdderPinAction& InAction,
		UOptimusNodePin* InSourcePin
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override; 
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	FString NodePath;

	IOptimusNodeAdderPinProvider::FAdderPinAction Action;	
	
	FString SourcePinPath;

	TArray<FString> AddedPinPaths;
};

USTRUCT()
struct FOptimusNodeGraphAction_PackageKernelFunction :
	public FOptimusAction
{
	GENERATED_BODY();

	FOptimusNodeGraphAction_PackageKernelFunction() = default;

	FOptimusNodeGraphAction_PackageKernelFunction(
		UOptimusNode_CustomComputeKernel* InKernelNode,
		FName InNodeName);

	UOptimusNode *GetNode(IOptimusPathResolver* InRoot) const;	
	
protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the graph we're creating the packaged node in.
	FString GraphPath;

	// Node Data to both create packaged node and reconstruct the un-packaged node on undo.
	FName NodeName;
	FVector2D NodePosition;
	FName Category;
	FName KernelName;
	FIntVector GroupSize;
	TArray<FOptimusParameterBinding> InputBindings;
	TArray<FOptimusParameterBinding> OutputBindings;
	FString ShaderSource;

	// Once we've created the node class, we store it here, in case we need to remove it on
	// undo.
	FString NodeClassName;
	
	// The path of the node we just created
	FString NodePath;
};


USTRUCT()
struct FOptimusNodeGraphAction_UnpackageKernelFunction :
	public FOptimusAction
{
	GENERATED_BODY();

	FOptimusNodeGraphAction_UnpackageKernelFunction() = default;

	FOptimusNodeGraphAction_UnpackageKernelFunction(
		UOptimusNode_ComputeKernelFunction* InKernelFunction,
		FName InNodeName);

	UOptimusNode *GetNode(IOptimusPathResolver* InRoot) const;	
	
protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The path of the graph we're creating the packaged node in.
	FString GraphPath;

	// The path to the node class for the function
	FString ClassPath;

	// The name of the custom kernel node to create.
	FName NodeName;

	// The position to place it at. Everything else is copied from the class. 
	FVector2D NodePosition;
	
	// The path of the node we just created
	FString NodePath;
};
