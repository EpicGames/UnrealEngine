// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusResourceActions.h"

#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "OptimusResourceDescription.h"

#include "Serialization/ObjectWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusResourceActions)


FOptimusResourceAction_AddResource::FOptimusResourceAction_AddResource(
	FOptimusDataTypeRef InDataType,
    FName InName,
    FOptimusDataDomain InDataDomain
	)
{
	ResourceName = InName;
	DataType = InDataType;
	DataDomain = InDataDomain;

	SetTitlef(TEXT("Add resource '%s'"), *ResourceName.ToString());
}


UOptimusResourceDescription* FOptimusResourceAction_AddResource::GetResource(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveResource(ResourceName);
}


bool FOptimusResourceAction_AddResource::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusDeformer *Deformer = Cast<UOptimusDeformer>(InRoot);

	UOptimusResourceDescription* Resource = Deformer->CreateResourceDirect(ResourceName);
	if (!Resource)
	{
		return false;
	}

	// The name should not have changed.
	check(Resource->GetFName() == ResourceName);

	Resource->ResourceName = Resource->GetFName();
	Resource->DataType = DataType;
	Resource->DataDomain = DataDomain;

	if (!Deformer->AddResourceDirect(Resource, INDEX_NONE))
	{
		Optimus::RemoveObject(Resource);
		return false;
	}
	
	ResourceName = Resource->GetFName();
	return true;
}


bool FOptimusResourceAction_AddResource::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusResourceDescription* Resource = GetResource(InRoot);
	if (!Resource)
	{
		return false;
	}

	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();
	return Deformer && Deformer->RemoveResourceDirect(Resource);
}


FOptimusResourceAction_RemoveResource::FOptimusResourceAction_RemoveResource(
	const UOptimusResourceDescription* InResource
	)
{
	if (ensure(InResource))
	{
		ResourceName = InResource->GetFName();
		DataType = InResource->DataType;
		ResourceIndex = InResource->GetIndex();

		SetTitlef(TEXT("Remove resource '%s'"), *InResource->GetName());
	}
}


bool FOptimusResourceAction_RemoveResource::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusResourceDescription* Resource = InRoot->ResolveResource(ResourceName);
	if (!Resource)
	{
		return false;
	}

	{
		Optimus::FBinaryObjectWriter ResourceArchive(Resource, ResourceData);
	}

	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();
	return Deformer && Deformer->RemoveResourceDirect(Resource);
}


bool FOptimusResourceAction_RemoveResource::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	
	UOptimusResourceDescription* Resource = Deformer->CreateResourceDirect(ResourceName);
	if (!Resource)
	{
		return false;
	}

	// The names should match since the name should have remained unique.
	check(Resource->GetFName() == ResourceName);

	// Fill in the stored data
	{
		Optimus::FBinaryObjectReader ResourceArchive(Resource, ResourceData);
	}

	if (!Deformer->AddResourceDirect(Resource, ResourceIndex))
	{
		Optimus::RemoveObject(Resource);
		return false;
	}
	
	return true;
}


FOptimusResourceAction_RenameResource::FOptimusResourceAction_RenameResource(
	UOptimusResourceDescription* InResource, 
	FName InNewName
	)
{
	if (ensure(InResource))
	{
		OldName = InResource->GetFName();
		NewName = InNewName;

		SetTitlef(TEXT("Rename resource to '%s'"), *NewName.ToString());
	}
}


bool FOptimusResourceAction_RenameResource::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusResourceDescription* Resource = InRoot->ResolveResource(OldName);
	if (!Resource)
	{
		return false;
	}
	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();
	
	return Deformer && Deformer->RenameResourceDirect(Resource, NewName);
}


bool FOptimusResourceAction_RenameResource::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusResourceDescription* Resource = InRoot->ResolveResource(NewName);
	if (!Resource)
	{
		return false;
	}
	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();

	return Deformer && Deformer->RenameResourceDirect(Resource, OldName);
}


FOptimusResourceAction_SetDataType::FOptimusResourceAction_SetDataType(
	UOptimusResourceDescription* InResource,
	FOptimusDataTypeRef InDataType
	)
{
	if (ensure(InResource) && ensure(InDataType.IsValid()))
	{
		ResourceName = InResource->GetFName();
		NewDataType = InDataType;
		OldDataType = InResource->DataType;

		SetTitlef(TEXT("Set Resource Data Type"));
	}
}


bool FOptimusResourceAction_SetDataType::Do(IOptimusPathResolver* InRoot)
{
	return SetDataType(InRoot, NewDataType);
}


bool FOptimusResourceAction_SetDataType::Undo(IOptimusPathResolver* InRoot)
{
	return SetDataType(InRoot, OldDataType);
}


bool FOptimusResourceAction_SetDataType::SetDataType(
	IOptimusPathResolver* InRoot,
	FOptimusDataTypeRef InDataType
	) const
{
	UOptimusResourceDescription* Resource = InRoot->ResolveResource(ResourceName);
	if (!Resource)
	{
		return false;
	}
	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();
	
	return Deformer && Deformer->SetResourceDataTypeDirect(Resource, InDataType);
}


FOptimusResourceAction_SetDataDomain::FOptimusResourceAction_SetDataDomain(
	UOptimusResourceDescription* InResource,
	const FOptimusDataDomain& InDataDomain
	)
{
	if (ensure(InResource))
	{
		ResourceName = InResource->GetFName();
		NewDataDomain = InDataDomain;
		OldDataDomain = InResource->DataDomain;

		SetTitlef(TEXT("Set Resource Data Domain"));
	}
}


bool FOptimusResourceAction_SetDataDomain::Do(
	IOptimusPathResolver* InRoot
	)
{
	return SetDataDomain(InRoot, NewDataDomain);
}


bool FOptimusResourceAction_SetDataDomain::Undo(
	IOptimusPathResolver* InRoot
	)
{
	return SetDataDomain(InRoot, OldDataDomain);
}


bool FOptimusResourceAction_SetDataDomain::SetDataDomain(
	IOptimusPathResolver* InRoot,
	const FOptimusDataDomain& InDataDomain
	) const
{
	UOptimusResourceDescription* Resource = InRoot->ResolveResource(ResourceName);
	if (!Resource)
	{
		return false;
	}
	UOptimusDeformer* Deformer = Resource->GetOwningDeformer();
	
	return Deformer && Deformer->SetResourceDataDomainDirect(Resource, InDataDomain);
}
