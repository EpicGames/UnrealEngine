// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusAction.h"
#include "OptimusDataDomain.h"

#include "OptimusDataType.h"

#include "OptimusResourceActions.generated.h"


class UOptimusDeformer;
class UOptimusResourceDescription;


USTRUCT()
struct FOptimusResourceAction_AddResource : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusResourceAction_AddResource() = default;

	FOptimusResourceAction_AddResource(
		FOptimusDataTypeRef InDataType,
		FName InName,
		FOptimusDataDomain InDataDomain = FOptimusDataDomain()
	);

	UOptimusResourceDescription* GetResource(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the resource to create.
	FName ResourceName;

	// The data type of the resource
	FOptimusDataTypeRef DataType;

	// The desired data domain of the resource
	FOptimusDataDomain DataDomain;
};


USTRUCT()
struct FOptimusResourceAction_RemoveResource :
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusResourceAction_RemoveResource() = default;

	FOptimusResourceAction_RemoveResource(
		const UOptimusResourceDescription* InResource
	);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the resource to re-create
	FName ResourceName;

	// The data type of the resource
	FOptimusDataTypeRef DataType;

	// The index of the removed resource within the resource list
	int32 ResourceIndex = 0;

	// The stored resource data.
	TArray<uint8> ResourceData;
};


USTRUCT()
struct FOptimusResourceAction_RenameResource : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusResourceAction_RenameResource() = default;

	FOptimusResourceAction_RenameResource(
	    UOptimusResourceDescription* InResource,
		FName InNewName);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The new name to give the resource.
	FName NewName;

	// The old name of the resource.
	FName OldName;
};


USTRUCT()
struct FOptimusResourceAction_SetDataType : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusResourceAction_SetDataType() = default;

	FOptimusResourceAction_SetDataType(
	    UOptimusResourceDescription* InResource,
		FOptimusDataTypeRef InDataType
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	bool SetDataType(
		IOptimusPathResolver* InRoot, 
		FOptimusDataTypeRef InDataType
		) const;

	// The name of the resource to update.
	FName ResourceName;

	// The data type of the resource
	FOptimusDataTypeRef NewDataType;

	// The data type of the resource
	FOptimusDataTypeRef OldDataType;
};


USTRUCT()
struct FOptimusResourceAction_SetDataDomain : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusResourceAction_SetDataDomain() = default;

	FOptimusResourceAction_SetDataDomain(
		UOptimusResourceDescription* InResource,
		const FOptimusDataDomain& InDataDomain
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	bool SetDataDomain(
		IOptimusPathResolver* InRoot, 
		const FOptimusDataDomain& InDataDomain
		) const;

	// The name of the resource to update.
	FName ResourceName;

	// The data domain of the resource
	FOptimusDataDomain NewDataDomain;

	// The data type of the resource
	FOptimusDataDomain OldDataDomain;
};
