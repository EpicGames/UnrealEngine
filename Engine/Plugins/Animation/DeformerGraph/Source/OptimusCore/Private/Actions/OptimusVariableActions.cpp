// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusVariableActions.h"

#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "OptimusVariableDescription.h"
#include "OptimusValueContainer.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusVariableActions)


FOptimusVariableAction_AddVariable::FOptimusVariableAction_AddVariable(
	FOptimusDataTypeRef InDataType, 
	FName InName
	)
{
	VariableName = InName;
	DataType = InDataType;

	SetTitlef(TEXT("Add variable '%s'"), *VariableName.ToString());
}


UOptimusVariableDescription* FOptimusVariableAction_AddVariable::GetVariable(
	IOptimusPathResolver* InRoot
	) const
{
	return InRoot->ResolveVariable(VariableName);
}


bool FOptimusVariableAction_AddVariable::Do(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);

	UOptimusVariableDescription* Variable = Deformer->CreateVariableDirect(VariableName);
	if (!Variable)
	{
		return false;
	}

	// The name should not have changed.
	check(Variable->GetFName() == VariableName);


	Variable->VariableName = Variable->GetFName();
	Variable->SetDataType(DataType);

	if (!Deformer->AddVariableDirect(Variable, INDEX_NONE))
	{
		Optimus::RemoveObject(Variable);
		return false;
	}
	
	VariableName = Variable->GetFName();
	return true;
}


bool FOptimusVariableAction_AddVariable::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusVariableDescription* Variable = GetVariable(InRoot);
	if (!Variable)
	{
		return false;
	}

	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	return Deformer->RemoveVariableDirect(Variable);
}


FOptimusVariableAction_RemoveVariable::FOptimusVariableAction_RemoveVariable(
	UOptimusVariableDescription* InVariable
	)
{
	if (ensure(InVariable))
	{
		VariableName = InVariable->GetFName();
		DataType = InVariable->DataType;
		VariableIndex = InVariable->GetIndex();

		SetTitlef(TEXT("Remove variable '%s'"), *InVariable->GetName());
	}
}


bool FOptimusVariableAction_RemoveVariable::Do(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);

	UOptimusVariableDescription* Variable = Deformer->ResolveVariable(VariableName);
	if (!Variable)
	{
		return false;
	}

	{
		Optimus::FBinaryObjectWriter VarArchive(Variable, VariableData);
	}

	return Deformer->RemoveVariableDirect(Variable);
}


bool FOptimusVariableAction_RemoveVariable::Undo(IOptimusPathResolver* InRoot)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);

	UOptimusVariableDescription* Variable = Deformer->CreateVariableDirect(VariableName);
	if (!Variable)
	{
		return false;
	}

	// The names should match since the name should have remained unique.
	check(Variable->GetFName() == VariableName);

	// Fill in the stored data
	{
		Optimus::FBinaryObjectReader VarArchive(Variable, VariableData);
	}
	
	if (!Deformer->AddVariableDirect(Variable, VariableIndex))
	{
		Optimus::RemoveObject(Variable);
		return false;
	}
	
	return true;
}


FOptimusVariableAction_RenameVariable::FOptimusVariableAction_RenameVariable(
	UOptimusVariableDescription* InVariable, 
	FName InNewName
	)
{
	if (ensure(InVariable))
	{
		OldName = InVariable->GetFName();
		NewName = InNewName;

		SetTitlef(TEXT("Rename variable to '%s'"), *NewName.ToString());
	}
}


bool FOptimusVariableAction_RenameVariable::Do(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	UOptimusVariableDescription* Variable = Deformer->ResolveVariable(OldName);

	return Variable && Deformer->RenameVariableDirect(Variable, NewName);
}


bool FOptimusVariableAction_RenameVariable::Undo(
	IOptimusPathResolver* InRoot
	)
{
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InRoot);
	UOptimusVariableDescription* Variable = Deformer->ResolveVariable(NewName);

	return Variable && Deformer->RenameVariableDirect(Variable, OldName);
}



FOptimusVariableAction_SetDataType::FOptimusVariableAction_SetDataType(
	UOptimusVariableDescription* InVariable,
	FOptimusDataTypeRef InDataType
	)
{
	if (ensure(InVariable) && ensure(InDataType.IsValid()))
	{
		VariableName = InVariable->GetFName();
		NewDataType = InDataType;
		OldDataType = InVariable->DataType;

		SetTitlef(TEXT("Set Variable Data Type"));
	}
}


bool FOptimusVariableAction_SetDataType::Do(IOptimusPathResolver* InRoot)
{
	return SetDataType(InRoot, NewDataType);
}


bool FOptimusVariableAction_SetDataType::Undo(IOptimusPathResolver* InRoot)
{
	return SetDataType(InRoot, OldDataType);
}


bool FOptimusVariableAction_SetDataType::SetDataType(
	IOptimusPathResolver* InRoot,
	FOptimusDataTypeRef InDataType
	) const
{
	UOptimusVariableDescription* Variable = InRoot->ResolveVariable(VariableName);
	if (!Variable)
	{
		return false;
	}
	UOptimusDeformer* Deformer = Variable->GetOwningDeformer();
	
	return Deformer && Deformer->SetVariableDataTypeDirect(Variable, InDataType);
}
