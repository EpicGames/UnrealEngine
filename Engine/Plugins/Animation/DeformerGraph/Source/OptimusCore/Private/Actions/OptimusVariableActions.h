// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusAction.h"

#include "OptimusDataType.h"

#include "OptimusVariableActions.generated.h"


class UOptimusDeformer;
class UOptimusVariableDescription;


USTRUCT()
struct FOptimusVariableAction_AddVariable : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusVariableAction_AddVariable() = default;

	FOptimusVariableAction_AddVariable(
	    FOptimusDataTypeRef InDataType,
	    FName InName
		);

	UOptimusVariableDescription* GetVariable(IOptimusPathResolver* InRoot) const;

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the variable to create.
	FName VariableName;

	// The data type of the variable
	FOptimusDataTypeRef DataType;
};


USTRUCT()
struct FOptimusVariableAction_RemoveVariable : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusVariableAction_RemoveVariable() = default;

	FOptimusVariableAction_RemoveVariable(
	    UOptimusVariableDescription* InVariable);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The name of the variable to remove and re-create on undo.
	FName VariableName;

	// The data type of the variable
	FOptimusDataTypeRef DataType;

	// THe index of the variable in the list
	int32 VariableIndex = 0;
	
	// The stored variable data.
	TArray<uint8> VariableData;

	// The stored default value data
	TArray<uint8> DefaultValueData;
};


USTRUCT()
struct FOptimusVariableAction_RenameVariable : 
	public FOptimusAction
{
	GENERATED_BODY()

public:
	FOptimusVariableAction_RenameVariable() = default;

	FOptimusVariableAction_RenameVariable(
	    UOptimusVariableDescription* InVariable,
	    FName InNewName);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	// The new name to give the variable.
	FName NewName;

	// The old name of the variable.
	FName OldName;
};


USTRUCT()
struct FOptimusVariableAction_SetDataType : 
	public FOptimusAction
{
	GENERATED_BODY()

	FOptimusVariableAction_SetDataType() = default;

	FOptimusVariableAction_SetDataType(
		UOptimusVariableDescription* InVariable,
		FOptimusDataTypeRef InDataType
		);

protected:
	bool Do(IOptimusPathResolver* InRoot) override;
	bool Undo(IOptimusPathResolver* InRoot) override;

private:
	bool SetDataType(
		IOptimusPathResolver* InRoot, 
		FOptimusDataTypeRef InDataType
		) const;

	// The name of the variable to update.
	FName VariableName;

	// The new data type to give the variable, used when running Do.
	FOptimusDataTypeRef NewDataType;

	// The old data type of the variable, used when running Undo.
	FOptimusDataTypeRef OldDataType;
};
