// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSceneComponentSource.h"

#include "Components/SceneComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusSceneComponentSource)


#define LOCTEXT_NAMESPACE "OptimusSceneComponentSource"


FText UOptimusSceneComponentSource::GetDisplayName() const
{
	return LOCTEXT("SceneComponent", "Scene Component");
}


TSubclassOf<UActorComponent> UOptimusSceneComponentSource::GetComponentClass() const
{
	return USceneComponent::StaticClass();
}


TArray<FName> UOptimusSceneComponentSource::GetExecutionDomains() const
{
	return {};
}


#undef LOCTEXT_NAMESPACE
