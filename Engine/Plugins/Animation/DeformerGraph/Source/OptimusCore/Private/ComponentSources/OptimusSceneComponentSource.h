// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComponentSource.h"

#include "OptimusSceneComponentSource.generated.h"


UCLASS()
class UOptimusSceneComponentSource :
	public UOptimusComponentSource
{
	GENERATED_BODY()
public:
	// UOptimusComponentSource implementations
	FText GetDisplayName() const override;
	FName GetBindingName() const override { return FName("Scene"); }
	TSubclassOf<UActorComponent> GetComponentClass() const override;
	TArray<FName> GetExecutionDomains() const override;
};
