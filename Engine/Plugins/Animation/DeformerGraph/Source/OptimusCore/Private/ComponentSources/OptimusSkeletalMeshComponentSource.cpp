// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSkeletalMeshComponentSource.h"

#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusSkeletalMeshComponentSource)


#define LOCTEXT_NAMESPACE "OptimusSkeletalMeshComponentSource"


TSubclassOf<UActorComponent> UOptimusSkeletalMeshComponentSource::GetComponentClass() const
{
	return USkeletalMeshComponent::StaticClass();
}


FText UOptimusSkeletalMeshComponentSource::GetDisplayName() const
{
	return LOCTEXT("SkeletalMeshComponent", "Skeletal Mesh Component");
}


#undef LOCTEXT_NAMESPACE
