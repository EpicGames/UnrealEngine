// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusSkinnedMeshComponentSource.h"

#include "OptimusSkeletalMeshComponentSource.generated.h"


UCLASS()
class UOptimusSkeletalMeshComponentSource :
	public UOptimusSkinnedMeshComponentSource
{
	GENERATED_BODY()
public:
	// UOptimusComponentSource implementations
	FText GetDisplayName() const override;
	FName GetBindingName() const override { return FName("SkeletalMesh"); }
	TSubclassOf<UActorComponent> GetComponentClass() const override;
};
