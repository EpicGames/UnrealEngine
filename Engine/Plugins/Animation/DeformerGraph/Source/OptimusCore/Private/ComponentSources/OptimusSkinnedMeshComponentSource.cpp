// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSkinnedMeshComponentSource.h"

#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusSkinnedMeshComponentSource)


#define LOCTEXT_NAMESPACE "OptimusSkinnedMeshComponentSource"


FName UOptimusSkinnedMeshComponentSource::Domains::Vertex("Vertex");
FName UOptimusSkinnedMeshComponentSource::Domains::Triangle("Triangle");
FName UOptimusSkinnedMeshComponentSource::Domains::DuplicateVertex("DuplicateVertex");


FText UOptimusSkinnedMeshComponentSource::GetDisplayName() const
{
	return LOCTEXT("SkinnedMeshComponent", "Skinned Mesh Component");
}

TSubclassOf<UActorComponent> UOptimusSkinnedMeshComponentSource::GetComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


TArray<FName> UOptimusSkinnedMeshComponentSource::GetExecutionDomains() const
{
	return {Domains::Vertex, Domains::Triangle};
}

int32 UOptimusSkinnedMeshComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(InComponent);
	return SkinnedMeshComponent ? SkinnedMeshComponent->GetPredictedLODLevel() : 0;
}

uint32 UOptimusSkinnedMeshComponentSource::GetDefaultNumInvocations(const UActorComponent* InComponent,
                                                                    int32 InLod) const
{
	const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(InComponent);
	if (!SkinnedMeshComponent)
	{
		return 0;
	}

	const FSkeletalMeshObject* SkeletalMeshObject = SkinnedMeshComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return 0;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[InLod];

	return LodRenderData->RenderSections.Num();
}

bool UOptimusSkinnedMeshComponentSource::GetComponentElementCountsForExecutionDomain(
	FName InDomainName,
	const UActorComponent* InComponent,
	int32 InLodIndex,
	TArray<int32>& OutInvocationElementCounts
	) const
{
	const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(InComponent);
	if (!SkinnedMeshComponent)
	{
		return false;
	}

	const FSkeletalMeshObject* SkeletalMeshObject = SkinnedMeshComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return false;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[InLodIndex];

	OutInvocationElementCounts.Reset();
#if 0
	// FIXME: Commenting out for now. An execution domains for bones is currently ill-defined.
	// -- Should they be per-section?
	// -- If just a single invocation, what happens when they get intermingled in expressions with vertex/triangle? 
	if (InDomainName == UOptimusSkinnedMeshComponentSource::Domains::Bone)
	{
		// TODO: We need better way to iterate over all active bones. 
		OutInvocationElementCounts.Add(static_cast<int32>(LodRenderData->GetSkinWeightVertexBuffer()->GetMaxBoneInfluences()));
		return true;
	}
#endif
	
	if (InDomainName == Domains::Triangle || InDomainName == Domains::Vertex)
	{
		const int32 NumInvocations = LodRenderData->RenderSections.Num();

		OutInvocationElementCounts.Reset();
		OutInvocationElementCounts.Reserve(NumInvocations);
		for (int32 InvocationIndex = 0; InvocationIndex < NumInvocations; ++InvocationIndex)
		{
			FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];
			const int32 NumThreads = InDomainName == Domains::Vertex ? RenderSection.NumVertices : RenderSection.NumTriangles;
			OutInvocationElementCounts.Add(NumThreads);
		}
	
		return true;
	}

	// Unknown execution domain.
	return false;
}

#undef LOCTEXT_NAMESPACE
