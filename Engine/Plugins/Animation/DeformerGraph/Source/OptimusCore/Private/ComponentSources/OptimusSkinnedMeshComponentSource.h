// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComponentSource.h"

#include "OptimusSkinnedMeshComponentSource.generated.h"


UCLASS()
class UOptimusSkinnedMeshComponentSource :
	public UOptimusComponentSource
{
	GENERATED_BODY()
public:
	struct Domains
	{
		static FName Vertex;
		static FName Triangle;
		static FName DuplicateVertex;
	};
	
	// UOptimusComponentSource implementations
	FText GetDisplayName() const override;
	FName GetBindingName() const override { return FName("SkinnedMesh"); }
	TSubclassOf<UActorComponent> GetComponentClass() const override;
	TArray<FName> GetExecutionDomains() const override;
	int32 GetLodIndex(const UActorComponent* InComponent) const override;
	uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	bool GetComponentElementCountsForExecutionDomain(FName InDomainName, const UActorComponent* InComponent, int32 InLodIndex, TArray<int32>& OutInvocationElementCounts) const override;
};
