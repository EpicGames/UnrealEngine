// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigUnit_Optimus.h"

#include "Components/SkeletalMeshComponent.h"

#if WITH_EDITOR
#include "RigVMModel/RigVMController.h"
#endif 
#include "Units/RigUnitContext.h"
#include "ControlRig.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusHelpers.h"
#include "OptimusVariableDescription.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Optimus)

const TCHAR* FRigUnit_AddOptimusDeformer::DeformerTraitName = TEXT("DeformerGraphAsset");
const TCHAR* FRigUnit_AddOptimusDeformer::DeformerSettingsTraitName = TEXT("Settings");

FString FRigVMTrait_OptimusDeformer::GetDisplayName() const
{
	return TEXT("Deformer Graph Asset");
}

#if WITH_EDITOR
bool FRigVMTrait_OptimusDeformer::ShouldCreatePinForProperty(const FProperty* InProperty) const
{
	if(!Super::ShouldCreatePinForProperty(InProperty))
	{
		return false;
	}
	
	return InProperty->GetFName() != GET_MEMBER_NAME_CHECKED(FRigVMTrait_OptimusDeformer, DeformerGraph);
}
#endif


bool FRigUnit_AddOptimusDeformer::IsVariableTraitName(const FString& InTraitName)
{
	return InTraitName != DeformerTraitName &&
		InTraitName != DeformerSettingsTraitName;
}


void FRigUnit_AddOptimusDeformer::OnUnitNodeCreated(FRigVMUnitNodeCreatedContext& InContext) const
{
	FRigUnitMutable::OnUnitNodeCreated(InContext);
#if WITH_EDITOR
	if (InContext.GetReason() == ERigVMNodeCreatedReason::NodeSpawner)
	{
		InContext.GetController()->AddTrait(InContext.GetNodeName(), *FRigVMTrait_OptimusDeformer::StaticStruct()->GetPathName(), DeformerTraitName);
		InContext.GetController()->AddTrait(InContext.GetNodeName(), *FRigVMTrait_OptimusDeformerSettings::StaticStruct()->GetPathName(), DeformerSettingsTraitName);
	}
#endif
}

TArray<FRigVMUserWorkflow> FRigUnit_AddOptimusDeformer::GetSupportedWorkflows(const UObject* InSubject) const
{
	TArray<FRigVMUserWorkflow> Workflows = Super::GetSupportedWorkflows(InSubject);
	
#if WITH_EDITOR

	Workflows.Emplace(TEXT("Refresh Variables"), TEXT("Populate the node with available variables in the Deformer Graph"), ERigVMUserWorkflowType::NodeContext,
		FRigVMPerformUserWorkflowDelegate::CreateLambda([](const URigVMUserWorkflowOptions* InOptions, UObject* InController)
		{
			URigVMController* Controller = CastChecked<URigVMController>(InController);
			if(URigVMNode* Node = InOptions->GetSubject<URigVMNode>())
			{
				TSharedPtr<FStructOnScope> StructOnScope = Node->GetTraitInstance(DeformerTraitName, true);
				FRigVMTrait_OptimusDeformer* TraitInstance = reinterpret_cast<FRigVMTrait_OptimusDeformer*>(StructOnScope->GetStructMemory());
				UOptimusDeformer* Deformer = TraitInstance->DeformerGraph.LoadSynchronous();
				
				// TODO: Ideally pin display name should update immediately when the asset changes
				Controller->SetPinDisplayName(Node->FindTrait(DeformerTraitName)->GetPinPath(),
					Deformer ?
					TraitInstance->DeformerGraph.GetAssetName() : TEXT("Deformer Graph Asset(Unassigned)")) ;

				const TArray<URigVMController::FLinkedPath> LinkedPaths = URigVMController::GetLinkedPaths(Node);
				TMap<FString, URigVMController::FPinState> PinStates = Controller->GetPinStates(Node);
				
				TArray<FString> TraitNames = Node->GetTraitNames();

				for (const FString& TraitName : TraitNames)
				{
					if (IsVariableTraitName(TraitName))
					{
						Controller->RemoveTrait(Node->GetFName(), *TraitName);
					}
				}
				
				if (Deformer)
				{
					for (UOptimusVariableDescription* Variable : Deformer->GetVariables())
					{
						FName TraitName = NAME_None;
						if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FIntProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerIntVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FIntProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerIntArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntPoint>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt2Variable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntPoint>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt2ArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt3Variable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntVector>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt3ArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector4>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt4Variable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FIntVector4>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerInt4ArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FDoubleProperty::StaticClass()) ||
							Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerFloatVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FDoubleProperty::StaticClass()) ||
							Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FFloatProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerFloatArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector2D>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVector2Variable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector2D>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVector2ArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVectorVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVectorArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FVector4>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVector4Variable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FVector4>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerVector4ArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FLinearColor>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerLinearColorVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FLinearColor>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerLinearColorArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FQuat>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerQuatVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FQuat>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerQuatArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FRotator>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerRotatorVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FRotator>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerRotatorArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FTransform>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerTransformVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(TBaseStructure<FTransform>::Get()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerTransformArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerNameVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FNameProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerNameArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindType(*FBoolProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerBoolVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						else if (Variable->DataType == FOptimusDataTypeRegistry::Get().FindArrayType(*FBoolProperty::StaticClass()))
						{
							TraitName = Controller->AddTrait(Node->GetFName(), *FRigVMTrait_SetDeformerBoolArrayVariable::StaticStruct()->GetPathName(), Variable->VariableName);
						}
						
						Controller->SetPinExpansion(Node->FindTrait(TraitName)->GetPinPath(), true, true);
						const FString& DefaultValueString = Variable->DefaultValueStruct.GetValueAsString();
						// Array pin's value string from DefaultValueStruct can be empty if it is empty, but control rig expects "()" instead of empty default value,
						// better to simply use control rig supplied default value in this case
						if (!DefaultValueString.IsEmpty())
						{
							Controller->SetPinDefaultValue(Node->FindTrait(TraitName, FRigVMTrait_OptimusVariableBase::GetValuePinName())->GetPinPath(), DefaultValueString);
						}
					}
				}

				Controller->ApplyPinStates(Node, PinStates, {}, true);
				Controller->RestoreLinkedPaths(LinkedPaths, {}, true);

				return true;
			}
			return false;
		}), URigVMUserWorkflowOptions::StaticClass());
#endif
	
	return Workflows;
	
}


FRigUnit_AddOptimusDeformer_Execute()
{
	if (!DeformerInstanceGuid.IsValid())
	{
		DeformerInstanceGuid = FGuid::NewGuid();
	}
	
	USkeletalMeshComponent* RigMeshComponent = Cast<USkeletalMeshComponent>(ExecuteContext.GetMutableOwningComponent());
	
	if(RigMeshComponent)
	{
		const TArrayView<const FRigVMTraitScope> Traits = ExecuteContext.GetTraits();
		
		// Deformer Graph Trait spawns a task to adds the deformer on game thread
		// Settings Trait enqueues the deformer based on execution settings
		// Variable Traits sets variable value on deformer instances created by this rig unit

		// By default, a deformer is added to all child components if possible.
		// While Add is done on game thread, enqueue and set variable are done on anim thread
		// Given that parent component always ticks before child components, it should be safe for the parent
		// component to modify the deformer instance manager on the child components

		const FRigVMTrait_OptimusDeformerSettings* SettingsTraitPtr = nullptr;
		
		for(const FRigVMTraitScope& Scope : Traits)
		{
			if (const FRigVMTrait_OptimusDeformerSettings* SettingsTrait = Scope.GetTrait<FRigVMTrait_OptimusDeformerSettings>())
			{
				SettingsTraitPtr = SettingsTrait;
				break;
			}
		}

		check(SettingsTraitPtr != nullptr);
		
		struct Local
		{
			static TArray<USkeletalMeshComponent*> GetComponentsToProcess(USkeletalMeshComponent* RigMeshComponent, bool DeformChildComponents, FName ExcludeChildComponentsWithTag)
			{
				TArray<USkeletalMeshComponent*> ComponentsToProcess = {RigMeshComponent};
				
				if (DeformChildComponents)
				{
					TArray<USceneComponent*> ChildComponents;
					RigMeshComponent->GetChildrenComponents(true, ChildComponents);
					
					const AActor* RigMeshActor = RigMeshComponent->GetOwner();
					for (USceneComponent* Component : ChildComponents)
					{
						if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
						{
							const AActor* ChildMeshActor = SkeletalMeshComponent->GetOwner();						
#if WITH_EDITOR
							const bool bIsTempActor = ChildMeshActor && ChildMeshActor != RigMeshActor && ChildMeshActor->bIsEditorPreviewActor;
#else
							constexpr bool bIsTempActor = false;
#endif						
							if (!bIsTempActor)
							{
								if (!Component->ComponentHasTag(ExcludeChildComponentsWithTag))
								{
									ComponentsToProcess.Add(SkeletalMeshComponent);
								}
							}
						}
					}	
				}

				return ComponentsToProcess;
			}
		};
		
		TArray<USkeletalMeshComponent*> ComponentsToProcess =
			Local::GetComponentsToProcess(
				RigMeshComponent, SettingsTraitPtr->DeformChildComponents, SettingsTraitPtr->ExcludeChildComponentsWithTag);
				
		for (const USkeletalMeshComponent* ComponentToProcess : ComponentsToProcess)
		{
			// Currently, there is only one deformer instance used by all LODs, so use LOD 0 here for now
			// This might change in the future depending on how per-LOD instance is implemented
			if (UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = Cast<UOptimusDeformerDynamicInstanceManager>(ComponentToProcess->GetMeshDeformerInstanceForLOD(0)))
			{
				DeformerInstanceManager->EnqueueProducerDeformer(DeformerInstanceGuid, SettingsTraitPtr->ExecutionPhase, SettingsTraitPtr->ExecutionGroup);
			}	
		}
		
		for(const FRigVMTraitScope& Scope : Traits)
		{
			if(const FRigVMTrait_OptimusDeformer* DeformerTrait = Scope.GetTrait<FRigVMTrait_OptimusDeformer>())
			{
				if (!DeformerTrait->DeformerGraph.IsNull())
				{
					auto AddDeformerToComponentsIfNeeded = [
						WeakMesh = TWeakObjectPtr<USkeletalMeshComponent>(RigMeshComponent),
						WeakRig = TWeakObjectPtr<UControlRig>(ExecuteContext.ControlRig),
						DeformerInstanceGuid,
						DeformChildComponents = SettingsTraitPtr->DeformChildComponents,
						ExcludeChildComponentsWithTag = SettingsTraitPtr->ExcludeChildComponentsWithTag
						](UOptimusDeformer* DeformerGraph)
						{
							check(IsInGameThread());

							if (WeakMesh.IsValid() && WeakRig.IsValid())
							{
								USkeletalMeshComponent* RigMeshComponent = WeakMesh.Get();
							
								TArray<USkeletalMeshComponent*> ComponentsToProcess =
									Local::GetComponentsToProcess(RigMeshComponent, DeformChildComponents, ExcludeChildComponentsWithTag);

								for (USkeletalMeshComponent* ComponentToProcess : ComponentsToProcess)
								{
									// Currently only one deformer for all LODs so using LOD0 for now is fine
									// It may change in the future depending on how per-LOD deformer instance is implemented
									if (!ComponentToProcess->GetMeshDeformerInstanceForLOD(0))
									{
										// Every time we re-set the mesh deformers we have to wait for anim eval to complete to avoid anim thread accessing
										// deformer instances while we are trying to modify it. Similar to OnUnregister for skeletal mesh component
										ComponentToProcess->HandleExistingParallelEvaluationTask(true, false);
								
										// In case there is no mesh deformer running, force turning it on and use a default deformer
										// in project settings
										ComponentToProcess->SetAlwaysUseMeshDeformer(true);
									}
						
									if (UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = Cast<UOptimusDeformerDynamicInstanceManager>(ComponentToProcess->GetMeshDeformerInstanceForLOD(0)))
									{
										UControlRig* Rig = WeakRig.Get();
										if (!DeformerInstanceManager->GetDeformerInstance(DeformerInstanceGuid))
										{
											DeformerInstanceManager->AddProducerDeformer(Rig, DeformerInstanceGuid, DeformerGraph);
										}
									}
								}
							}	
						};

					if (UOptimusDeformer* DeformerGraph = DeformerTrait->DeformerGraph.Get())
					{
						ExecuteOnGameThread(UE_SOURCE_LOCATION,
							[AddDeformerToComponentsIfNeeded, WeakDeformer = TWeakObjectPtr<UOptimusDeformer>(DeformerGraph)]()
								{
									if (WeakDeformer.IsValid())
									{
										AddDeformerToComponentsIfNeeded(WeakDeformer.Get());
									}
								});
					}
					else
					{
						DeformerTrait->DeformerGraph.LoadAsync(
							FLoadSoftObjectPathAsyncDelegate::CreateLambda(
								[AddDeformerToComponentsIfNeeded](const FSoftObjectPath&, UObject* InLoadedObject)
								{
									if (UOptimusDeformer* DeformerGraph = Cast<UOptimusDeformer>(InLoadedObject))
									{
										AddDeformerToComponentsIfNeeded(DeformerGraph);
									}
								}));	
					}
				
				}	
			}
			else if (const FRigVMTrait_OptimusVariableBase* VariableTrait = Scope.GetTrait<FRigVMTrait_OptimusVariableBase>())
			{
				for (const USkeletalMeshComponent* ComponentToProcess : ComponentsToProcess)
				{
					// Currently, there is only one deformer instance used by all LODs, so use LOD 0 here for now
					// This might change in the future depending on how per-LOD instance is implemented
					if (UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = Cast<UOptimusDeformerDynamicInstanceManager>(ComponentToProcess->GetMeshDeformerInstanceForLOD(0)))
					{
						if (DeformerInstanceGuid.IsValid())
						{
							if (UOptimusDeformerInstance* DeformerInstance = DeformerInstanceManager->GetDeformerInstance(DeformerInstanceGuid))
							{
								VariableTrait->SetValue(DeformerInstance);
							}
						}
					}
				}
			}
		}
	}
}

const TCHAR* FRigVMTrait_OptimusVariableBase::GetValuePinName()
{
	return TEXT("Value");
}

void FRigVMTrait_SetDeformerIntVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetIntVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerIntArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetIntArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt2Variable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt2Variable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt2ArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt2ArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt3Variable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt3Variable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt3ArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt3ArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt4Variable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt4Variable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerInt4ArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetInt4ArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerFloatVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetFloatVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerFloatArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetFloatArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVector2Variable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVector2Variable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVector2ArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVector2ArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVectorVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVectorVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVectorArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVectorArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVector4Variable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVector4Variable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerVector4ArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetVector4ArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerLinearColorVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetLinearColorVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerLinearColorArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetLinearColorArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerQuatVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetQuatVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerQuatArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetQuatArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerRotatorVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetRotatorVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerRotatorArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetRotatorArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerTransformVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetTransformVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerTransformArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetTransformArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerNameVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetNameVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerNameArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetNameArrayVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerBoolVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetBoolVariable(*GetName(), Value);
}

void FRigVMTrait_SetDeformerBoolArrayVariable::SetValue(UOptimusDeformerInstance* InInstance) const
{
	InInstance->SetBoolArrayVariable(*GetName(), Value);
}
