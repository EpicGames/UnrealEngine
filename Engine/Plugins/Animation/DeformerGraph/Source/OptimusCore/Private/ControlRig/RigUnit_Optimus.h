// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Units/RigUnit.h"
#include "RigVMCore/RigVMTrait.h"
#include "OptimusDeformer.h"
#include "OptimusDeformerDynamicInstanceManager.h"
#include "RigUnit_Optimus.generated.h"

#define UE_API OPTIMUSCORE_API

/*
 * The trait representing the optimus deformer kernel for Control Rig
 */
USTRUCT(meta=(DocumentationPolicy = "Strict"))
struct FRigVMTrait_OptimusDeformer: public FRigVMTrait
{
	GENERATED_BODY()

	FRigVMTrait_OptimusDeformer() = default;
	UE_API FString GetDisplayName() const override;
#if WITH_EDITOR
	UE_API bool ShouldCreatePinForProperty(const FProperty* InProperty) const override;
#endif

	UPROPERTY(EditAnywhere, Category = "Deformer Graph")
	TSoftObjectPtr<UOptimusDeformer> DeformerGraph;
};

/*
 * The trait representing the base settings for an optimus deformer in Control Rig
 */
USTRUCT(meta=(DocumentationPolicy = "Strict"))
struct FRigVMTrait_OptimusDeformerSettings: public FRigVMTrait
{
	GENERATED_BODY()

	FRigVMTrait_OptimusDeformerSettings() = default;

	// The phase at which the deformer will execute
	UPROPERTY(EditAnywhere, Category = "Trait", meta=(Input))
	EOptimusDeformerExecutionPhase ExecutionPhase = EOptimusDeformerExecutionPhase::AfterDefaultDeformer;
	
	// Deformers are first sorted by execution group index, then by the order in which they are added
	UPROPERTY(EditAnywhere, Category = "Trait", meta=(Input))
	int32 ExecutionGroup = 1;

	// Whether to apply the deformer to all child components as well
	UPROPERTY(EditAnywhere, Category = "Trait", meta=(Input))
	bool DeformChildComponents = true;

	// Deformer won't be applied to child components that have the specified component tag
	UPROPERTY(EditAnywhere, Category = "Trait", meta=(Input))
	FName ExcludeChildComponentsWithTag = NAME_None;
};

/** Adds a deformer to the Skeletal Mesh Component*/
USTRUCT(meta = (DisplayName = "Add Deformer", Category = "Deformer Graph", NodeColor="1 1 1"))
struct FRigUnit_AddOptimusDeformer: public FRigUnitMutable
{
	GENERATED_BODY()
	
	static UE_API const TCHAR* DeformerTraitName;
	static UE_API const TCHAR* DeformerSettingsTraitName;

	static UE_API bool IsVariableTraitName(const FString& InTraitName);

	FRigUnit_AddOptimusDeformer() = default;
	
	UE_API void OnUnitNodeCreated(FRigVMUnitNodeCreatedContext& InContext) const override;
	UE_API TArray<FRigVMUserWorkflow> GetSupportedWorkflows(const UObject* InSubject) const override;
	
	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Hidden))
	FGuid DeformerInstanceGuid;
};

/*
 * The base class for all optimus deformer variable traits for Control Rig
 */
USTRUCT(meta=(DocumentationPolicy = "Strict"))
struct FRigVMTrait_OptimusVariableBase: public FRigVMTrait
{
	GENERATED_BODY()

	virtual void SetValue(UOptimusDeformerInstance* InInstance) const {};
	static const TCHAR* GetValuePinName();
};

/*
 * A trait to set an Int variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerIntVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	int32 Value = 0;	
};

/*
 * A trait to set an Int array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerIntArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<int32> Value;	
};

/*
 * A trait to set an Int2 variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt2Variable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FIntPoint Value = FIntPoint::ZeroValue;	
};

/*
 * A trait to set an Int2 array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt2ArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FIntPoint> Value;	
};

/*
 * A trait to set an Int3 variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt3Variable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FIntVector Value = FIntVector::ZeroValue;	
};

/*
 * A trait to set an Int3 array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt3ArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FIntVector> Value;	
};

/*
 * A trait to set an Int4 variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt4Variable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FIntVector4 Value = FIntVector4::ZeroValue;	
};

/*
 * A trait to set an Int4 array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerInt4ArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FIntVector4> Value;	
};

/*
 * A trait to set a double variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerFloatVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()
	
	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	double Value = 0.0;	
};

/*
 * A trait to set a double array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerFloatArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()
	
	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<double> Value;	
};

/*
 * A trait to set a Vector2 variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVector2Variable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FVector2D Value = FVector2D::ZeroVector;	
};

/*
 * A trait to set a Vector2 array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVector2ArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FVector2D> Value;	
};

/*
 * A trait to set a Vector variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVectorVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FVector Value = FVector::ZeroVector; 
};

/*
 * A trait to set a Vector array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVectorArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FVector> Value;	
};

/*
 * A trait to set a Vector4 variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVector4Variable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FVector4 Value = FVector4::Zero();	
};

/*
 * A trait to set a Vector4 array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerVector4ArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FVector4> Value;	
};

/*
 * A trait to set a LinearColor variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerLinearColorVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FLinearColor Value = FLinearColor::Black;	
};

/*
 * A trait to set a LinearColor array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerLinearColorArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FLinearColor> Value;	
};

/*
 * A trait to set a Quat variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerQuatVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FQuat Value = FQuat::Identity;	
};

/*
 * A trait to set a Quat array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerQuatArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FQuat> Value;	
};

/*
 * A trait to set a Rotator variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerRotatorVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FRotator Value = FRotator::ZeroRotator;	
};

/*
 * A trait to set a Rotator array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerRotatorArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FRotator> Value;	
};

/*
 * A trait to set a Transform variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerTransformVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FTransform Value = FTransform::Identity;	
};

/*
 * A trait to set a Transform array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerTransformArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FTransform> Value;	
};

/*
 * A trait to set a Name variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerNameVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	FName Value = NAME_None;	
};

/*
 * A trait to set a Name array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerNameArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<FName> Value;	
};

/*
 * A trait to set a Bool variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerBoolVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	bool Value = false;	
};

/*
 * A trait to set a Bool array variable on an optimus deformer in Control Rig
 */
USTRUCT(BlueprintType)
struct FRigVMTrait_SetDeformerBoolArrayVariable: public FRigVMTrait_OptimusVariableBase
{
	GENERATED_BODY()

	UE_API void SetValue(UOptimusDeformerInstance* InInstance) const override;
	
	UPROPERTY(EditAnywhere, Category = "Trait")
	TArray<bool> Value;	
};

#undef UE_API
