// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceAdvancedSkeleton.h"

#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusHelpers.h"
#include "OptimusDataDomain.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"
#include "SystemTextures.h"
#include "Animation/SkinWeightProfileManager.h"
#include "Nodes/OptimusNode_DataInterface.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceAdvancedSkeleton)

const TCHAR* FOptimusAnimAttributeBufferDescription::PinNameDelimiter = TEXT(" - ");
const TCHAR* FOptimusAnimAttributeBufferDescription::HlslIdDelimiter = TEXT("_");


FOptimusAnimAttributeBufferDescription& FOptimusAnimAttributeBufferDescription::Init(const FString& InName, const FOptimusDataTypeRef& InDataType)
{
	Name = InName;
	DataType = InDataType;

	DefaultValueStruct.SetType(DataType.Resolve());
	// Caller should ensure that the name is unique
	HlslId = InName;
	PinName = *InName;
	
	return *this;
}


void FOptimusAnimAttributeBufferDescription::UpdatePinNameAndHlslId(bool bInIncludeTypeName)
{
	PinName = *GetFormattedId(PinNameDelimiter, bInIncludeTypeName);
	HlslId = GetFormattedId(HlslIdDelimiter, bInIncludeTypeName);
}

FString FOptimusAnimAttributeBufferDescription::GetFormattedId(
	const FString& InDelimiter, bool bInIncludeTypeName) const
{
	FString UniqueId;
			
	if (bInIncludeTypeName)
	{
		UniqueId += DataType.Resolve()->DisplayName.ToString();
		UniqueId += InDelimiter;
	}

	UniqueId += Name;	

	return  UniqueId;
}

FName UOptimusAdvancedSkeletonDataInterface::GetSkinWeightProfilePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusAdvancedSkeletonDataInterface, SkinWeightProfile);
}

FString UOptimusAdvancedSkeletonDataInterface::GetUnusedAttributeName(int32 CurrentAttributeIndex, const FString& InName) const
{
	TSet<FString> UsedNames;

	TArray<FOptimusCDIPinDefinition> PinDefinitions = GetPinDefinitions_Internal(true , CurrentAttributeIndex);
	for (const FOptimusCDIPinDefinition& Definition : PinDefinitions)
	{
		UsedNames.Add(Definition.PinName.ToString());
	}

	int32 Suffix = 0;
	FString NewName = InName;
	while (UsedNames.Contains(NewName))
	{
		NewName = FString::Printf(TEXT("%s_%d"), *InName, Suffix);
		Suffix++;
	}

	return NewName;
}

void UOptimusAdvancedSkeletonDataInterface::UpdateAttributePinNamesAndHlslIds()
{
	const int32 NumAttributes = AttributeBufferArray.Num(); 

	TMap<FString, TArray<int32>> AttributesByName;

	for (int32 Index = 0; Index < NumAttributes; Index++)
	{
		const FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];
		AttributesByName.FindOrAdd(Attribute.Name).Add(Index);
	}

	for (const TTuple<FString, TArray<int32>>& AttributeGroup : AttributesByName)
	{
		// For attributes that share the same name, prepend type name to make sure pin names are unique
		bool bMoreThanOneType = false;

		TOptional<FOptimusDataTypeRef> LastType;
		
		for (int32 Index : AttributeGroup.Value)
		{
			const FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];

			if (!LastType.IsSet())
			{
				LastType = Attribute.DataType;
			}
			else if (Attribute.DataType != LastType.GetValue())
			{
				bMoreThanOneType = true;
			}

			if (bMoreThanOneType)
			{
				break;
			}
		}
		
		for (int32 Index : AttributeGroup.Value)
		{
			FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];

			Attribute.UpdatePinNameAndHlslId(bMoreThanOneType);
		}
	}
}

TArray<FOptimusCDIPinDefinition> UOptimusAdvancedSkeletonDataInterface::GetPinDefinitions_Internal(bool bGetAllPossiblePins, int32 AttributeIndexToExclude) const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumBones", "ReadNumBones", Optimus::DomainName::Vertex, "ReadNumVertices"});
	Defs.Add({"BoneMatrix", "ReadBoneMatrix", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"BoneWeight", "ReadBoneWeight", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"WeightedBoneMatrix", "ReadWeightedBoneMatrix", Optimus::DomainName::Vertex, "ReadNumVertices" });
	
	if (bGetAllPossiblePins || bEnableLayeredSkinning)
	{
		Defs.Add({"LayeredBoneMatrix", "ReadLayeredBoneMatrix", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
		Defs.Add({"WeightedLayeredBoneMatrix", "ReadWeightedLayeredBoneMatrix", Optimus::DomainName::Vertex, "ReadNumVertices" });
	}

	for (int32 Index = 0; Index < AttributeBufferArray.Num(); Index++)
	{
		if (Index != AttributeIndexToExclude)
		{
			const FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];
			Defs.Add({Attribute.PinName, FString::Printf(TEXT("Read%s"), *Attribute.HlslId), {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
		}
	}

	return Defs;	
}


const FOptimusAnimAttributeBufferDescription& UOptimusAdvancedSkeletonDataInterface::AddAnimAttribute(const FString& InName,
                                                                                                      const FOptimusDataTypeRef& InDataType)
{
	return AttributeBufferArray.InnerArray.AddDefaulted_GetRef()
		.Init(GetUnusedAttributeName(AttributeBufferArray.InnerArray.Num() - 1, InName), InDataType);
}

#if WITH_EDITOR
void UOptimusAdvancedSkeletonDataInterface::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
	const FName BasePropertyName = (PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None);
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeBufferArray, InnerArray));
		
		bool bHasAttributeIdChanged =
			(PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusAnimAttributeBufferDescription, Name)) ||
			(PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataTypeRef, TypeName));
				
		if (bHasAttributeIdChanged)
		{
			if(ensure(AttributeBufferArray.IsValidIndex(ChangedIndex)))
			{
				FOptimusAnimAttributeBufferDescription& ChangedAttribute = AttributeBufferArray[ChangedIndex];
				
				FName OldPinName = ChangedAttribute.PinName;

				if (ChangedAttribute.Name.IsEmpty())
				{
					ChangedAttribute.Name = TEXT("EmptyName");
				}

				ChangedAttribute.Name = GetUnusedAttributeName(ChangedIndex, ChangedAttribute.Name);
				
				UpdateAttributePinNamesAndHlslIds();
				OnPinDefinitionRenamedDelegate.Execute(OldPinName, ChangedAttribute.PinName);
			}
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataTypeRef, TypeName))
		{
			FOptimusAnimAttributeBufferDescription& ChangedAttribute = AttributeBufferArray[ChangedIndex];

			// Update the default value container accordingly
			ChangedAttribute.DefaultValueStruct.SetType(ChangedAttribute.DataType);
			OnPinDefinitionChangedDelegate.Execute();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusAdvancedSkeletonDataInterface, bEnableLayeredSkinning))
		{
			OnPinDefinitionChangedDelegate.Execute();
		}

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusAdvancedSkeletonDataInterface, SkinWeightProfile))
		{
			OnDisplayNameChangedDelegate.Execute();
		}
	}
	else 
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeBufferArray, InnerArray))
			{
				const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeBufferArray, InnerArray));
				FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[ChangedIndex];
			
				// Default to a float attribute
				Attribute.Init(GetUnusedAttributeName(ChangedIndex, TEXT("EmptyName")), 
					FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()));
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeBufferArray, InnerArray))
			{	
				const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeBufferArray, InnerArray));
				FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[ChangedIndex];
			
				Attribute.Name = GetUnusedAttributeName(ChangedIndex, Attribute.Name);
				Attribute.UpdatePinNameAndHlslId();
			}
		}
		
		OnPinDefinitionChangedDelegate.Execute();
	}
}
#endif

FString UOptimusAdvancedSkeletonDataInterface::GetDisplayName() const
{
	FString WeightProfileName = TEXT("Default Skin Weights");
	
	if (SkinWeightProfile != NAME_None)
	{
		WeightProfileName = SkinWeightProfile.ToString();
	}

	return FString::Printf(TEXT("Skeleton - %s"), *WeightProfileName);
}

TArray<FOptimusCDIPinDefinition> UOptimusAdvancedSkeletonDataInterface::GetPinDefinitions() const
{
	return GetPinDefinitions_Internal();
}

TArray<FOptimusCDIPropertyPinDefinition> UOptimusAdvancedSkeletonDataInterface::GetPropertyPinDefinitions() const
{
	TArray<FOptimusCDIPropertyPinDefinition> PropertyPinDefinitions;

	const FOptimusDataTypeHandle NameType = FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass());
	
	PropertyPinDefinitions.Add(
		{GetSkinWeightProfilePropertyName(), NameType}
	);

	return PropertyPinDefinitions;
}


TSubclassOf<UActorComponent> UOptimusAdvancedSkeletonDataInterface::GetRequiredComponentClass() const
{
	return USkeletalMeshComponent::StaticClass();
}

void UOptimusAdvancedSkeletonDataInterface::Initialize()
{
}


void UOptimusAdvancedSkeletonDataInterface::RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode)
{
	if(InNode)
	{
		OnPinDefinitionChangedDelegate.BindUObject(InNode, &UOptimusNode::RecreatePinsFromPinDefinitions);
		OnPinDefinitionRenamedDelegate.BindUObject(InNode, &UOptimusNode::RenamePinFromPinDefinition);
		OnDisplayNameChangedDelegate.BindUObject(InNode, &UOptimusNode::UpdateDisplayNameFromDataInterface);
	}
}

void UOptimusAdvancedSkeletonDataInterface::OnDataTypeChanged(FName InTypeName)
{
	Super::OnDataTypeChanged(InTypeName);

	for (FOptimusAnimAttributeBufferDescription& AttributeDescription : AttributeBufferArray)
	{
		if (AttributeDescription.DataType.TypeName == InTypeName)
		{
			AttributeDescription.DefaultValueStruct.SetType(AttributeDescription.DataType);
		}
	}
}

void UOptimusAdvancedSkeletonDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumBones"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneWeight"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWeightedBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);

	if (bEnableLayeredSkinning)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("ReadLayeredBoneMatrix"))
			.AddReturnType(EShaderFundamentalType::Float, 3, 4)
			.AddParam(EShaderFundamentalType::Uint)
			.AddParam(EShaderFundamentalType::Uint);

		OutFunctions.AddDefaulted_GetRef()
			.SetName(TEXT("ReadWeightedLayeredBoneMatrix"))
			.AddReturnType(EShaderFundamentalType::Float, 3, 4)
			.AddParam(EShaderFundamentalType::Uint);
	}
	
	for (int32 Index = 0; Index < AttributeBufferArray.Num(); Index++)
	{
		const FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];

		OutFunctions.AddDefaulted_GetRef()
		.SetName(FString::Printf(TEXT("Read%s"), *Attribute.HlslId))
		.AddReturnType(Attribute.DataType->ShaderValueType)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
	}

}

BEGIN_SHADER_PARAMETER_STRUCT(FAdvancedSkeletonDataInterfaceDefaultParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumBoneInfluences)
	SHADER_PARAMETER(uint32, InputWeightStride)
	SHADER_PARAMETER(uint32, InputWeightIndexSize)
	SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightStream)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightLookupStream)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float3x4>, LayeredBoneMatrices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusAdvancedSkeletonDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	FShaderParametersMetadataBuilder Builder;
	Builder.AddIncludedStruct<FAdvancedSkeletonDataInterfaceDefaultParameters>();

	TArray<FShaderParametersMetadata*> NestedStructs;

	for (int32 Index = 0; Index < AttributeBufferArray.Num(); Index++)
	{
		const FOptimusAnimAttributeBufferDescription& Attribute = AttributeBufferArray[Index];
		FShaderValueTypeHandle ArrayShaderType = FShaderValueType::MakeDynamicArrayType(Attribute.DataType->ShaderValueType);
		ComputeFramework::AddParamForType(Builder, *Attribute.HlslId, ArrayShaderType, NestedStructs);	
	}

	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("AnimAttributeBuffers"));

	InOutAllocations.ShaderParameterMetadatas.Add(ShaderParameterMetadata);
	InOutAllocations.ShaderParameterMetadatas.Append(NestedStructs);

	// Add the generated nested struct to our builder.
	InOutBuilder.AddNestedStruct(UID, ShaderParameterMetadata);
}

void UOptimusAdvancedSkeletonDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	// Need to be able to support these permutations according to the skeletal mesh settings.
	// todo[CF]: I think GPUSKIN_UNLIMITED_BONE_INFLUENCE and GPUSKIN_BONE_INDEX_UINT16/GPUSKIN_BONE_WEIGHTS_UINT16 are mutually exclusive. So we could save permutations here.
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_BONES"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_INDEX_UINT16"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"), 2);
}

TCHAR const* UOptimusAdvancedSkeletonDataInterface::SkeletonTemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceAdvancedSkeleton.ush");
TCHAR const* UOptimusAdvancedSkeletonDataInterface::AttributeTemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceAdvancedSkeletonAnimAttribute.ush");


void UOptimusAdvancedSkeletonDataInterface::GetShaderHash(FString& InOutKey) const
{
}

void UOptimusAdvancedSkeletonDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> SkeletonTemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString SkeletonTemplateFile;
	LoadShaderSourceFile(SkeletonTemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &SkeletonTemplateFile, nullptr);
	OutHLSL += FString::Format(*SkeletonTemplateFile, SkeletonTemplateArgs);
	
	FString AttributeTemplateFile;
	LoadShaderSourceFile(AttributeTemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &AttributeTemplateFile, nullptr);
	
	for (int32 Index = 0; Index < AttributeBufferArray.Num(); Index++)
	{
		FOptimusAnimAttributeBufferDescription AttributeBufferDescription = AttributeBufferArray[Index];
		TMap<FString, FStringFormatArg> AttributeTemplateArgs =
		{
			{ TEXT("DataInterfaceName"), InDataInterfaceName },
			{ TEXT("TypeName"), AttributeBufferDescription.DataType->ShaderValueType->ToString()},
			{ TEXT("AttributeName"), AttributeBufferDescription.HlslId},
		};
		OutHLSL += FString::Format(*AttributeTemplateFile, AttributeTemplateArgs);	
	}
}

FOptimusAnimAttributeBufferRuntimeData::FOptimusAnimAttributeBufferRuntimeData(const FOptimusAnimAttributeBufferDescription& InDescription)
{
	Name = *InDescription.Name;
	HlslId = *InDescription.HlslId;
	
	Offset = 0;
	Size = InDescription.DataType->ShaderValueSize;

	const FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	ConvertFunc = Registry.FindPropertyValueConvertFunc(InDescription.DataType.TypeName);

	AttributeType = Registry.FindAttributeType(InDescription.DataType.TypeName);

	CachedDefaultValue = InDescription.DefaultValueStruct.GetShaderValue(InDescription.DataType);	
}

UComputeDataProvider* UOptimusAdvancedSkeletonDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusAdvancedSkeletonDataProvider* Provider = NewObject<UOptimusAdvancedSkeletonDataProvider>();
	Provider->Init(this, Cast<USkeletalMeshComponent>(InBinding));
	return Provider;
}
void FOptimusBoneTransformBuffer::SetData(FSkeletalMeshLODRenderData const& InLodRenderData, const TArray<FTransform>& InBoneTransforms, bool bUnifiedBoneMap)
{
	if (InBoneTransforms.IsEmpty())
	{
		return;
	}
	const FName Matrix34TypeName = FOptimusDataTypeRegistry::Matrix34TypeName;
	const FOptimusDataTypeHandle Matrix34TypeHandle = FOptimusDataTypeRegistry::Get().FindType(Matrix34TypeName);
	int32 Matrix34ShaderSize = Matrix34TypeHandle->ShaderValueSize;

	const int32 NumUniqueSections = bUnifiedBoneMap ? 1 : InLodRenderData.RenderSections.Num();

	BufferData.AddDefaulted(NumUniqueSections);
	NumBones.AddDefaulted(NumUniqueSections);

	for (int32 SectionIndex = 0; SectionIndex < NumUniqueSections; ++SectionIndex)
	{
		FSkelMeshRenderSection const& RenderSection = InLodRenderData.RenderSections[SectionIndex];
		NumBones[SectionIndex] = RenderSection.BoneMap.Num();

		BufferData[SectionIndex].AddDefaulted(NumBones[SectionIndex] * Matrix34ShaderSize);
		for (int32 BoneIndex = 0; BoneIndex < NumBones[SectionIndex]; BoneIndex++)
		{
			int32 FinalBoneIndex = RenderSection.BoneMap[BoneIndex];
			uint8* BoneDataPtr = BufferData[SectionIndex].GetData() + BoneIndex * Matrix34ShaderSize;
			Optimus::ConvertFTransformToFMatrix3x4(InBoneTransforms[FinalBoneIndex],
				{ {BoneDataPtr , Matrix34ShaderSize} ,{}} );
		}
	}		
}

bool FOptimusBoneTransformBuffer::HasData() const
{
	return NumBones.Num() > 0;
}

void FOptimusBoneTransformBuffer::AllocateResources(FRDGBuilder& GraphBuilder)
{
	if (!HasData())
	{
		return;
	}
	
	BufferRefPerSection.AddDefaulted(BufferData.Num());
	BufferSRVPerSection.AddDefaulted(BufferData.Num());	

	// If we are using a raw type alias for the buffer then we need to adjust stride and count.
	for (int32 InvocationIndex = 0 ; InvocationIndex < BufferData.Num(); InvocationIndex++)
	{
		int32 Stride = BufferData[InvocationIndex].Num() / NumBones[InvocationIndex];
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(Stride,  NumBones[InvocationIndex]), TEXT("BoneTransformBuffer"), ERDGBufferFlags::None);
		FRDGBufferSRVRef BufferSRV = GraphBuilder.CreateSRV(Buffer);

		BufferRefPerSection[InvocationIndex] = Buffer;
		BufferSRVPerSection[InvocationIndex] = BufferSRV;

		GraphBuilder.QueueBufferUpload(Buffer, BufferData[InvocationIndex].GetData(), BufferData[InvocationIndex].Num(), ERDGInitialDataFlags::None);	
	}	
}

void UOptimusAdvancedSkeletonDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}

void UOptimusAdvancedSkeletonDataProvider::Init(const UOptimusAdvancedSkeletonDataInterface* InDataInterface, USkeletalMeshComponent* InSkeletalMesh)
{
	WeakDataInterface = InDataInterface;
	bEnableLayeredSkinning = InDataInterface->bEnableLayeredSkinning;
	SkeletalMesh = InSkeletalMesh;
	SkinWeightProfile = InDataInterface->SkinWeightProfile;

	// Convert description to runtime data
	for (const FOptimusAnimAttributeBufferDescription& Attribute : InDataInterface->AttributeBufferArray)
	{
		AttributeBufferRuntimeData.Add(Attribute);
	}

	// Compute offset within the shader parameter buffer for each attribute
	FShaderParametersMetadataBuilder Builder;
	FShaderParametersMetadataAllocations Allocations;
	InDataInterface->GetShaderParameters(TEXT("Dummy"), Builder, Allocations);

	{
		struct FParametersMetadataScope
		{
			FParametersMetadataScope(FShaderParametersMetadata* InMetadata) : ShaderParameterMetadata(InMetadata){};
			~FParametersMetadataScope() {delete ShaderParameterMetadata;};
			
			FShaderParametersMetadata* ShaderParameterMetadata = nullptr;
		};

		
		FParametersMetadataScope MetadataScope(Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("UAnimAttributeDataInterface")));

		TArray<FShaderParametersMetadata::FMember> const& DummyMember = MetadataScope.ShaderParameterMetadata->GetMembers();
		check(DummyMember.Num()==1);
		FShaderParametersMetadata::FMember const& Dummy = DummyMember.Last();
		TArray<FShaderParametersMetadata::FMember> const& DataInterfaceParameterMembers = Dummy.GetStructMetadata()->GetMembers();

		check(DataInterfaceParameterMembers.Num() == AttributeBufferRuntimeData.Num() + 1);

		int32 AttributeMemberStart = 1;
		
		for (int32 Index = 0; Index < AttributeBufferRuntimeData.Num(); ++Index)
		{
			FOptimusAnimAttributeBufferRuntimeData& RuntimeData = AttributeBufferRuntimeData[Index];
			int32 AttributeIndexInParameter = AttributeMemberStart + Index;
			check(RuntimeData.HlslId == DataInterfaceParameterMembers[AttributeIndexInParameter].GetName());
		
			RuntimeData.Offset = DataInterfaceParameterMembers[AttributeIndexInParameter].GetOffset();
		}

		ParameterBufferSize = MetadataScope.ShaderParameterMetadata->GetSize();
	}

	// Request profiles early here for all the LODs that can run this deformer so that we can avoid requesting profiles
	// during gameplay or LOD transitions as it can cause T-Posing for a few frames due to GPU readback delay
	Optimus::RequestSkinWeightProfileForDeformer(InSkeletalMesh, SkinWeightProfile);
}

void UOptimusAdvancedSkeletonDataProvider::ComputeBoneTransformsForLayeredSkinning(
	TArray<FTransform>& OutBoneTransforms,
	FSkeletalMeshLODRenderData const& InLodRenderData, 
	const FReferenceSkeleton& InRefSkeleton)
{
	if (!bIsLayeredSkinInitialized)
	{
		bIsLayeredSkinInitialized = true;

		CachedWeightedBoneIndices.Reset();
		CachedBoundaryBoneIndex.Reset();
		CachedLayerSpaceInitialBoneTransform.Reset();
		
		const TArray<FTransform>& InitialBoneSpaceTransforms = InRefSkeleton.GetRefBonePose();
		
		// 1. Look for all bones with non-zero weights in this skin weight profile
		CachedWeightedBoneIndices.Reserve(InRefSkeleton.GetNum());
		
		FSkinWeightVertexBuffer const* WeightBuffer = InLodRenderData.GetSkinWeightVertexBuffer();
		if (InLodRenderData.SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
		{
			const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
			WeightBuffer = InLodRenderData.SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
		}

		for (uint32 VertexIndex = 0; VertexIndex < WeightBuffer->GetNumVertices(); VertexIndex++)
		{
			// Find the render section, which we need to find the final bone index.
			int32 SectionIndex = INDEX_NONE;
			int32 SectionVertexIndex = INDEX_NONE;
			InLodRenderData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

			uint32 VertexWeightOffset = 0;
			uint32 VertexInfluenceCount = 0;
			WeightBuffer->GetVertexInfluenceOffsetCount(VertexIndex, VertexWeightOffset, VertexInfluenceCount);
			for (uint32 InfluenceIndex = 0 ; InfluenceIndex < VertexInfluenceCount; InfluenceIndex++)
			{
				float Weight = WeightBuffer->GetBoneWeight(VertexIndex, InfluenceIndex);
				if (Weight > 0)
				{
					int32 SectionBoneIndex = WeightBuffer->GetBoneIndex(VertexIndex, InfluenceIndex);
					int32 FinalBoneIndex = InLodRenderData.RenderSections[SectionIndex].BoneMap[SectionBoneIndex];
					CachedWeightedBoneIndices.Add(FinalBoneIndex);
				}
			}
		}

		// 2. For each weighted bone, find its boundary bone
		// Boundary bone is defined as any bone that is weighted and none of its parents(except root) are weighted,
		// "sitting at the boundary of weighted and unweighted"
		// E.g. If a bone is weighted, its parent is not weighted, but its grand parent is weighted, the grand parent is a boundary bone candidate
		CachedBoundaryBoneIndex.Init(INDEX_NONE, InRefSkeleton.GetNum());
		for (int32 BoneIndex = 1; BoneIndex < InRefSkeleton.GetNum(); BoneIndex++)
		{
			
			int32 ParentIndex = InRefSkeleton.GetParentIndex(BoneIndex);

			if (CachedBoundaryBoneIndex[ParentIndex] == INDEX_NONE)
			{
				if (CachedWeightedBoneIndices.Contains(BoneIndex))
				{
					// This must be the first weighted bone we have encountered that has no weighted parents
					CachedBoundaryBoneIndex[BoneIndex] = BoneIndex;
				}
			}
			else
			{
				// Child always inherit boundary bone from parent
				CachedBoundaryBoneIndex[BoneIndex] = CachedBoundaryBoneIndex[ParentIndex];
			}
		}

		// 3. For all children of the boundary bone, compute their initial layer space transform (rooted at the parent of the boundary)
		CachedLayerSpaceInitialBoneTransform.AddDefaulted(InRefSkeleton.GetNum());
		for (int32 BoneIndex = 1; BoneIndex < InRefSkeleton.GetNum(); BoneIndex++)
		{
			if (CachedBoundaryBoneIndex[BoneIndex] != INDEX_NONE)
			{
				int32 ParentIndex = InRefSkeleton.GetParentIndex(BoneIndex);

				if (CachedBoundaryBoneIndex[ParentIndex] == INDEX_NONE)
				{
					CachedLayerSpaceInitialBoneTransform[BoneIndex] = InitialBoneSpaceTransforms[BoneIndex];		
				}
				else
				{
					// Accumulate Ref bone space transforms
					CachedLayerSpaceInitialBoneTransform[BoneIndex] = InitialBoneSpaceTransforms[BoneIndex] * CachedLayerSpaceInitialBoneTransform[ParentIndex];
				}
				
			}
		}
	}
	
	const TArray<FTransform>& CurrentComponentSpaceBoneMatrix = SkeletalMesh->GetComponentSpaceTransforms();

	// 4. InverseBindMatrix for each weighted bone is:
	// the inverse of (initial layer space transform * current global transform of parent of the layer boundary bone)
	// and because we are looking at the current global of the boundary bone parent, this InverseBindMatrix can change every frame
	TArray<FTransform> LayeredBoneMatrix;
	LayeredBoneMatrix.AddDefaulted(InRefSkeleton.GetNum());
	for (int32 BoneIndex : CachedWeightedBoneIndices)
	{
		if (BoneIndex == 0)
		{
			// Root bone is where we dump all the weights, so ignore any root transform if we are doing layered skinning
			LayeredBoneMatrix[BoneIndex] = FTransform::Identity;
			continue;
		}

		int32 BoundaryBoneParentIndex = InRefSkeleton.GetParentIndex(CachedBoundaryBoneIndex[BoneIndex]);
		LayeredBoneMatrix[BoneIndex] =
			(CachedLayerSpaceInitialBoneTransform[BoneIndex] * CurrentComponentSpaceBoneMatrix[BoundaryBoneParentIndex]).Inverse() * CurrentComponentSpaceBoneMatrix[BoneIndex];
	}

	OutBoneTransforms = MoveTemp(LayeredBoneMatrix);
}

FComputeDataProviderRenderProxy* UOptimusAdvancedSkeletonDataProvider::GetRenderProxy()
{
	FOptimusAdvancedSkeletonDataProviderProxy* Proxy = new FOptimusAdvancedSkeletonDataProviderProxy();

	if (SkeletalMesh.IsValid() && SkeletalMesh->MeshObject)
	{
		const UOptimusAdvancedSkeletonDataInterface* DataInterface = WeakDataInterface.Get();
		UOptimusDeformerInstance* DeformerInstance = WeakDeformerInstance.Get();
		if (DeformerInstance && DataInterface)
		{
			FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride(
						DataInterface,
						UOptimusAdvancedSkeletonDataInterface::GetSkinWeightProfilePropertyName()
						);

			TValueOrError<FName, EPropertyBagResult> Value = ValueContainer.Value.GetValueName(FOptimusValueContainerStruct::ValuePropertyName);
			if (Value.HasValue())
			{
				if (SkinWeightProfile != Value.GetValue())
				{
					SkinWeightProfile = Value.GetValue();
					if (bEnableLayeredSkinning)
					{
						bIsLayeredSkinInitialized = false;
					}
				}
			}
		}
		
		FSkeletalMeshObject* SkeletalMeshObject = SkeletalMesh->MeshObject;
		const int32 LodIndex = SkeletalMeshObject->GetLOD();
		const FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		const FSkeletalMeshLODRenderData& LodRenderData = SkeletalMeshRenderData.LODRenderData[LodIndex];

		if (!Optimus::IsSkinWeightProfileAvailable(LodRenderData, SkinWeightProfile))
		{
			Optimus::RequestSkinWeightProfileForDeformer(SkeletalMesh.Get(), SkinWeightProfile, LodIndex);
		}
		else
		{
			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetSkinnedAsset()->GetRefSkeleton();
			
			TArray<FTransform> LayeredBoneMatrices;
			
			if (bEnableLayeredSkinning)
			{
				ComputeBoneTransformsForLayeredSkinning(LayeredBoneMatrices, LodRenderData, RefSkeleton);
			}
			
			// Per-Bone Animation Attributes
			const UE::Anim::FMeshAttributeContainer& AttributeContainer = SkeletalMesh->GetCustomAttributes();

			TArray<TArray<uint8>> Attributes;
			
			Attributes.AddDefaulted(AttributeBufferRuntimeData.Num());
			for (int32 AttributeIndex = 0; AttributeIndex < AttributeBufferRuntimeData.Num(); ++AttributeIndex)
			{
				const FOptimusAnimAttributeBufferRuntimeData& AttributeData = AttributeBufferRuntimeData[AttributeIndex];

				const int32 NumBones = RefSkeleton.GetNum();
				Attributes[AttributeIndex].AddDefaulted(NumBones * AttributeData.Size);
				for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
				{
					uint8* BoneDataPtr = Attributes[AttributeIndex].GetData() + BoneIndex * AttributeData.Size;
					const UE::Anim::FAttributeId Id = {AttributeData.Name, FCompactPoseBoneIndex(BoneIndex)} ;

					bool bIsValueSet = false;
				
					FOptimusDataTypeRegistry::PropertyValueConvertFuncT ConvertFunc = AttributeData.ConvertFunc;

					if (ConvertFunc)
					{
						UScriptStruct* AttributeType = AttributeData.AttributeType;
				
						if (const uint8* Attribute = AttributeContainer.Find(AttributeType, Id))
						{
							bIsValueSet = true;
						
							const uint8* ValuePtr = Attribute;
							int32 ValueSize = AttributeType->GetStructureSize();

							// TODO: use a specific function to extract the value from the attribute
							// it works for now because even if the attribute type != actual value type
							// it should only have a single property, whose type == actual property type
						
							ConvertFunc(
								{ValuePtr, ValueSize},
								{ {BoneDataPtr, AttributeData.Size}, {} });
						}
				
						// Use the default value if the attribute was not found
						if (!bIsValueSet)
						{
							const uint8* DefaultValuePtr = AttributeData.CachedDefaultValue.ShaderValue.GetData();
							const uint32 DefaultValueSize = AttributeData.CachedDefaultValue.ShaderValue.Num();

							FMemory::Memcpy(BoneDataPtr, DefaultValuePtr, DefaultValueSize);
						}
					}
				}
			}
			

			// Pipe data into proxy
			Proxy->SceneInterface = SkeletalMesh->GetScene();
			Proxy->ComponentId = SkeletalMesh->GetPrimitiveSceneId();
			
			Proxy->SkinWeightProfile = SkinWeightProfile;
			Proxy->ParameterBuffer.AddDefaulted(ParameterBufferSize);
			Proxy->LayeredBoneMatrices = MoveTemp(LayeredBoneMatrices);
			Proxy->AttributeBufferRuntimeData = AttributeBufferRuntimeData;
			Proxy->Attributes = MoveTemp(Attributes);
		}
	}
	
	return Proxy;
}


FOptimusAdvancedSkeletonDataProviderProxy::FOptimusAdvancedSkeletonDataProviderProxy()
{
}

bool FOptimusAdvancedSkeletonDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (InValidationData.ParameterStructSize != ParameterBuffer.Num())
	{
		return false;
	}
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	if (LodRenderData->RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}
	
	FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
	if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
	{
		const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
		WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
	}
	
	if (WeightBuffer == nullptr)
	{
		return false;
	}

	return true;
}

void FOptimusAdvancedSkeletonDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const& LodRenderData = SkeletalMeshRenderData.LODRenderData[LodIndex];

	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();
	const int32 NumUniqueSections = bUnifiedBoneMap ? 1 : LodRenderData.RenderSections.Num();

	FallbackSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32))));

	LayeredBoneMatrixBuffer.SetData(LodRenderData, LayeredBoneMatrices, bUnifiedBoneMap);
	
	LayeredBoneMatrixBuffer.AllocateResources(GraphBuilder);

	// Per-Bone Animation Attribute 
	BuffersPerAttributePerSection.AddDefaulted(NumUniqueSections);
	BufferSRVsPerAttributePerSection.AddDefaulted(NumUniqueSections);

	AttributeBuffers.AddDefaulted(NumUniqueSections);
	for (int32 SectionIndex = 0; SectionIndex < NumUniqueSections; ++SectionIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData.RenderSections[SectionIndex];
		AttributeBuffers[SectionIndex].AddDefaulted(AttributeBufferRuntimeData.Num());
		for (int32 AttributeIndex = 0; AttributeIndex < AttributeBufferRuntimeData.Num(); ++AttributeIndex)
		{
			const FOptimusAnimAttributeBufferRuntimeData& AttributeData = AttributeBufferRuntimeData[AttributeIndex];
			const int32 NumBones = RenderSection.BoneMap.Num();
			AttributeBuffers[SectionIndex][AttributeIndex].AddDefaulted(NumBones * AttributeData.Size);

			for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
			{
				uint8* TargetBoneDataPtr = AttributeBuffers[SectionIndex][AttributeIndex].GetData() + BoneIndex * AttributeData.Size;
				int32 FinalBoneIndex = RenderSection.BoneMap[BoneIndex];

				int32 SourceDataOffset = FinalBoneIndex * AttributeData.Size;
				if (ensure((SourceDataOffset + AttributeData.Size) <= Attributes[AttributeIndex].Num()))
				{
					uint8* SourceBoneDataPtr = Attributes[AttributeIndex].GetData() + SourceDataOffset;
					FMemory::Memcpy(TargetBoneDataPtr, SourceBoneDataPtr,  AttributeData.Size);	
				}
			}
		}
		
	}
	
	for (int32 InvocationIndex = 0 ; InvocationIndex < NumUniqueSections; InvocationIndex++)
	{
		BuffersPerAttributePerSection[InvocationIndex].AddDefaulted(AttributeBuffers[InvocationIndex].Num());
		BufferSRVsPerAttributePerSection[InvocationIndex].AddDefaulted(AttributeBuffers[InvocationIndex].Num());
		
		FSkelMeshRenderSection const& RenderSection = LodRenderData.RenderSections[InvocationIndex];
		int32 NumBones = RenderSection.BoneMap.Num();

		for (int32 AttributeIndex = 0; AttributeIndex < AttributeBuffers[InvocationIndex].Num(); ++AttributeIndex)
		{
			const TArray<uint8>& BoneData = AttributeBuffers[InvocationIndex][AttributeIndex];

			int32 Stride = BoneData.Num() / NumBones;
			
			FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(Stride,  NumBones), TEXT("AttributeBuffer"), ERDGBufferFlags::None);
			FRDGBufferSRVRef BufferSRV = GraphBuilder.CreateSRV(Buffer);

			BuffersPerAttributePerSection[InvocationIndex][AttributeIndex] = Buffer;
			BufferSRVsPerAttributePerSection[InvocationIndex][AttributeIndex] = BufferSRV;

			GraphBuilder.QueueBufferUpload(Buffer, BoneData.GetData(), BoneData.Num(), ERDGInitialDataFlags::None);	
		}
	}
}

struct FAdvancedSkeletonDataInterfacePermutationIds
{
	uint32 EnableDeformerBones = 0;
	uint32 UnlimitedBoneInfluence = 0;
	uint32 BoneIndexUint16 = 0;
	uint32 BoneWeightsUint16 = 0;

	FAdvancedSkeletonDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_BONES"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerBones = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"));
			static uint32 Hash = GetTypeHash(Name);
			UnlimitedBoneInfluence = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_INDEX_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneIndexUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneWeightsUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusAdvancedSkeletonDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FAdvancedSkeletonDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		
		FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
		if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
		{
			const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
			WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
		}
		
		check(WeightBuffer != nullptr);
		FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
		const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
		FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;
		const bool bValidBones = (BoneBufferSRV != nullptr) && (SkinWeightBufferSRV != nullptr) && (!bUnlimitedBoneInfluences || InputWeightLookupStreamSRV != nullptr);
		const bool bUse16BitBoneIndex = WeightBuffer->Use16BitBoneIndex();
		const bool bUse16BitBoneWeights = WeightBuffer->Use16BitBoneWeight();

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidBones ? PermutationIds.EnableDeformerBones : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUnlimitedBoneInfluences ? PermutationIds.UnlimitedBoneInfluence : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneIndex ? PermutationIds.BoneIndexUint16 : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneWeights ? PermutationIds.BoneWeightsUint16 : 0);
	}
}

void FOptimusAdvancedSkeletonDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
	if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
	{
		const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
		WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
	}
	check(WeightBuffer != nullptr);
		
	FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
	const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
	FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;

	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		uint8* ParameterBufferPtr = (InDispatchData.ParameterBuffer + InDispatchData.ParameterBufferOffset + InDispatchData.ParameterBufferStride * InvocationIndex);
		
		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		const int32 BoneBufferIndex = bUnifiedBoneMap ? 0 : InvocationIndex;

		FDefaultParameters* Parameters = reinterpret_cast<FDefaultParameters*>(ParameterBufferPtr);
		Parameters->NumVertices = LodRenderData->GetNumVertices();
		Parameters->NumBoneInfluences = WeightBuffer->GetMaxBoneInfluences();
		Parameters->InputWeightStride = WeightBuffer->GetConstantInfluencesVertexStride();
		Parameters->InputWeightIndexSize = WeightBuffer->GetBoneIndexByteSize() | (WeightBuffer->GetBoneWeightByteSize() << 8);
		Parameters->BoneMatrices = BoneBufferSRV != nullptr ? BoneBufferSRV : NullSRVBinding;
		Parameters->InputWeightStream = SkinWeightBufferSRV != nullptr ? SkinWeightBufferSRV : NullSRVBinding;
		Parameters->InputWeightLookupStream = InputWeightLookupStreamSRV != nullptr ? InputWeightLookupStreamSRV : NullSRVBinding;
		Parameters->LayeredBoneMatrices =
			LayeredBoneMatrixBuffer.HasData() && LayeredBoneMatrixBuffer.BufferSRVPerSection[BoneBufferIndex] != nullptr ?
				LayeredBoneMatrixBuffer.BufferSRVPerSection[BoneBufferIndex] : FallbackSRV;

		for(int32 AttributeIndex = 0 ; AttributeIndex < AttributeBufferRuntimeData.Num(); AttributeIndex++)
		{
			*((FRDGBufferSRV**)(ParameterBufferPtr + AttributeBufferRuntimeData[AttributeIndex].Offset)) = BufferSRVsPerAttributePerSection[BoneBufferIndex][AttributeIndex];
		}
	}
}
