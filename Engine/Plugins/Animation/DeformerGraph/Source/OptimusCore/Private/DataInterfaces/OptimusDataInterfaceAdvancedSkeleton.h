// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDataType.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusValueContainerStruct.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceAdvancedSkeleton.generated.h"

#define UE_API OPTIMUSCORE_API

struct FOptimusAnimAttributeDescription;
class FSceneInterface;
class FSkeletalMeshObject;
class FAdvancedSkeletonDataInterfaceDefaultParameters;
class USkeletalMeshComponent;
class FSkeletalMeshLODRenderData;
struct FReferenceSkeleton;

USTRUCT()
struct FOptimusAnimAttributeBufferDescription
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FString Name;

	UPROPERTY(EditAnywhere, Category = "Data Interface", meta=(UseInPerBoneAnimAttribute))
	FOptimusDataTypeRef DataType;

	// Default value if the animation attribute is not found
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FOptimusValueContainerStruct DefaultValueStruct;

	UPROPERTY()
	FString HlslId;

	UPROPERTY()
	FName PinName;

	static const TCHAR* PinNameDelimiter;
	static const TCHAR* HlslIdDelimiter;
	
	void UpdatePinNameAndHlslId(bool bInIncludeTypeName = true);
	
	// Helpers
	FOptimusAnimAttributeBufferDescription& Init(const FString& InName, const FOptimusDataTypeRef& InDataType);
	
private:
	FString GetFormattedId(
		const FString& InDelimiter,
		bool bInIncludeTypeName) const;
};

USTRUCT()
struct FOptimusAnimAttributeBufferArray
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, DisplayName = "Animation Attributes" ,Category = "Data Interface")
	TArray<FOptimusAnimAttributeBufferDescription> InnerArray;

	template <typename Predicate>
	const FOptimusAnimAttributeBufferDescription* FindByPredicate(Predicate Pred) const
	{
		return InnerArray.FindByPredicate(Pred);
	}

	bool IsEmpty() const
	{
		return InnerArray.IsEmpty();
	}

	FOptimusAnimAttributeBufferDescription& Last(int32 IndexFromTheEnd = 0)
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	const FOptimusAnimAttributeBufferDescription& Last(int32 IndexFromTheEnd = 0) const
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	FOptimusAnimAttributeBufferArray& operator=(const TArray<FOptimusAnimAttributeBufferDescription>& Rhs)
	{
		InnerArray = Rhs;
		return *this;
	}
	
	int32 Num() const { return InnerArray.Num();}
	bool IsValidIndex(int32 Index) const { return Index < InnerArray.Num() && Index >= 0; }
	const FOptimusAnimAttributeBufferDescription& operator[](int32 InIndex) const { return InnerArray[InIndex]; }
	FOptimusAnimAttributeBufferDescription& operator[](int32 InIndex) { return InnerArray[InIndex]; }
	FORCEINLINE	TArray<FOptimusAnimAttributeBufferDescription>::RangedForIteratorType      begin()       { return InnerArray.begin(); }
	FORCEINLINE	TArray<FOptimusAnimAttributeBufferDescription>::RangedForConstIteratorType begin() const { return InnerArray.begin(); }
	FORCEINLINE	TArray<FOptimusAnimAttributeBufferDescription>::RangedForIteratorType      end()         { return InnerArray.end();   }
	FORCEINLINE	TArray<FOptimusAnimAttributeBufferDescription>::RangedForConstIteratorType end() const   { return InnerArray.end();   }
};


/* Compute Framework Data Interface for skeletal data. */

/* Skeleton Data Interface provides data related to skinning. */ 
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusAdvancedSkeletonDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()
	
public:
	
	
#if WITH_EDITOR
	UE_API void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif


	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;

	UE_API void Initialize() override;
	bool CanPinDefinitionChange() override {return true;};
	UE_API void RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode) override;

	UE_API void OnDataTypeChanged(FName InTypeName) override;
	//~ End UOptimusComputeDataInterface Interface
	
	UE_API const FOptimusAnimAttributeBufferDescription& AddAnimAttribute(const FString& InName, const FOptimusDataTypeRef& InDataType);


	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("AdvancedSkeleton"); }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface
	
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FName SkinWeightProfile = NAME_None;

	// If turned on, another set of bone matrices are computed per-frame to allow for layered skinning.
	// It is typically used with a secondary skin weight profile storing the weights of a subset of bones like tweaker bones.
	// The bind matrices for these bones are dynamic and computed based on their parent's current transform instead of initial transform
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	bool bEnableLayeredSkinning = false;

	// Per-bone animation attributes, allows for custom bone data to be used in kernels, one of the places you can create
	// animation attributes is Control Rig
	UPROPERTY(EditAnywhere, Category = "Data Interface", meta = (ShowOnlyInnerProperties))
	FOptimusAnimAttributeBufferArray AttributeBufferArray;

private:
	friend class UOptimusAdvancedSkeletonDataProvider;
	static UE_API FName GetSkinWeightProfilePropertyName();
	
	UE_API FString GetUnusedAttributeName(int32 CurrentAttributeIndex ,const FString& InName) const;
	UE_API void UpdateAttributePinNamesAndHlslIds();
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions_Internal(bool bGetAllPossiblePins = false, int32 AttributeIndexToExclude = INDEX_NONE) const;	
	static UE_API TCHAR const* SkeletonTemplateFilePath;
	static UE_API TCHAR const* AttributeTemplateFilePath;
	FOnPinDefinitionChanged OnPinDefinitionChangedDelegate;
	FOnPinDefinitionRenamed OnPinDefinitionRenamedDelegate;
	FSimpleDelegate OnDisplayNameChangedDelegate;
};

// Runtime data with cached values baked out from AttributeDescription
struct FOptimusAnimAttributeBufferRuntimeData
{
	FOptimusAnimAttributeBufferRuntimeData() = default;
	
	FOptimusAnimAttributeBufferRuntimeData(const FOptimusAnimAttributeBufferDescription& InDescription);
	
	FName Name;
	FName HlslId;

	int32 Offset = INDEX_NONE;

	int32 Size = 0;

	FOptimusDataTypeRegistry::PropertyValueConvertFuncT	ConvertFunc = nullptr;

	UScriptStruct* AttributeType = nullptr;

	FShaderValueContainer CachedDefaultValue;
};

struct FOptimusBoneTransformBuffer
{
	TArray<FRDGBufferRef> BufferRefPerSection;
	TArray<FRDGBufferSRVRef> BufferSRVPerSection;
	TArray<TArray<uint8>> BufferData;
	TArray<int32> NumBones;

	void SetData(FSkeletalMeshLODRenderData const& InLodRenderData, const TArray<FTransform>& InBoneTransforms, bool bInUnifiedBoneMap);
	bool HasData() const;

	void AllocateResources(FRDGBuilder& GraphBuilder);
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusAdvancedSkeletonDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor

{
	GENERATED_BODY()

public:
	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;
	
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMesh = nullptr;

	TArray<FOptimusAnimAttributeBufferRuntimeData> AttributeBufferRuntimeData;

	int32 ParameterBufferSize = 0;

	FName SkinWeightProfile = NAME_None;
	
	bool bEnableLayeredSkinning = true;
	bool bIsLayeredSkinInitialized = false;
	TSet<uint32> CachedWeightedBoneIndices;
	TArray<int32> CachedBoundaryBoneIndex;
	TArray<FTransform> CachedLayerSpaceInitialBoneTransform; 
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	void Init(
		const UOptimusAdvancedSkeletonDataInterface* InDataInterface,
		USkeletalMeshComponent* InSkeletalMesh
	);

	void ComputeBoneTransformsForLayeredSkinning(TArray<FTransform>& OutBoneTransforms, FSkeletalMeshLODRenderData const& InLodRenderData, const FReferenceSkeleton& InRefSkeleton);
private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance;

	TWeakObjectPtr<const UOptimusAdvancedSkeletonDataInterface> WeakDataInterface;
};

class FOptimusAdvancedSkeletonDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusAdvancedSkeletonDataProviderProxy();

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

	using FDefaultParameters = FAdvancedSkeletonDataInterfaceDefaultParameters;

	// Data supplied by data provider
	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	FName SkinWeightProfile = NAME_None;
	
	TArray<uint8> ParameterBuffer;
	
	TArray<FTransform> LayeredBoneMatrices;
	TArray<FOptimusAnimAttributeBufferRuntimeData> AttributeBufferRuntimeData;
	TArray<TArray<uint8>> Attributes;

	// Proxy managed data to be uploaded to GPU
	FOptimusBoneTransformBuffer LayeredBoneMatrixBuffer;
	
	TArray<TArray<FRDGBufferRef>> BuffersPerAttributePerSection;
	TArray<TArray<FRDGBufferSRVRef>> BufferSRVsPerAttributePerSection;
	TArray<TArray<TArray<uint8>>> AttributeBuffers;
	
	FRDGBufferSRVRef FallbackSRV = nullptr;
};

#undef UE_API
