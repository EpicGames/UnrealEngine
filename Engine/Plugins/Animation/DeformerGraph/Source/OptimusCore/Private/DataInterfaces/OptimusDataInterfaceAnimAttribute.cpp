// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceAnimAttribute.h"

#include "Engine/SkeletalMesh.h"
#include "OptimusDataDomain.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusObjectVersion.h"
#include "OptimusValueContainer.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Components/SkeletalMeshComponent.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Nodes/OptimusNode_DataInterface.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceAnimAttribute)

static const FString PinNameDelimiter = TEXT(" - ");
static const FString HlslIdDelimiter = TEXT("_");



FOptimusAnimAttributeDescription& FOptimusAnimAttributeDescription::Init(const FString& InName, FName InBoneName,
	const FOptimusDataTypeRef& InDataType)
{
	Name = InName;
	BoneName = InBoneName;
	DataType = InDataType;

	DefaultValueStruct.SetType(DataType.Resolve());
	// Caller should ensure that the name is unique
	HlslId = InName;
	PinName = *InName;
	
	return *this;
}


void FOptimusAnimAttributeDescription::UpdatePinNameAndHlslId(bool bInIncludeBoneName, bool bInIncludeTypeName)
{
	PinName = *GetFormattedId(PinNameDelimiter, bInIncludeBoneName, bInIncludeTypeName);
	HlslId = GetFormattedId(HlslIdDelimiter, bInIncludeBoneName, bInIncludeTypeName);
}


FString FOptimusAnimAttributeDescription::GetFormattedId(
	const FString& InDelimiter, bool bInIncludeBoneName, bool bInIncludeTypeName) const
{
	FString UniqueId;
			
	if (bInIncludeBoneName)
	{
		if (BoneName != NAME_None)
		{
			 UniqueId += BoneName.ToString();
			 UniqueId += InDelimiter;
		}
	}
			
	if (bInIncludeTypeName)
	{
		 UniqueId += DataType.Resolve()->DisplayName.ToString();
		 UniqueId += InDelimiter;
	}

	UniqueId += Name;	

	return  UniqueId;
}

UOptimusAnimAttributeDataInterface::UOptimusAnimAttributeDataInterface()
{
}


#if WITH_EDITOR
void UOptimusAnimAttributeDataInterface::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
	const FName BasePropertyName = (PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None);
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray));
		
		bool bHasAttributeIdChanged =
			(PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeDescription, Name)) ||
			(PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeDescription, BoneName))||
			(PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusDataTypeRef, TypeName));
				
		if (bHasAttributeIdChanged)
		{
			if(ensure(AttributeArray.IsValidIndex(ChangedIndex)))
			{
				FOptimusAnimAttributeDescription& ChangedAttribute = AttributeArray[ChangedIndex];
				
				FName OldPinName = ChangedAttribute.PinName;

				if (ChangedAttribute.Name.IsEmpty())
				{
					ChangedAttribute.Name = TEXT("EmptyName");
				}

				for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
				{
					const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
					if (Index != ChangedIndex)
					{
						if (Attribute.Name == ChangedAttribute.Name &&
							Attribute.BoneName == ChangedAttribute.BoneName &&
							Attribute.DataType == ChangedAttribute.DataType )
						{
							// This particular change caused a Id clash, resolve it by changing the attribute name
							ChangedAttribute.Name = GetUnusedAttributeName(ChangedAttribute.Name);
						}
					}
				}

				UpdateAttributePinNamesAndHlslIds();
				OnPinDefinitionRenamedDelegate.Execute(OldPinName, ChangedAttribute.PinName);
			}
		}

		if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusDataTypeRef, TypeName))
		{
			FOptimusAnimAttributeDescription& ChangedAttribute = AttributeArray[ChangedIndex];

			// Update the default value container accordingly
			ChangedAttribute.DefaultValueStruct.SetType(ChangedAttribute.DataType);
			OnPinDefinitionChangedDelegate.Execute();
		}
	}
	else 
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray))
			{
				const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray));
				FOptimusAnimAttributeDescription& Attribute = AttributeArray[ChangedIndex];
			
				// Default to a float attribute
				Attribute.Init(GetUnusedAttributeName(TEXT("EmptyName")), NAME_None,
					FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()));
			}
		}
		else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray))
			{	
				const int32 ChangedIndex = PropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray));
				FOptimusAnimAttributeDescription& Attribute = AttributeArray[ChangedIndex];
			
				Attribute.Name = GetUnusedAttributeName(Attribute.Name);
				Attribute.UpdatePinNameAndHlslId();
			}
		}
		
		OnPinDefinitionChangedDelegate.Execute();
	}
}
#endif


void UOptimusAnimAttributeDataInterface::PostLoad()
{
	Super::PostLoad();
	
	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::PropertyBagValueContainer)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		
		for (FOptimusAnimAttributeDescription& Description : AttributeArray)
		{
			if (Description.DefaultValue_DEPRECATED)
			{
				Description.DefaultValue_DEPRECATED->ConditionalPostLoad();
			}
		}

		for (FOptimusAnimAttributeDescription& Description : AttributeArray)
		{
			if (Description.DefaultValue_DEPRECATED)
			{
				Description.DefaultValueStruct = Description.DefaultValue_DEPRECATED->MakeValueContainerStruct();
				Description.DefaultValue_DEPRECATED = nullptr;
			}
		}
		
PRAGMA_ENABLE_DEPRECATION_WARNINGS	
	}
}


FString UOptimusAnimAttributeDataInterface::GetDisplayName() const
{
	return TEXT("Animation Attributes");
}

TArray<FOptimusCDIPinDefinition> UOptimusAnimAttributeDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;

	for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
		Defs.Add({Attribute.PinName, FString::Printf(TEXT("Read%s"), *Attribute.HlslId)});
	}
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusAnimAttributeDataInterface::GetRequiredComponentClass() const
{
	return USkeletalMeshComponent::StaticClass();
}

void UOptimusAnimAttributeDataInterface::Initialize()
{
	AddAnimAttribute(TEXT("EmptyName"), NAME_None,
			FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()) );
}

void UOptimusAnimAttributeDataInterface::RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode)
{
	if(InNode)
	{
		OnPinDefinitionChangedDelegate.BindUObject(InNode, &UOptimusNode::RecreatePinsFromPinDefinitions);
		OnPinDefinitionRenamedDelegate.BindUObject(InNode, &UOptimusNode::RenamePinFromPinDefinition);
	}
}


void UOptimusAnimAttributeDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];

		OutFunctions.AddDefaulted_GetRef()
		.SetName(FString::Printf(TEXT("Read%s"), *Attribute.HlslId))
		.AddReturnType(Attribute.DataType->ShaderValueType);
	}
}

void UOptimusAnimAttributeDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	FShaderParametersMetadataBuilder Builder;
	
	TArray<FShaderParametersMetadata*> NestedStructs;
	for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
		ComputeFramework::AddParamForType(Builder, *Attribute.HlslId, Attribute.DataType->ShaderValueType, NestedStructs);
	}

	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("UAnimAttributeDataInterface"));

	InOutAllocations.ShaderParameterMetadatas.Add(ShaderParameterMetadata);
	InOutAllocations.ShaderParameterMetadatas.Append(NestedStructs);

	// Add the generated nested struct to our builder.
	InOutBuilder.AddNestedStruct(UID, ShaderParameterMetadata);
}

void UOptimusAnimAttributeDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];

		const FString TypeName = Attribute.DataType->ShaderValueType->ToString();
		const bool bIsStruct = Attribute.DataType->ShaderValueType->Type == EShaderFundamentalType::Struct;

		if (ensure(!TypeName.IsEmpty()))
		{
			if (!bIsStruct)
			{
				// Add uniforms.
				OutHLSL += FString::Printf(TEXT("%s %s_%s;\n"), 
					*TypeName, 
					*InDataInterfaceName, 
					*Attribute.HlslId);
					
				// Add function getters.
				OutHLSL += FString::Printf(TEXT("%s Read%s_%s()\n{\n\treturn %s_%s;\n}\n"), 
					*TypeName,
					*Attribute.HlslId,
					*InDataInterfaceName, 
					*InDataInterfaceName, 
					*Attribute.HlslId);
			}
			else if (bIsStruct)
			{
				struct FStructParser 
				{
					void WalkStruct(FString const& InDataInterfaceName, FShaderValueTypeHandle ShaderValueTypeHandle, const TArray<FString>& InPrefix)
					{
						const TArray<FShaderValueType::FStructElement>& StructElements =
							ShaderValueTypeHandle->StructElements;

						for (const FShaderValueType::FStructElement& StructElement : StructElements)
						{
							FString ElementName = StructElement.Name.ToString();

							TArray<FString> NewPrefix = InPrefix;
							NewPrefix.Add(ElementName);
							
							FString ElementHlslId = FString::Join(NewPrefix, TEXT("_"));
							FString ElementAccessor = FString::Join(NewPrefix, TEXT("."));
							
							if (StructElement.Type->Type == EShaderFundamentalType::Struct && !StructElement.Type->bIsDynamicArray)
							{
								WalkStruct(InDataInterfaceName, StructElement.Type, NewPrefix);
							}
							else
							{
								FString ElementType = StructElement.Type.ValueTypePtr->ToString();

								// Add uniforms.
								MemberHlsl.Add(FString::Printf(TEXT("%s %s_%s;\n"),
									*ElementType, 
									*InDataInterfaceName,
									*ElementHlslId));
						
								// Add function getters.
								MemberHlsl.Add(FString::Printf(TEXT("%s Read%s_%s()\n{\n\treturn %s_%s;\n}\n"),
									*ElementType,
									*ElementHlslId,
									*InDataInterfaceName,
									*InDataInterfaceName,
									*ElementHlslId));

								StructMemberCopyHlsl.Add(FString::Printf(TEXT("%s = %s_%s;\n"),
									*ElementAccessor, 
									*InDataInterfaceName,
									*ElementHlslId));
							}
						}
					}

					// Top level Hlsl code for flattened list of struct members, they are shown as structname_membername1_....
					TArray<FString> MemberHlsl;
					// Statements to copy the list of struct members into a local struct that user can access in a struct-like manner
					TArray<FString> StructMemberCopyHlsl;
				};
			
				FStructParser Parser;
				
				Parser.WalkStruct(InDataInterfaceName, Attribute.DataType->ShaderValueType, {Attribute.HlslId});

				// Top level Hlsl code for flattened list of struct members
				for (const FString& MemberHlslString : Parser.MemberHlsl)
				{
					OutHLSL += MemberHlslString;
				}
				
				// Add final user facing struct getters.
				OutHLSL += FString::Printf(TEXT("%s Read%s_%s()\n"),
					*TypeName,
					*Attribute.HlslId,
					*InDataInterfaceName);
				
				OutHLSL += FString::Printf(TEXT("{\n"));

				// Declare a local struct
				OutHLSL += FString::Printf(TEXT("\t%s %s;\n"), *TypeName, *Attribute.HlslId);

				// Copy each member from their flattened form into the local struct
				for (const FString& CopyHlsl : Parser.StructMemberCopyHlsl)
				{
					OutHLSL += TEXT("\t") + CopyHlsl;
				}
				
				// Return the local struct for easy user access
				OutHLSL += FString::Printf(TEXT("\treturn %s;\n"), *Attribute.HlslId);
				
				OutHLSL += FString::Printf(TEXT("}\n"));
			}
		}
	}
}

void UOptimusAnimAttributeDataInterface::GetStructDeclarations(TSet<FString>& OutStructsSeen,
	TArray<FString>& OutStructs) const
{
	Super::GetStructDeclarations(OutStructsSeen, OutStructs);

	auto CollectStructs = [&OutStructsSeen, &OutStructs](const FOptimusAnimAttributeArray& InAttributeArray)
	{
		for (const FOptimusAnimAttributeDescription &Attribute: InAttributeArray)
		{
			const FShaderValueType& ValueType = *Attribute.DataType->ShaderValueType;
			if (ValueType.Type == EShaderFundamentalType::Struct)
			{
				// Collect all unique types used in this type
				TArray<FShaderValueTypeHandle> StructTypes = Attribute.DataType->ShaderValueType->GetMemberStructTypes();
				StructTypes.Add(Attribute.DataType->ShaderValueType);

				for (const FShaderValueTypeHandle& TypeHandle : StructTypes )
				{
					const FString StructName = TypeHandle->ToString();
					if (!OutStructsSeen.Contains(StructName))
					{
						// Add their declaration from the inner most struct to outer most ones
						OutStructs.Add(TypeHandle->GetTypeDeclaration() + TEXT(";\n\n"));
						OutStructsSeen.Add(StructName);
					}	
				}
			}
		}
	};

	CollectStructs(AttributeArray);
}

void UOptimusAnimAttributeDataInterface::GetShaderHash(FString& InOutKey) const
{
	// UComputeGraph::BuildKernelSource hashes the result of GetHLSL()
	// Only append additional hashes here if the HLSL contains any additional includes
}

UComputeDataProvider* UOptimusAnimAttributeDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusAnimAttributeDataProvider* Provider = NewObject<UOptimusAnimAttributeDataProvider>();
	Provider->Init(Cast<USkeletalMeshComponent>(InBinding), AttributeArray.InnerArray);
	return Provider;
}

const FOptimusAnimAttributeDescription& UOptimusAnimAttributeDataInterface::AddAnimAttribute(const FString& InName, FName InBoneName,
	const FOptimusDataTypeRef& InDataType)
{
	return AttributeArray.InnerArray.AddDefaulted_GetRef()
		.Init(GetUnusedAttributeName(InName), InBoneName, InDataType);
}


void UOptimusAnimAttributeDataInterface::OnDataTypeChanged(FName InTypeName)
{
	for (FOptimusAnimAttributeDescription& AttributeDescription : AttributeArray)
	{
		if (AttributeDescription.DataType.TypeName == InTypeName)
		{
			AttributeDescription.DefaultValueStruct.SetType(AttributeDescription.DataType);
		}
	}
}

FString UOptimusAnimAttributeDataInterface::GetUnusedAttributeName(const FString& InName) const
{
	TMap<FString, int32> AttributeNames;
	for (int32 Index = 0; Index < AttributeArray.Num(); Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
						
		AttributeNames.FindOrAdd(Attribute.Name);
	}

	int32 Suffix = 0;
	FString NewName = InName;
	while (AttributeNames.Contains(NewName))
	{
		NewName = FString::Printf(TEXT("%s_%d"), *InName, Suffix);
		Suffix++;
	}

	return NewName;
}

void UOptimusAnimAttributeDataInterface::UpdateAttributePinNamesAndHlslIds()
{
	const int32 NumAttributes = AttributeArray.Num(); 

	TMap<FString, TArray<int32>> AttributesByName;

	for (int32 Index = 0; Index < NumAttributes; Index++)
	{
		const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
		AttributesByName.FindOrAdd(Attribute.Name).Add(Index);
	}

	for (const TTuple<FString, TArray<int32>>& AttributeGroup : AttributesByName)
	{
		// For attributes that share the same name, prepend type name or bone name
		// or both to make sure pin names are unique
		bool bMoreThanOneTypes = false;
		bool bMoreThanOneBones = false;

		TOptional<FOptimusDataTypeRef> LastType;
		TOptional<FName> LastBone;
		
		for (int32 Index : AttributeGroup.Value)
		{
			const FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];
			
			if (!LastBone.IsSet())
			{
				LastBone = Attribute.BoneName;
			}
			else if (Attribute.BoneName!= LastBone.GetValue())
			{
				bMoreThanOneBones = true;
			}
			
			if (!LastType.IsSet())
			{
				LastType = Attribute.DataType;
			}
			else if (Attribute.DataType != LastType.GetValue())
			{
				bMoreThanOneTypes = true;
			}

			if (bMoreThanOneBones && bMoreThanOneTypes)
			{
				break;
			}
		}
		
		for (int32 Index : AttributeGroup.Value)
		{
			FOptimusAnimAttributeDescription& Attribute = AttributeArray[Index];

			Attribute.UpdatePinNameAndHlslId(bMoreThanOneBones, bMoreThanOneTypes);
		}
	}
}


FOptimusAnimAttributeRuntimeData::FOptimusAnimAttributeRuntimeData(
	const FOptimusAnimAttributeDescription& InDescription)
{
	Name = *InDescription.Name;
	HlslId = *InDescription.HlslId;
	BoneName = InDescription.BoneName;
	CachedBoneIndex = 0;
	
	Offset = 0;
	Size = InDescription.DataType->ShaderValueSize;

	const FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	ConvertFunc = Registry.FindPropertyValueConvertFunc(InDescription.DataType.TypeName);

	ArrayMetadata = Registry.FindArrayMetadata(InDescription.DataType.TypeName);

	AttributeType = Registry.FindAttributeType(InDescription.DataType.TypeName);

	CachedDefaultValue = InDescription.DefaultValueStruct.GetShaderValue(InDescription.DataType);
}

void UOptimusAnimAttributeDataProvider::Init(
	USkeletalMeshComponent* InSkeletalMesh,
	const TArray<FOptimusAnimAttributeDescription>& InAttributeArray
	)
{
	SkeletalMesh = InSkeletalMesh;

	// Convert description to runtime data
	for (const FOptimusAnimAttributeDescription& Attribute : InAttributeArray)
	{
		AttributeRuntimeData.Add(Attribute);
	}

	for (FOptimusAnimAttributeRuntimeData& Attribute : AttributeRuntimeData)
	{
		// Skip this step in case that there is no skeletal mesh, this can happen if
		// the preview scene does not have a preview mesh assigned
		if (SkeletalMesh.IsValid() && SkeletalMesh->GetSkeletalMeshAsset())
		{
			if (Attribute.BoneName != NAME_None)
			{
				Attribute.CachedBoneIndex = SkeletalMesh->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(Attribute.BoneName);
			}
			else
			{
				// default to look for the attribute on the root bone
				Attribute.CachedBoneIndex = 0;
			}
		}
	}

	// Compute offset within the shader parameter buffer for each attribute
	FShaderParametersMetadataBuilder Builder;

	TArray<FShaderParametersMetadata*> AllocatedMetadatas;
	TArray<FShaderParametersMetadata*> NestedStructs;
	for (const FOptimusAnimAttributeDescription& Attribute : InAttributeArray)
	{
		ComputeFramework::AddParamForType(Builder, *Attribute.HlslId, Attribute.DataType->ShaderValueType, NestedStructs);
	}

	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("UAnimAttributeDataInterface"));
	AllocatedMetadatas.Add(ShaderParameterMetadata);
	AllocatedMetadatas.Append(NestedStructs);

	TotalNumArrays = 0;
	TArray<FShaderParametersMetadata::FMember> const& Members = ShaderParameterMetadata->GetMembers();
	for (int32 Index = 0; Index < AttributeRuntimeData.Num(); ++Index)
	{
		FOptimusAnimAttributeRuntimeData& RuntimeData = AttributeRuntimeData[Index];
		check(RuntimeData.HlslId == Members[Index].GetName());
		
		RuntimeData.Offset = Members[Index].GetOffset();

		RuntimeData.ArrayIndexStart = TotalNumArrays;
		TotalNumArrays += RuntimeData.ArrayMetadata.Num();
	}

	AttributeBufferSize = ShaderParameterMetadata->GetSize();

	for (const FShaderParametersMetadata* AllocatedMetaData : AllocatedMetadatas)
	{
		delete AllocatedMetaData;
	}
}

FComputeDataProviderRenderProxy* UOptimusAnimAttributeDataProvider::GetRenderProxy()
{
	FOptimusAnimAttributeDataProviderProxy* Proxy = new FOptimusAnimAttributeDataProviderProxy(AttributeBufferSize, TotalNumArrays);
	
	const FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();
	
	const UE::Anim::FMeshAttributeContainer& AttributeContainer = SkeletalMesh->GetCustomAttributes();
	
	for (int32 Index = 0; Index < AttributeRuntimeData.Num(); ++Index)
	{
		const FOptimusAnimAttributeRuntimeData& AttributeData = AttributeRuntimeData[Index];

		for (int32 ArrayIndex = 0; ArrayIndex < AttributeData.ArrayMetadata.Num(); ArrayIndex++)
		{
			int32 ToplevelArrayIndex = AttributeData.ArrayIndexStart + ArrayIndex;
			const FOptimusDataTypeRegistry::FArrayMetadata& Metadata = AttributeData.ArrayMetadata[ArrayIndex];
			if (ensure(Proxy->AttributeArrayMetadata.IsValidIndex(ToplevelArrayIndex)))
			{
				Proxy->AttributeArrayMetadata[ToplevelArrayIndex].Offset = AttributeData.Offset + Metadata.ShaderValueOffset;
				Proxy->AttributeArrayMetadata[ToplevelArrayIndex].ElementSize = Metadata.ElementShaderValueSize;
			}
		}

		const UE::Anim::FAttributeId Id = {AttributeData.Name, FCompactPoseBoneIndex(AttributeData.CachedBoneIndex)} ;

		bool bIsValueSet = false;
		
		FOptimusDataTypeRegistry::PropertyValueConvertFuncT ConvertFunc = AttributeData.ConvertFunc;

		if (ConvertFunc)
		{
			UScriptStruct* AttributeType = AttributeData.AttributeType;
		
			if (const uint8* Attribute = AttributeContainer.Find(AttributeType, Id))
			{
				bIsValueSet = true;
				
				const uint8* ValuePtr = Attribute;
				int32 ValueSize = AttributeType->GetStructureSize();

				// TODO: use a specific function to extract the value from the attribute
				// it works for now because even if the attribute type != actual value type
				// it should only have a single property, whose type == actual property type
				
				ConvertFunc(
					{ValuePtr, ValueSize},
					{
						{Proxy->AttributeBuffer.GetData() + AttributeData.Offset, AttributeData.Size},
						{Proxy->AttributeArrayData.GetData() + AttributeData.ArrayIndexStart, AttributeData.ArrayMetadata.Num()}	
					});
			}
		
			// Use the default value if the attribute was not found
			if (!bIsValueSet)
			{
				const uint8* DefaultValuePtr = AttributeData.CachedDefaultValue.ShaderValue.GetData();
				const uint32 DefaultValueSize = AttributeData.CachedDefaultValue.ShaderValue.Num();

				FMemory::Memcpy(&Proxy->AttributeBuffer[AttributeData.Offset], DefaultValuePtr, DefaultValueSize);

				if (ensure(AttributeData.ArrayMetadata.Num() == AttributeData.CachedDefaultValue.ArrayList.Num()))
				{
					for (int32 ArrayIndex = 0; ArrayIndex < AttributeData.CachedDefaultValue.ArrayList.Num(); ArrayIndex++)
					{
						const int32 ToplevelArrayIndex = AttributeData.ArrayIndexStart + ArrayIndex;
						if (ensure(Proxy->AttributeArrayData.IsValidIndex(ToplevelArrayIndex)))
						{
							Proxy->AttributeArrayData[ToplevelArrayIndex] = AttributeData.CachedDefaultValue.ArrayList[ArrayIndex];
						}
					}
				}	
			}
		}
	}
	
	return Proxy;
}

FOptimusAnimAttributeDataProviderProxy::FOptimusAnimAttributeDataProviderProxy(
	int32 InAttributeBufferSize,
	int32 InTotalNumArrays)
{
	AttributeBuffer.AddDefaulted(InAttributeBufferSize);
	AttributeArrayMetadata.AddDefaulted(InTotalNumArrays);
	AttributeArrayData.AddDefaulted(InTotalNumArrays);
}

bool FOptimusAnimAttributeDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != AttributeBuffer.Num())
	{
		return false;
	}
	if (AttributeArrayData.Num() != AttributeArrayMetadata.Num())
	{
		return false;
	}

	return true;
}

void FOptimusAnimAttributeDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	ArrayBuffers.Reset();
	ArrayBufferSRVs.Reset();

	if (ensure(AttributeArrayData.Num() == AttributeArrayMetadata.Num()))
	{
		for (int32 ArrayIndex = 0; ArrayIndex < AttributeArrayMetadata.Num(); ArrayIndex++)
		{
			const FArrayMetadata& ArrayMetadata = AttributeArrayMetadata[ArrayIndex];
			const TArray<uint8>& ArrayData = AttributeArrayData[ArrayIndex].ArrayOfValues;
			
			ArrayBuffers.Add(
				GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateStructuredDesc(
						ArrayMetadata.ElementSize,
						FMath::Max(ArrayData.Num() / ArrayMetadata.ElementSize, 1)), TEXT("Optimus.AnimAttributeInnerBuffer")));
			ArrayBufferSRVs.Add(GraphBuilder.CreateSRV(ArrayBuffers.Last()));
			GraphBuilder.QueueBufferUpload(ArrayBuffers.Last(), ArrayData.GetData(), ArrayData.Num(), ERDGInitialDataFlags::None);	
		}
	}
}

void FOptimusAnimAttributeDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		uint8* ParameterBuffer = (InDispatchData.ParameterBuffer + InDispatchData.ParameterBufferOffset + InDispatchData.ParameterBufferStride * InvocationIndex);
		FMemory::Memcpy(ParameterBuffer, AttributeBuffer.GetData(), InDispatchData.ParameterStructSize);

		for (int32 ArrayIndex = 0; ArrayIndex < AttributeArrayMetadata.Num(); ArrayIndex++)
		{
			const FArrayMetadata& ArrayMetadata = AttributeArrayMetadata[ArrayIndex];
			*((FRDGBufferSRV**)(ParameterBuffer + ArrayMetadata.Offset)) = ArrayBufferSRVs[ArrayIndex];
		}
	}
}
