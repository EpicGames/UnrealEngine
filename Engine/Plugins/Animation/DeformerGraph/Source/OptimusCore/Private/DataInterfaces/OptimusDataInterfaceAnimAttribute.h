// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "OptimusDataType.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusValueContainerStruct.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "OptimusDataInterfaceAnimAttribute.generated.h"

#define UE_API OPTIMUSCORE_API

class USkeletalMeshComponent;
class UOptimusValueContainer;
class FRDGBuilder;
class FRDGBuffer;
class FRDGBufferSRV;

USTRUCT()
struct FOptimusAnimAttributeDescription
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FString Name;

	// Used to look for attributes associated with a specific bone. Default to use the root bone
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FName BoneName;

	UPROPERTY(EditAnywhere, Category = "Data Interface", meta=(UseInAnimAttribute))
	FOptimusDataTypeRef DataType;

	// Default value if the animation attribute is not found
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FOptimusValueContainerStruct DefaultValueStruct;

	UPROPERTY()
	FString HlslId;

	UPROPERTY()
	FName PinName;
	
	void UpdatePinNameAndHlslId(bool bInIncludeBoneName = true, bool bInIncludeTypeName = true);
	
	// Helpers
	FOptimusAnimAttributeDescription& Init(const FString& InName, FName InBoneName,
	const FOptimusDataTypeRef& InDataType);
	
private:
	friend class UOptimusAnimAttributeDataInterface;
	
	FString GetFormattedId(
		const FString& InDelimiter,
		bool bInIncludeBoneName,
		bool bInIncludeTypeName) const;

	// Deprecated
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "use DefaultValueStruct instead"))
	TObjectPtr<UOptimusValueContainer> DefaultValue_DEPRECATED = nullptr;
};

USTRUCT()
struct FOptimusAnimAttributeArray
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, DisplayName = "Animation Attributes" ,Category = "Data Interface")
	TArray<FOptimusAnimAttributeDescription> InnerArray;

	template <typename Predicate>
	const FOptimusAnimAttributeDescription* FindByPredicate(Predicate Pred) const
	{
		return InnerArray.FindByPredicate(Pred);
	}

	bool IsEmpty() const
	{
		return InnerArray.IsEmpty();
	}

	FOptimusAnimAttributeDescription& Last(int32 IndexFromTheEnd = 0)
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	const FOptimusAnimAttributeDescription& Last(int32 IndexFromTheEnd = 0) const
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	FOptimusAnimAttributeArray& operator=(const TArray<FOptimusAnimAttributeDescription>& Rhs)
	{
		InnerArray = Rhs;
		return *this;
	}
	
	int32 Num() const { return InnerArray.Num();}
	bool IsValidIndex(int32 Index) const { return Index < InnerArray.Num() && Index >= 0; }
	const FOptimusAnimAttributeDescription& operator[](int32 InIndex) const { return InnerArray[InIndex]; }
	FOptimusAnimAttributeDescription& operator[](int32 InIndex) { return InnerArray[InIndex]; }
	FORCEINLINE	TArray<FOptimusAnimAttributeDescription>::RangedForIteratorType      begin()       { return InnerArray.begin(); }
	FORCEINLINE	TArray<FOptimusAnimAttributeDescription>::RangedForConstIteratorType begin() const { return InnerArray.begin(); }
	FORCEINLINE	TArray<FOptimusAnimAttributeDescription>::RangedForIteratorType      end()         { return InnerArray.end();   }
	FORCEINLINE	TArray<FOptimusAnimAttributeDescription>::RangedForConstIteratorType end() const   { return InnerArray.end();   }
};


/** Compute Framework Data Interface for reading animation attributes on skeletal mesh. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusAnimAttributeDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:

	UE_API UOptimusAnimAttributeDataInterface();

#if WITH_EDITOR
	UE_API void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif
	UE_API void PostLoad() override;
	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	
	UE_API void Initialize() override;
	bool CanPinDefinitionChange() override {return true;};
	UE_API void RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode) override;
	
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("AnimAttribute"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API void GetStructDeclarations(TSet<FString>& OutStructsSeen, TArray<FString>& OutStructs) const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UE_API const FOptimusAnimAttributeDescription& AddAnimAttribute(const FString& InName, FName InBoneName, const FOptimusDataTypeRef& InDataType);

	UE_API void OnDataTypeChanged(FName InTypeName) override;
	
	UPROPERTY(EditAnywhere, Category = "Animation Attribute", meta = (ShowOnlyInnerProperties))
	FOptimusAnimAttributeArray AttributeArray;

private:
	UE_API FString GetUnusedAttributeName(const FString& InName) const;
	UE_API void UpdateAttributePinNamesAndHlslIds();
	
	FOnPinDefinitionChanged OnPinDefinitionChangedDelegate;
	FOnPinDefinitionRenamed OnPinDefinitionRenamedDelegate;
};

// Runtime data with cached values baked out from AttributeDescription
struct FOptimusAnimAttributeRuntimeData
{
	FOptimusAnimAttributeRuntimeData() = default;
	
	FOptimusAnimAttributeRuntimeData(const FOptimusAnimAttributeDescription& InDescription);
	
	FName Name;
	FName HlslId;

	FName BoneName;

	int32 CachedBoneIndex = 0;
	
	int32 Offset = INDEX_NONE;

	int32 Size = 0;

	int32 ArrayIndexStart = INDEX_NONE;

	FOptimusDataTypeRegistry::PropertyValueConvertFuncT	ConvertFunc = nullptr;

	TArray<FOptimusDataTypeRegistry::FArrayMetadata> ArrayMetadata;

	UScriptStruct* AttributeType = nullptr;

	FShaderValueContainer CachedDefaultValue;
};

/** Compute Framework Data Provider for reading animation attributes on skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusAnimAttributeDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UOptimusAnimAttributeDataProvider() = default;
	
	void Init(
		USkeletalMeshComponent* InSkeletalMesh,
		const TArray<FOptimusAnimAttributeDescription>& InAttributeArray
	);
	
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMesh = nullptr;

	TArray<FOptimusAnimAttributeRuntimeData> AttributeRuntimeData;

	int32 AttributeBufferSize = 0;
	
	int32 TotalNumArrays = 0;
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusAnimAttributeDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	struct FArrayMetadata
	{
		int32 Offset;
		int32 ElementSize;
	};
	
	FOptimusAnimAttributeDataProviderProxy(
		int32 InAttributeBufferSize,
		int32 InTotalNumArrays
	);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

public:
	TArray<uint8> AttributeBuffer;
	TArray<FArrayMetadata> AttributeArrayMetadata;
	TArray<FArrayShaderValue> AttributeArrayData;

private:
	TArray<FRDGBuffer*> ArrayBuffers;
	TArray<FRDGBufferSRV*> ArrayBufferSRVs;
};

#undef UE_API
