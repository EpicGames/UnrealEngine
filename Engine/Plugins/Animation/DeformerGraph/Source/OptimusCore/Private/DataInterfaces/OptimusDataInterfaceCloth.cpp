// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceCloth.h"

#include "Components/SkeletalMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderParameterMetadataBuilder.h"
#include "OptimusDataDomain.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"
#include "ShaderCompilerCore.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceCloth)

FString UOptimusClothDataInterface::GetDisplayName() const
{
	return TEXT("Cloth");
}

TArray<FOptimusCDIPinDefinition> UOptimusClothDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "ClothToLocal", "ReadClothToLocal" });
	Defs.Add({ "ClothWeight", "ReadClothWeight", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "ClothPosition", "ReadClothPosition", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "ClothTangentX", "ReadClothTangentX", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "ClothTangentZ", "ReadClothTangentZ", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusClothDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusClothDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadClothToLocal"))
		.AddReturnType(EShaderFundamentalType::Float, 4, 4);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadClothWeight"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadClothPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadClothTangentX"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadClothTangentZ"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FClothDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, BaseVertexIndex)
	SHADER_PARAMETER(uint32, InputStreamStart)
	SHADER_PARAMETER(uint32, NumInfluencesPerVertex)
	SHADER_PARAMETER(float, ClothBlendWeight)
	SHADER_PARAMETER(FVector3f, MeshScale)
	SHADER_PARAMETER(FMatrix44f, ClothToLocal)
	SHADER_PARAMETER_SRV(Buffer<float4>, ClothBuffer)
	SHADER_PARAMETER_SRV(Buffer<float2>, ClothPositionsAndNormalsBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusClothDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FClothDataInterfaceParameters>(UID);
}

void UOptimusClothDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_CLOTH"), 2);
}

TCHAR const* UOptimusClothDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceCloth.ush");

TCHAR const* UOptimusClothDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusClothDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusClothDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusClothDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusClothDataProvider* Provider = NewObject<UOptimusClothDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusClothDataProvider::GetRenderProxy()
{
	return new FOptimusClothDataProviderProxy(SkinnedMesh.Get());
}


FOptimusClothDataProviderProxy::FOptimusClothDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent != nullptr && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
		FrameNumber = SkinnedMeshComponent->GetScene() != nullptr ? SkinnedMeshComponent->GetScene()->GetFrameNumber() + 1 : 0; // +1 matches the logic for FrameNumberToPrepare in FSkeletalMeshObjectGPUSkin::Update()

		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SkinnedMeshComponent))
		{
			ClothBlendWeight = SkeletalMeshComponent->ClothBlendWeight;
			MeshScale = (FVector3f)SkeletalMeshComponent->GetComponentScale();
		}
	}
}

bool FOptimusClothDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[SkeletalMeshObject->GetLOD()].RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}

	return true;
}

struct FClothDataInterfacePermutationIds
{
	uint32 EnableDeformerCloth = 0;

	FClothDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_CLOTH"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerCloth = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusClothDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FClothDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		const bool bPreviousFrame = false;
		FSkeletalMeshDeformerHelpers::FClothBuffers ClothBuffers = FSkeletalMeshDeformerHelpers::GetClothBuffersForReading(SkeletalMeshObject, LodIndex, InvocationIndex, FrameNumber, bPreviousFrame);
		const bool bValidCloth = (ClothBuffers.ClothInfluenceBuffer != nullptr) && (ClothBuffers.ClothSimulatedPositionAndNormalBuffer != nullptr);

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidCloth ? PermutationIds.EnableDeformerCloth : 0);
	}
}

void FOptimusClothDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		const uint32 NumWrapDeformerWeights = RenderSection.ClothMappingDataLODs.Num() > 0 ? RenderSection.ClothMappingDataLODs[0].Num() : 0;
		const bool bMultipleWrapDeformerInfluences = RenderSection.NumVertices < NumWrapDeformerWeights;
		const int32 NumClothInfluencesPerVertex = bMultipleWrapDeformerInfluences ? 5 : 1; // From ClothingMeshUtils.cpp. Could make this a permutation like with skin cache.

		const bool bPreviousFrame = false;
		FSkeletalMeshDeformerHelpers::FClothBuffers ClothBuffers = FSkeletalMeshDeformerHelpers::GetClothBuffersForReading(SkeletalMeshObject, LodIndex, InvocationIndex, FrameNumber, bPreviousFrame);
		const bool bValidCloth = (ClothBuffers.ClothInfluenceBuffer != nullptr) && (ClothBuffers.ClothSimulatedPositionAndNormalBuffer != nullptr);

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.BaseVertexIndex = RenderSection.BaseVertexIndex;
		Parameters.InputStreamStart = ClothBuffers.ClothInfluenceBufferOffset;
		Parameters.ClothBlendWeight = bValidCloth ? ClothBlendWeight : 0.f;
		Parameters.MeshScale = bValidCloth ? MeshScale : FVector3f::OneVector;
		Parameters.NumInfluencesPerVertex = bValidCloth ? NumClothInfluencesPerVertex : 0;
		Parameters.ClothToLocal = ClothBuffers.ClothToLocal;
		Parameters.ClothBuffer = bValidCloth ? ClothBuffers.ClothInfluenceBuffer : NullSRVBinding;
		Parameters.ClothPositionsAndNormalsBuffer = bValidCloth ? ClothBuffers.ClothSimulatedPositionAndNormalBuffer : NullSRVBinding;
	}
}
