// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceConnectivity.h"

#include "Components/SkeletalMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceConnectivity)

namespace
{
	// Code initially taken from NiagaraDataInterfaceSkeletalMeshConnectivity.cpp
	// But changed for vertex-vertex connectivity instead of vertex-triangle connectivity.
	static void BuildAdjacencyBuffer(const FSkeletalMeshLODRenderData& LodRenderData, int32 MaxAdjacencyCount, TArray<uint32>& Buffer)
	{
		const FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodRenderData.MultiSizeIndexContainer.GetIndexBuffer();
		const uint32 IndexCount = IndexBuffer->Num();

		const FPositionVertexBuffer& VertexBuffer = LodRenderData.StaticVertexBuffers.PositionVertexBuffer;
		const uint32 VertexCount = LodRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();

		Buffer.SetNum(MaxAdjacencyCount * VertexCount, EAllowShrinking::Yes);
		FMemory::Memset(Buffer.GetData(), 0xFF, Buffer.Num() * sizeof(uint32));

		int32 BaseTriangle = 0;
		int32 BaseVertex = 0;
		for (int32 SectionIndex = 0; SectionIndex < LodRenderData.RenderSections.Num(); ++SectionIndex)
		{
			FSkelMeshRenderSection const& RenderSection = LodRenderData.RenderSections[SectionIndex];
			int32 NumTriangles = RenderSection.NumTriangles;
			int32 NumVertices = RenderSection.NumVertices;
		
			TArray<uint32> RedirectionArray;
			RedirectionArray.SetNum(VertexCount);
			TMap<FVector, int32 /*UniqueVertexIndex*/> UniqueIndexMap;

			for (int32 TriangleIt = BaseTriangle; TriangleIt < BaseTriangle + NumTriangles; ++TriangleIt)
			{
				const uint32 V[3] =
				{
					IndexBuffer->Get(TriangleIt * 3 + 0),
					IndexBuffer->Get(TriangleIt * 3 + 1),
					IndexBuffer->Get(TriangleIt * 3 + 2)
				};

				const FVector P[3] =
				{
					(FVector)VertexBuffer.VertexPosition(V[0]),
					(FVector)VertexBuffer.VertexPosition(V[1]),
					(FVector)VertexBuffer.VertexPosition(V[2])
				};

				for (int32 i = 0; i < 3; ++i)
				{
					const uint32 VertexIndex = RedirectionArray[V[i]] = UniqueIndexMap.FindOrAdd(P[i], V[i]);

					TArrayView<uint32> AdjacentVertices = MakeArrayView(Buffer.GetData() + VertexIndex * MaxAdjacencyCount, MaxAdjacencyCount);

					for (int32 a = 1; a < 3; ++a)
					{
						const uint32 AdjacentVertexIndex = V[(i + a) % 3];

						int32 InsertionPoint = 0;
						while (InsertionPoint < MaxAdjacencyCount)
						{
							const uint32 Test = AdjacentVertices[InsertionPoint];

							if (Test == INDEX_NONE)
							{
								AdjacentVertices[InsertionPoint] = AdjacentVertexIndex;
								break;
							}

							if (AdjacentVertexIndex == Test)
							{
								break;
							}

							if (AdjacentVertexIndex < Test)
							{
								// skip empty entries
								int32 ShiftIt = MaxAdjacencyCount - 1;
								while (AdjacentVertices[ShiftIt - 1] == INDEX_NONE)
								{
									--ShiftIt;
								}

								// shift the results down and then insert
								do
								{
									AdjacentVertices[ShiftIt] = AdjacentVertices[ShiftIt - 1];
									--ShiftIt;
								} while (ShiftIt > InsertionPoint);

								AdjacentVertices[InsertionPoint] = AdjacentVertexIndex;
								break;
							}

							++InsertionPoint;
						}
					}
				}
			}

			for (int32 VertexIt = BaseVertex + 1; VertexIt < BaseVertex + NumVertices; ++VertexIt)
			{
				// if this vertex has a sibling we copy the data over
				const int32 SiblingIndex = RedirectionArray[VertexIt];
				if (SiblingIndex != VertexIt)
				{
					FMemory::Memcpy(Buffer.GetData() + VertexIt * MaxAdjacencyCount, Buffer.GetData() + SiblingIndex * MaxAdjacencyCount, MaxAdjacencyCount * sizeof(uint32));
				}
			}

			BaseTriangle += NumTriangles;
			BaseVertex += NumVertices;
		}
	}
}


FString UOptimusConnectivityDataInterface::GetDisplayName() const
{
	return TEXT("Connectivity");
}

TArray<FOptimusCDIPinDefinition> UOptimusConnectivityDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "NumConnectedVertices", "ReadNumConnectedVertices", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "ConnectedVertex", "ReadConnectedVertex",  {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Index0, "ReadNumConnectedVertices"}} });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusConnectivityDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusConnectivityDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumConnectedVertices"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadConnectedVertex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FConnectivityDataInterfaceParameters, )
SHADER_PARAMETER(uint32, NumVertices)
SHADER_PARAMETER(uint32, MaxConnectedVertexCount)
SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ConnectivityBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusConnectivityDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FConnectivityDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusConnectivityDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceConnectivity.ush");

TCHAR const* UOptimusConnectivityDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusConnectivityDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusConnectivityDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusConnectivityDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusConnectivityDataProvider* Provider = NewObject<UOptimusConnectivityDataProvider>();

	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);

	if (Provider->SkinnedMesh != nullptr)
	{
		// Build adjacency and store with the provider.
		// todo[CF]: We need to move this to the skeletal mesh and make part of cooked mesh data instead.
		FSkeletalMeshRenderData const* SkeletalMeshRenderData = Provider->SkinnedMesh->GetSkeletalMeshRenderData();
		if (SkeletalMeshRenderData != nullptr)
		{
			Provider->AdjacencyBufferPerLod.SetNum(SkeletalMeshRenderData->NumInlinedLODs);
			for (int32 LodIndex = 0; LodIndex < SkeletalMeshRenderData->NumInlinedLODs; ++LodIndex)
			{
				FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData->LODRenderData[LodIndex];
				BuildAdjacencyBuffer(*LodRenderData, UOptimusConnectivityDataInterface::MaxConnectedVertexCount, Provider->AdjacencyBufferPerLod[LodIndex]);
			}
		}
	}

	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusConnectivityDataProvider::GetRenderProxy()
{
	return new FOptimusConnectivityDataProviderProxy(SkinnedMesh.Get(), AdjacencyBufferPerLod);
}


FOptimusConnectivityDataProviderProxy::FOptimusConnectivityDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent, TArray< TArray<uint32> >& InAdjacencyBufferPerLod)
	: AdjacencyBufferPerLod(InAdjacencyBufferPerLod)
{
	if (SkinnedMeshComponent != nullptr && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
	}
}

bool FOptimusConnectivityDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject || AdjacencyBufferPerLod.Num() == 0)
	{
		return false;
	}

	return true;
}

void FOptimusConnectivityDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	TArray<uint32> const& ConnectivityData = AdjacencyBufferPerLod[LodIndex];

	// todo[CF]: Updating buffer every frame is obviously bad, but just getting things working initially.
	ConnectivityBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), ConnectivityData.Num()), TEXT("Optimus.Connectivity"));
	ConnectivityBufferSRV = GraphBuilder.CreateSRV(ConnectivityBuffer);
	GraphBuilder.QueueBufferUpload(ConnectivityBuffer, ConnectivityData.GetData(), ConnectivityData.Num() * sizeof(uint32), ERDGInitialDataFlags::None);
}

void FOptimusConnectivityDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.MaxConnectedVertexCount = UOptimusConnectivityDataInterface::MaxConnectedVertexCount;
		Parameters.ConnectivityBuffer = ConnectivityBufferSRV;
	}
}
