// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceConnectivity.generated.h"

#define UE_API OPTIMUSCORE_API

class FConnectivityDataInterfaceParameters;
class FRDGBuffer;
class FRDGBufferSRV;
class FSceneInterface;
class FSkeletalMeshObject;
class USkinnedMeshComponent;


/** Compute Framework Data Interface for reading skeletal mesh. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusConnectivityDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("Connectivity"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	// Hardcoded max connected count.
	static const int32 MaxConnectedVertexCount = 12;

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusConnectivityDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	TArray< TArray<uint32> > AdjacencyBufferPerLod;
};

class FOptimusConnectivityDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusConnectivityDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent, TArray< TArray<uint32> >& InAdjacencyBufferPerLod);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData);
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FConnectivityDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	TArray< TArray<uint32> > const& AdjacencyBufferPerLod;

	FRDGBuffer* ConnectivityBuffer = nullptr;
	FRDGBufferSRV* ConnectivityBufferSRV = nullptr;
};

#undef UE_API
