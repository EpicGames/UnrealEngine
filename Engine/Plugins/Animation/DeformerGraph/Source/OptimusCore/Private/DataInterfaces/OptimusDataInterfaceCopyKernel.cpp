// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceCopyKernel.h"

#include "OptimusComponentSource.h"
#include "OptimusExpressionEvaluator.h"
#include "OptimusHelpers.h"
#include "OptimusObjectVersion.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceCopyKernel)


const TCHAR* UOptimusCopyKernelDataInterface:: ReadNumThreadsFunctionName = TEXT("ReadNumThreads");
const TCHAR* UOptimusCopyKernelDataInterface:: ReadNumThreadsPerInvocationFunctionName = TEXT("ReadNumThreadsPerInvocation");
const TCHAR* UOptimusCopyKernelDataInterface:: ReadThreadIndexOffsetFunctionName = TEXT("ReadThreadIndexOffset");


void UOptimusCopyKernelDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(ReadNumThreadsFunctionName)
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint, 3));

	OutFunctions.AddDefaulted_GetRef()
		.SetName(ReadNumThreadsPerInvocationFunctionName)
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(ReadThreadIndexOffsetFunctionName)
		.AddReturnType(FShaderValueType::Get(EShaderFundamentalType::Uint));
}

BEGIN_SHADER_PARAMETER_STRUCT(FCopyKernelDataInterfaceParameters, )
	SHADER_PARAMETER(FUintVector3, NumThreads)
	SHADER_PARAMETER(uint32, NumThreadsPerInvocation)
	SHADER_PARAMETER(uint32, ThreadIndexOffset)
END_SHADER_PARAMETER_STRUCT()


void UOptimusCopyKernelDataInterface::GetShaderParameters(TCHAR const* UID,
	FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FCopyKernelDataInterfaceParameters>(UID);
}

const TCHAR* UOptimusCopyKernelDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceCopyKernel.ush");

const TCHAR* UOptimusCopyKernelDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusCopyKernelDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusCopyKernelDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusCopyKernelDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding,
	uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusCopyKernelDataProvider* Provider = NewObject<UOptimusCopyKernelDataProvider>();

	Provider->InitFromDataInterface(this, InBinding);

	return Provider;
}

void UOptimusCopyKernelDataInterface::SetExecutionDomain(const FString& InExecutionDomain)
{
	NumThreadsExpression = InExecutionDomain;
}

void UOptimusCopyKernelDataInterface::SetComponentBinding(const UOptimusComponentSourceBinding* InBinding)
{
	ComponentSourceBinding = InBinding;
}

const FString& UOptimusCopyKernelDataInterface::GetExecutionDomain() const
{
	return NumThreadsExpression;
}

const TCHAR* UOptimusCopyKernelDataInterface::GetReadNumThreadsFunctionName() const
{
	return ReadNumThreadsFunctionName;
}

const TCHAR* UOptimusCopyKernelDataInterface::GetReadNumThreadsPerInvocationFunctionName() const
{
	return ReadNumThreadsPerInvocationFunctionName;
}

const TCHAR* UOptimusCopyKernelDataInterface::GetReadThreadIndexOffsetFunctionName() const
{
	return ReadThreadIndexOffsetFunctionName;
}

void UOptimusCopyKernelDataProvider::InitFromDataInterface(const UOptimusCopyKernelDataInterface* InDataInterface, const UObject* InBinding)
{
	WeakComponent = Cast<UActorComponent>(InBinding);
	WeakComponentSource = InDataInterface->ComponentSourceBinding->GetComponentSource();
	ParseResult = Optimus::ParseExecutionDomainExpression(InDataInterface->NumThreadsExpression, WeakComponentSource);
}

FComputeDataProviderRenderProxy* UOptimusCopyKernelDataProvider::GetRenderProxy()
{
	TArray<int32> InvocationCounts;

	Optimus::EvaluateExecutionDomainExpressionParseResult(ParseResult, WeakComponentSource, WeakComponent,InvocationCounts);
	
	FOptimusCopyKernelDataProviderProxy* Proxy = new FOptimusCopyKernelDataProviderProxy(MoveTemp(InvocationCounts));
	return Proxy;
}

FOptimusCopyKernelDataProviderProxy::FOptimusCopyKernelDataProviderProxy(
	TArray<int32>&& InInvocationThreadCounts
	) :
	InvocationThreadCounts(MoveTemp(InInvocationThreadCounts))
{
	for (int32 Count : InvocationThreadCounts)
	{
		TotalThreadCount += Count;
	}
}

bool FOptimusCopyKernelDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (TotalThreadCount <= 0)
	{
		return false;
	}

	return true;
}

int32 FOptimusCopyKernelDataProviderProxy::GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const
{
	InOutThreadCounts.Reset(InvocationThreadCounts.Num());
	for (const int32 Count : InvocationThreadCounts)
	{
		InOutThreadCounts.Add({Count, 1, 1});
	}
	return InOutThreadCounts.Num();
}

void FOptimusCopyKernelDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);

	int32 NumDispatchedThreads = 0;
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumThreads = FUintVector3(TotalThreadCount, 1, 1);
		
		int32 NumThreadsPerInvocation = InDispatchData.bUnifiedDispatch ? TotalThreadCount : InvocationThreadCounts[InvocationIndex] ;
		Parameters.NumThreadsPerInvocation = NumThreadsPerInvocation;
		
		Parameters.ThreadIndexOffset = InDispatchData.bUnifiedDispatch ? 0 : NumDispatchedThreads;
		
		NumDispatchedThreads += NumThreadsPerInvocation;
	}
}


