// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusComputeKernelDataInterface.h"
#include "OptimusComponentSource.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDataType.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "OptimusExpressionEvaluator.h"

#include "OptimusDataInterfaceCopyKernel.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusComponentSource;
class UOptimusComponentSourceBinding;
class FCopyKernelDataInterfaceParameters;

UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusCopyKernelDataInterface :
	public UComputeDataInterface,
	public IOptimusComputeKernelDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("CopyKernelData"); }
	bool IsExecutionInterface() const override { return true; }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API const TCHAR* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	//~ Begin IOptimusComputeKernelDataInterface Interface
	UE_API void SetExecutionDomain(const FString& InExecutionDomain) override;
	UE_API void SetComponentBinding(const UOptimusComponentSourceBinding* InBinding) override;
	UE_API const FString& GetExecutionDomain() const override;
	UE_API const TCHAR* GetReadNumThreadsFunctionName() const override;
	UE_API const TCHAR* GetReadNumThreadsPerInvocationFunctionName() const override;
	UE_API const TCHAR* GetReadThreadIndexOffsetFunctionName() const override;
	//~ End IOptimusComputeKernelDataInterface Interface
	
	UPROPERTY()
	TWeakObjectPtr<const UOptimusComponentSourceBinding> ComponentSourceBinding;
	
	UPROPERTY()
	FString NumThreadsExpression;
	
protected:	
	static UE_API const TCHAR* TemplateFilePath;
	
	static UE_API const TCHAR* ReadNumThreadsFunctionName;
	static UE_API const TCHAR* ReadNumThreadsPerInvocationFunctionName;
	static UE_API const TCHAR* ReadThreadIndexOffsetFunctionName;
};

/** Compute Framework Data Provider for each copy kernel. */
UCLASS()
class UOptimusCopyKernelDataProvider :
	public UComputeDataProvider
{
	GENERATED_BODY()

public:
	void InitFromDataInterface(const UOptimusCopyKernelDataInterface* InDataInterface, const UObject* InBinding);
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

protected:
	TWeakObjectPtr<const UActorComponent> WeakComponent = nullptr;
	TWeakObjectPtr<const UOptimusComponentSource> WeakComponentSource = nullptr;
	TVariant<Optimus::Expression::FExpressionObject, Optimus::Expression::FParseError> ParseResult;
};

class FOptimusCopyKernelDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusCopyKernelDataProviderProxy(
		TArray<int32>&& InInvocationThreadCounts
		);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	int32 GetDispatchThreadCount(TArray<FIntVector>& InOutThreadCounts) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface
	
private:
	TArray<int32> InvocationThreadCounts;
	int32 TotalThreadCount = 0; 
	using FParameters = FCopyKernelDataInterfaceParameters;
};


#undef UE_API
