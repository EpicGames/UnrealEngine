// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceDebugDraw.h"

#include "OptimusCoreModule.h"
#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "OptimusDataTypeRegistry.h"
#include "RenderGraphBuilder.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceDebugDraw)

#define LOCTEXT_NAMESPACE "OptimusDebugDrawDataInterface"

UOptimusDebugDrawDataInterface::UOptimusDebugDrawDataInterface()
{
	if (ShaderPrint::IsSupported(GMaxRHIShaderPlatform))
	{
		bIsSupported = true;
	}
}

FString UOptimusDebugDrawDataInterface::GetDisplayName() const
{
	return TEXT("Debug Draw");
}

FName UOptimusDebugDrawDataInterface::GetCategory() const
{
	return CategoryName::DataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusDebugDrawDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "DebugDraw", "ReadDebugDraw" });
	return Defs;
}

TSubclassOf<UActorComponent> UOptimusDebugDrawDataInterface::GetRequiredComponentClass() const
{
	 return USkinnedMeshComponent::StaticClass();
}

void UOptimusDebugDrawDataInterface::RegisterTypes() 
{
	FOptimusDataTypeRegistry::Get().RegisterType(
		FName("FDebugDraw"),
		FText::FromString(TEXT("FDebugDraw")),
		FShaderValueType::Get(FName("FDebugDraw"), { FShaderValueType::FStructElement(FName("LocalToWorld"), FShaderValueType::Get(EShaderFundamentalType::Float, 4, 4)) }),
		FName("FDebugDraw"),
		nullptr,
		FLinearColor(0.3f, 0.7f, 0.4f, 1.0f),
		EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);
}

TOptional<FText> UOptimusDebugDrawDataInterface::ValidateForCompile() const
{
	if (!bIsSupported)
	{
		return LOCTEXT("Unsupported Platform Setting", "Deformer Graph contains unsupported DebugDraw Data Interface and may not "
						"function properly. Either make sure DirectX 12 and Shader Model 6 are enabled in Project Settings, "
						"or remove the data interface from your graph completely");
	}

	return {};
}

void UOptimusDebugDrawDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	FShaderValueTypeHandle DebugDrawType = FOptimusDataTypeRegistry::Get().FindType(FName("FDebugDraw"))->ShaderValueType;

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDebugDraw"))
		.AddReturnType(DebugDrawType);
}

BEGIN_SHADER_PARAMETER_STRUCT(FDebugDrawDataInterfaceParameters, )
	SHADER_PARAMETER(FMatrix44f, LocalToWorld)
	SHADER_PARAMETER(FIntPoint, Resolution)
	SHADER_PARAMETER(FVector2f, FontSize)
	SHADER_PARAMETER(FVector2f, FontSpacing)
	SHADER_PARAMETER(uint32, MaxCharacterCount)
	SHADER_PARAMETER(uint32, MaxSymbolCount)
	SHADER_PARAMETER(uint32, MaxStateCount)
	SHADER_PARAMETER(uint32, MaxLineCount)
	SHADER_PARAMETER(uint32, MaxTriangleCount)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, StateBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWEntryBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusDebugDrawDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FDebugDrawDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusDebugDrawDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceDebugDraw.ush");

TCHAR const* UOptimusDebugDrawDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusDebugDrawDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusDebugDrawDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusDebugDrawDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusDebugDrawDataProvider* Provider = NewObject<UOptimusDebugDrawDataProvider>();
	Provider->PrimitiveComponent = Cast<UPrimitiveComponent>(InBinding);
	Provider->DebugDrawParameters = DebugDrawParameters;

	if (!bIsSupported)
	{
		UE_LOGF(LogOptimusCore, Error, "Deformer Graph aborted because it relies on unsupported DebugDraw DataInterface. Either remove the data interface or make sure DirectX 12 and Shader Model 6 is enabled in project settings for DebugDraw Data Interface to function, since DXC is required for shader compilation.");
	}
	
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusDebugDrawDataProvider::GetRenderProxy()
{
	return new FOptimusDebugDrawDataProviderProxy(PrimitiveComponent.Get(), DebugDrawParameters);
}


FOptimusDebugDrawDataProviderProxy::FOptimusDebugDrawDataProviderProxy(UPrimitiveComponent* InPrimitiveComponent, FOptimusDebugDrawParameters const& InDebugDrawParameters)
	: Setup(FIntRect(0, 0, 1920, 1080))
{
	Scene = InPrimitiveComponent != nullptr ? InPrimitiveComponent->GetScene() : nullptr;

	// Split LocalToWorld into a pre-translation and transform for large world coordinate support.
	FMatrix RenderMatrix = InPrimitiveComponent != nullptr ? InPrimitiveComponent->GetRenderMatrix() : FMatrix();
	FVector PreViewTranslation = -RenderMatrix.GetOrigin();
	LocalToWorld = FMatrix44f(RenderMatrix.ConcatTranslation(PreViewTranslation));

	Setup.FontSize = InDebugDrawParameters.FontSize;
	Setup.MaxLineCount = Setup.bEnabled ? InDebugDrawParameters.MaxLineCount : 0;
	Setup.MaxTriangleCount = Setup.bEnabled ? InDebugDrawParameters.MaxTriangleCount : 0;
	Setup.MaxCharacterCount = Setup.bEnabled ? InDebugDrawParameters.MaxCharacterCount : 0;
	Setup.PreViewTranslation = PreViewTranslation;

	ShaderPrint::GetParameters(Setup, ConfigParameters);

	// Force ShaderPrint system enable if requested.
	if (InDebugDrawParameters.bForceEnable)
	{
		ShaderPrint::SetEnabled(true);
	}
}

bool FOptimusDebugDrawDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (Scene == nullptr)
	{
		return false;
	}

	if (!ShaderPrint::IsSupported(GMaxRHIShaderPlatform))
	{
		return false;
	}
	
	return true;
}

void FOptimusDebugDrawDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	// Allocate ShaderPrint output buffers.
	FShaderPrintData ShaderPrintData = ShaderPrint::CreateShaderPrintData(GraphBuilder, Setup);
	
	// Cache parameters for later GatherDispatchData(). 
	// We only want the buffers that are created here. 
	// We don't use the ShaderPrint uniform buffer because we'll use the loose ConfigParameters instead.
	// That's so we can bind multiple instances of this data interface if we want.
	ShaderPrint::SetParameters(GraphBuilder, ShaderPrintData, CachedParameters);

	if (ShaderPrint::IsEnabled(ShaderPrintData))
	{
		// Enqueue for display at next view render.
		FFrozenShaderPrintData FrozenShaderPrintData = ShaderPrint::FreezeShaderPrintData(GraphBuilder, ShaderPrintData);
		ShaderPrint::SubmitShaderPrintData(FrozenShaderPrintData, Scene);
	}
}

void FOptimusDebugDrawDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.LocalToWorld = LocalToWorld;
		Parameters.Resolution = ConfigParameters.Resolution;
		Parameters.FontSize = ConfigParameters.FontSize;
		Parameters.FontSpacing = ConfigParameters.FontSpacing;
		Parameters.MaxCharacterCount = ConfigParameters.MaxCharacterCount;
		Parameters.MaxSymbolCount = ConfigParameters.MaxSymbolCount;
		Parameters.MaxStateCount = ConfigParameters.MaxStateCount;
		Parameters.MaxLineCount = ConfigParameters.MaxLineCount;
		Parameters.MaxTriangleCount = ConfigParameters.MaxTriangleCount;
		Parameters.StateBuffer = CachedParameters.ShaderPrint_StateBuffer;
		Parameters.RWEntryBuffer = CachedParameters.ShaderPrint_RWEntryBuffer;
	}
}

#undef LOCTEXT_NAMESPACE