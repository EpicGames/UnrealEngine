// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ShaderPrintParameters.h"

#include "OptimusDataInterfaceDebugDraw.generated.h"

#define UE_API OPTIMUSCORE_API

class FDebugDrawDataInterfaceParameters;
class UPrimitiveComponent;

/* User controllable debug draw settings. */
USTRUCT()
struct FOptimusDebugDrawParameters
{
	GENERATED_BODY()

	/** 
	 * Force enable debug rendering. 
	 * Otherwise "r.ShaderPrint 1" needs to be set.
	 */
	UPROPERTY(EditAnywhere, Category = DebugDraw)
	bool bForceEnable = false;
	/** Space to allocate for line collection. */
	UPROPERTY(EditAnywhere, Category = DebugDraw)
	int32 MaxLineCount = 10000;
	/** Space to allocate for triangle collection. */
	UPROPERTY(EditAnywhere, Category = DebugDraw)
	int32 MaxTriangleCount = 2000;
	/** Space to allocate for character collection. */
	UPROPERTY(EditAnywhere, Category = DebugDraw)
	int32 MaxCharacterCount = 2000;
	/** Font size for characters. */
	UPROPERTY(EditAnywhere, Category = DebugDraw)
	int32 FontSize = 8;
};

/** Compute Framework Data Interface for writing skinned mesh. */

/* Debug Draw Data Interface provides access to a set of debug drawing hlsl functions */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusDebugDrawDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	UE_API UOptimusDebugDrawDataInterface();
	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API FName GetCategory() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	UE_API void RegisterTypes() override;
	UE_API TOptional<FText> ValidateForCompile() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("DebugDraw"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	// Make sure DirectX12 and Shader Model 6 is enabled in project settings for DebugDraw to function, since DXC is required for shader compilation. 
	UPROPERTY(Transient, VisibleAnywhere, Category = DebugDraw, meta=(EditConditionHides, EditCondition="!bIsSupported"))
	bool bIsSupported = false;
	
	UPROPERTY(EditAnywhere, Category = DebugDraw, meta = (ShowOnlyInnerProperties))
	FOptimusDebugDrawParameters DebugDrawParameters;

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for writing skinned mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusDebugDrawDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UPrimitiveComponent> PrimitiveComponent = nullptr;

	FOptimusDebugDrawParameters DebugDrawParameters;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusDebugDrawDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusDebugDrawDataProviderProxy(UPrimitiveComponent* InPrimitiveComponent, FOptimusDebugDrawParameters const& InDebugDrawParameters);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FDebugDrawDataInterfaceParameters;

	FSceneInterface const* Scene;
	FMatrix44f LocalToWorld;
	ShaderPrint::FShaderPrintSetup Setup;
	ShaderPrint::FShaderPrintCommonParameters ConfigParameters;
	ShaderPrint::FShaderParameters CachedParameters;
};

#undef UE_API
