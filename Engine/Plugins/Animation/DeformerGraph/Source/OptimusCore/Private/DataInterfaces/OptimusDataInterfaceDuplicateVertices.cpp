// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceDuplicateVertices.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "OptimusDataDomain.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceDuplicateVertices)


FString UOptimusDuplicateVerticesDataInterface::GetDisplayName() const
{
	return TEXT("Duplicate Vertices");
}

TArray<FOptimusCDIPinDefinition> UOptimusDuplicateVerticesDataInterface::GetPinDefinitions() const
{
	FName Vertex(UOptimusSkinnedMeshComponentSource::Domains::Vertex);
	FName Triangle(UOptimusSkinnedMeshComponentSource::Domains::Triangle);
	FName DuplicateVertex(UOptimusSkinnedMeshComponentSource::Domains::DuplicateVertex);
	
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumVertices", "ReadNumVertices"});
	Defs.Add({"DuplicateVerticesStartAndLength", "ReadDuplicateVerticesStartAndLength", Vertex, "ReadDuplicateVerticesStartAndLength"});
	Defs.Add({"DuplicateVertex", "ReadDuplicateVertex", DuplicateVertex, "ReadDuplicateVertex"});
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusDuplicateVerticesDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusDuplicateVerticesDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumDuplicateVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDuplicateVerticesStartAndLength"))
		.AddReturnType(EShaderFundamentalType::Int, 2)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDuplicateVertex"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FDuplicateVerticesDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumDuplicateVertices)
	SHADER_PARAMETER(uint32, BaseVertexIndex)
	SHADER_PARAMETER_SRV(Buffer<uint>, DuplicateVertexStartAndLength)
	SHADER_PARAMETER_SRV(Buffer<uint>, DuplicateVertices)
END_SHADER_PARAMETER_STRUCT()

void UOptimusDuplicateVerticesDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FDuplicateVerticesDataInterfaceParameters>(UID);
}

void UOptimusDuplicateVerticesDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DUPLICATED_VERTICES"), 2);
}

TCHAR const* UOptimusDuplicateVerticesDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceDuplicateVertices.ush");

TCHAR const* UOptimusDuplicateVerticesDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusDuplicateVerticesDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusDuplicateVerticesDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusDuplicateVerticesDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusDuplicateVerticesDataProvider* Provider = NewObject<UOptimusDuplicateVerticesDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusDuplicateVerticesDataProvider::GetRenderProxy()
{
	return new FOptimusDuplicateVerticesDataProviderProxy(SkinnedMesh.Get());
}


FOptimusDuplicateVerticesDataProviderProxy::FOptimusDuplicateVerticesDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent != nullptr && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
	}
}

bool FOptimusDuplicateVerticesDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[SkeletalMeshObject->GetLOD()].RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}

	return true;
}

struct FDuplicateVerticesDataInterfacePermutationIds
{
	uint32 EnableDuplicateVertices = 0;

	FDuplicateVerticesDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DUPLICATED_VERTICES"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDuplicateVertices = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusDuplicateVerticesDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FDuplicateVerticesDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		FRHIShaderResourceView* DuplicateVertexStartAndLengthSRV = RenderSection.DuplicatedVerticesBuffer.LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferSRV;
		FRHIShaderResourceView* DuplicateVerticesSRV = RenderSection.DuplicatedVerticesBuffer.DuplicatedVerticesIndexBuffer.VertexBufferSRV;
		const bool bValidDuplicateIndices = (DuplicateVertexStartAndLengthSRV != nullptr) && (DuplicateVerticesSRV != nullptr);

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidDuplicateIndices ? PermutationIds.EnableDuplicateVertices : 0);
	}
}

void FOptimusDuplicateVerticesDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		FRHIShaderResourceView* DuplicateVertexStartAndLengthSRV = RenderSection.DuplicatedVerticesBuffer.LengthAndIndexDuplicatedVerticesIndexBuffer.VertexBufferSRV;
		FRHIShaderResourceView* DuplicateVerticesSRV = RenderSection.DuplicatedVerticesBuffer.DuplicatedVerticesIndexBuffer.VertexBufferSRV;
		const bool bValidDuplicateIndices = (DuplicateVertexStartAndLengthSRV != nullptr) && (DuplicateVerticesSRV != nullptr);

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumDuplicateVertices = bValidDuplicateIndices ? RenderSection.DuplicatedVerticesBuffer.DupVertData.Num() : 0; 
		Parameters.BaseVertexIndex = RenderSection.BaseVertexIndex;
		Parameters.DuplicateVertexStartAndLength = bValidDuplicateIndices ? DuplicateVertexStartAndLengthSRV : NullSRVBinding;
		Parameters.DuplicateVertices = bValidDuplicateIndices ? DuplicateVerticesSRV : NullSRVBinding;
	}
}
