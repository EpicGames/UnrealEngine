// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceGraph.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"
#include "ComputeFramework/ShaderParameterMetadataAllocation.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDeformerInstance.h"
#include "OptimusHelpers.h"
#include "OptimusVariableDescription.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceGraph)

void UOptimusGraphDataInterface::Init(TArray<FOptimusGraphVariableDescription> const& InVariables)
{
	Variables = InVariables;

	FShaderParametersMetadataBuilder Builder;
	TArray<FShaderParametersMetadata*> AllocatedStructMetadatas;
	TArray<FShaderParametersMetadata*> NestedStructMetadatas; 
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		ComputeFramework::AddParamForType(Builder, *Variable.Name, Variable.ValueType, NestedStructMetadatas);
	}
	
	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("UGraphDataInterface"));
	AllocatedStructMetadatas.Add(ShaderParameterMetadata);
	AllocatedStructMetadatas.Append(NestedStructMetadatas);
	

	TArray<FShaderParametersMetadata::FMember> const& Members = ShaderParameterMetadata->GetMembers();
	for (int32 VariableIndex = 0; VariableIndex < Variables.Num(); ++VariableIndex)
	{
		check(Variables[VariableIndex].Name == Members[VariableIndex].GetName());
		Variables[VariableIndex].Offset = Members[VariableIndex].GetOffset();
	}

	ParameterBufferSize = ShaderParameterMetadata->GetSize();

	for (const FShaderParametersMetadata* AllocatedData : AllocatedStructMetadatas)
	{
		delete AllocatedData;
	}
}

int32 UOptimusGraphDataInterface::FindFunctionIndex(const FOptimusValueIdentifier& InValueId) const
{
	for (int32 Index = 0 ; Index < Variables.Num(); Index++)
	{
		if (Variables[Index].ValueId == InValueId)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}



void UOptimusGraphDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.Reserve(OutFunctions.Num() + Variables.Num());
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		OutFunctions.AddDefaulted_GetRef()
			.SetName(FString::Printf(TEXT("Read%s"), *Variable.Name))
			.AddReturnType(Variable.ValueType);
	}
}

void UOptimusGraphDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	// Build metadata nested structure containing all variables.
	FShaderParametersMetadataBuilder Builder;
	TArray<FShaderParametersMetadata*> NestedStructs;
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		ComputeFramework::AddParamForType(Builder, *Variable.Name, Variable.ValueType, NestedStructs);
	}

	FShaderParametersMetadata* ShaderParameterMetadata = Builder.Build(FShaderParametersMetadata::EUseCase::ShaderParameterStruct, TEXT("UGraphDataInterface"));
	// Add the metadata to InOutAllocations so that it is released when we are done.
	InOutAllocations.ShaderParameterMetadatas.Add(ShaderParameterMetadata);
	InOutAllocations.ShaderParameterMetadatas.Append(NestedStructs);

	// Add the generated nested struct to our builder.
	InOutBuilder.AddNestedStruct(UID, ShaderParameterMetadata);
}

void UOptimusGraphDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	// Add uniforms.
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		OutHLSL += FString::Printf(TEXT("%s %s_%s;\n"), *Variable.ValueType->ToString(), *InDataInterfaceName, *Variable.Name);
	}
	// Add function getters.
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		OutHLSL += FString::Printf(TEXT("%s Read%s_%s()\n{\n\treturn %s_%s;\n}\n"), *Variable.ValueType->ToString(), *Variable.Name, *InDataInterfaceName, *InDataInterfaceName, *Variable.Name);
	}
}

UComputeDataProvider* UOptimusGraphDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusGraphDataProvider* Provider = NewObject<UOptimusGraphDataProvider>();
	Provider->Init(Variables, ParameterBufferSize);

	return Provider;
}

void UOptimusGraphDataInterface::PostLoad()
{
	Super::PostLoad();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (FOptimusGraphVariableDescription& VariableDescription : Variables)
	{
		if (!VariableDescription.Value_DEPRECATED.IsEmpty())
		{
			FOptimusDataTypeHandle DataType = FOptimusDataTypeRegistry::Get().FindType(VariableDescription.ValueType);
			VariableDescription.ShaderValue_DEPRECATED = DataType->MakeShaderValue();
			check(VariableDescription.ShaderValue_DEPRECATED.ArrayList.Num() == 0);
			VariableDescription.ShaderValue_DEPRECATED.ShaderValue = VariableDescription.Value_DEPRECATED;
			VariableDescription.Value_DEPRECATED.Reset();
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


void UOptimusGraphDataProvider::Init(const TArray<FOptimusGraphVariableDescription>& InVariables, int32 InParameterBufferSize)
{
	Variables = InVariables;
	ParameterBufferSize = InParameterBufferSize;

	int32 TotalNumArrays = 0;
	for (FOptimusGraphVariableDescription& Variable : Variables)
	{
		const TArray<FOptimusDataTypeRegistry::FArrayMetadata>& TypeArrayMetadata = FOptimusDataTypeRegistry::Get().FindArrayMetadata(FOptimusDataTypeRegistry::Get().FindType(Variable.ValueType)->TypeName);

		Variable.CachedArrayIndexStart = TotalNumArrays;
		TotalNumArrays += TypeArrayMetadata.Num();
	}
	
	ParameterArrayMetadata.AddDefaulted(TotalNumArrays);
	
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		const TArray<FOptimusDataTypeRegistry::FArrayMetadata>& TypeArrayMetadata = FOptimusDataTypeRegistry::Get().FindArrayMetadata(FOptimusDataTypeRegistry::Get().FindType(Variable.ValueType)->TypeName);

		for (int32 ArrayIndex = 0; ArrayIndex < TypeArrayMetadata.Num(); ArrayIndex++)
		{
			const int32 TopLevelArrayIndex = Variable.CachedArrayIndexStart + ArrayIndex;
			if (ensure(ParameterArrayMetadata.IsValidIndex(TopLevelArrayIndex)))
			{
				ParameterArrayMetadata[TopLevelArrayIndex].Offset = Variable.Offset + TypeArrayMetadata[ArrayIndex].ShaderValueOffset;
				ParameterArrayMetadata[TopLevelArrayIndex].ElementSize = TypeArrayMetadata[ArrayIndex].ElementShaderValueSize;
			}
		}
	}
}

FComputeDataProviderRenderProxy* UOptimusGraphDataProvider::GetRenderProxy()
{
	return new FOptimusGraphDataProviderProxy(WeakDeformerInstance.Get(), Variables, ParameterBufferSize, ParameterArrayMetadata);
}

void UOptimusGraphDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}

FOptimusGraphDataProviderProxy::FOptimusGraphDataProviderProxy(
	UOptimusDeformerInstance const* DeformerInstance,
	TArray<FOptimusGraphVariableDescription> const& Variables,
	int32 ParameterBufferSize,
	TArray<UOptimusGraphDataProvider::FArrayMetadata> const& InParameterArrayMetadata)
{
	// Get all variables from deformer instance and fill buffer.
	ParameterData.AddZeroed(ParameterBufferSize);

	ParameterArrayMetadata = InParameterArrayMetadata;
	ParameterArrayData.AddDefaulted(ParameterArrayMetadata.Num());
	
	if (DeformerInstance == nullptr)
	{
		return;
	}

	auto CopyVariableToBuffer = [this](int32 InOffset, int32 InArrayIndexStart, const FShaderValueContainer& InShaderValue)
	{
    	if (ensure(ParameterData.Num() >= InOffset + InShaderValue.ShaderValue.Num()))
    	{
    		FMemory::Memcpy(&ParameterData[InOffset], InShaderValue.ShaderValue.GetData(), InShaderValue.ShaderValue.Num());
    	
    		for (int32 ArrayIndex = 0; ArrayIndex < InShaderValue.ArrayList.Num(); ArrayIndex++)
    		{
    			const int32 ToplevelArrayIndex = InArrayIndexStart+ ArrayIndex;
    			if (ensure(ParameterArrayData.IsValidIndex(ToplevelArrayIndex)))
    			{
    				ParameterArrayData[ToplevelArrayIndex] = InShaderValue.ArrayList[ArrayIndex];
    			}
    		}
    	}	
	};

	
	for (FOptimusGraphVariableDescription const& Variable : Variables)
	{
		const FShaderValueContainer& ShaderValue = DeformerInstance->GetShaderValue(Variable.ValueId);

		CopyVariableToBuffer(Variable.Offset, Variable.CachedArrayIndexStart, ShaderValue);
	}
}

bool FOptimusGraphDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (ParameterData.Num() == 0)
	{
		// todo[CF]: Why can we end up here? Remove this condition if possible.
		return false;
	}

	if (InValidationData.ParameterStructSize != ParameterData.Num())
	{
		return false;
	}

	return true;
}

void FOptimusGraphDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	ParameterArrayBuffers.Reset();
	ParameterArrayBufferSRVs.Reset();


	for (int32 ArrayIndex = 0; ArrayIndex < ParameterArrayMetadata.Num(); ArrayIndex++)
	{
		const UOptimusGraphDataProvider::FArrayMetadata& ArrayMetadata = ParameterArrayMetadata[ArrayIndex];
		const TArray<uint8>& ArrayData = ParameterArrayData[ArrayIndex].ArrayOfValues;
		
		ParameterArrayBuffers.Add(
			GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(
					ArrayMetadata.ElementSize,
					FMath::Max(ArrayData.Num() / ArrayMetadata.ElementSize, 1)), TEXT("Optimus.GraphDataInterfaceInnerBuffer")));
		ParameterArrayBufferSRVs.Add(GraphBuilder.CreateSRV(ParameterArrayBuffers.Last()));
		GraphBuilder.QueueBufferUpload(ParameterArrayBuffers.Last(), ArrayData.GetData(), ArrayData.Num(), ERDGInitialDataFlags::None);	
	}
}

void FOptimusGraphDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		uint8* ParameterBuffer = (uint8*)(InDispatchData.ParameterBuffer + InDispatchData.ParameterBufferOffset + InDispatchData.ParameterBufferStride * InvocationIndex);
		FMemory::Memcpy(ParameterBuffer, ParameterData.GetData(), ParameterData.Num());

		for (int32 ArrayIndex = 0; ArrayIndex < ParameterArrayMetadata.Num(); ArrayIndex++)
		{
			const UOptimusGraphDataProvider::FArrayMetadata& ArrayMetadata = ParameterArrayMetadata[ArrayIndex];
			*((FRDGBufferSRV**)(ParameterBuffer + ArrayMetadata.Offset)) = ParameterArrayBufferSRVs[ArrayIndex];
		}
	}
}
