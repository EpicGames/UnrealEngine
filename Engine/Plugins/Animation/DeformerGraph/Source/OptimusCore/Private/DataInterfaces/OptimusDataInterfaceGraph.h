// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusValue.h"
#include "ComputeFramework/ComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"


#include "OptimusDataInterfaceGraph.generated.h"

class UOptimusDeformerInstance;
class UMeshComponent;
class FRDGBuffer;
class FRDGBufferSRV;

/** */
USTRUCT()
struct FOptimusGraphVariableDescription
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString	Name;

	UPROPERTY()
	FShaderValueTypeHandle ValueType;

	UPROPERTY()
	FOptimusValueIdentifier ValueId;

	UPROPERTY()
	int32 Offset = 0;

	// Cache below are set by the data provider, computed from serialized data
	int32 CachedArrayIndexStart = 0;

	
	// Deprecated 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ShaderValue instead"))
	TArray<uint8> Value_DEPRECATED;
	
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Values are now store on the deformer instance"))
	FShaderValueContainer ShaderValue_DEPRECATED;
	
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Node-Value mapping is now handled by deformer instance directly"))
	TSoftObjectPtr<UObject> SourceObject_DEPRECATED;
};

/** Compute Framework Data Interface used for marshaling compute graph parameters and variables. */
UCLASS(Category = ComputeFramework)
class UOptimusGraphDataInterface : public UComputeDataInterface
{
	GENERATED_BODY()

public:
	void Init(TArray<FOptimusGraphVariableDescription> const& InVariables);
	int32 FindFunctionIndex(const FOptimusValueIdentifier& InValueId) const;

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("Graph"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	void PostLoad() override;
private:
	
	// For PostLoad fixup
	friend class UOptimusDeformer;
	
	UPROPERTY()
	TArray<FOptimusGraphVariableDescription> Variables;

	UPROPERTY()
	int32 ParameterBufferSize = 0;
};

/** Compute Framework Data Provider for marshaling compute graph parameters and variables. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusGraphDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TArray<FOptimusGraphVariableDescription> Variables;

	int32 ParameterBufferSize = 0;


	struct FArrayMetadata
	{
		int32 Offset;
		int32 ElementSize;
	};
	
	TArray<FArrayMetadata> ParameterArrayMetadata;
	
	
	void Init(const TArray<FOptimusGraphVariableDescription>& InVariables, int32 InParameterBufferSize);
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin IOptimusDeformerInstanceAccessor Interface
	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;
	//~ End IOptimusDeformerInstanceAccessor Interface

private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance = nullptr;
};

class FOptimusGraphDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusGraphDataProviderProxy(
		UOptimusDeformerInstance const* DeformerInstance,
		TArray<FOptimusGraphVariableDescription> const& Variables, 
		int32 ParameterBufferSize,
		TArray<UOptimusGraphDataProvider::FArrayMetadata> const& InParameterArrayMetadata);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	TArray<uint8> ParameterData;

	TArray<UOptimusGraphDataProvider::FArrayMetadata> ParameterArrayMetadata;
	TArray<FArrayShaderValue> ParameterArrayData;

	TArray<FRDGBuffer*> ParameterArrayBuffers;
	TArray<FRDGBufferSRV*> ParameterArrayBufferSRVs;
};
