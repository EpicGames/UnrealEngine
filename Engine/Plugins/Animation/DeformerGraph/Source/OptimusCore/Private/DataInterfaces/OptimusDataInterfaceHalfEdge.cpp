// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceHalfEdge.h"

#include "Components/SkinnedMeshComponent.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "OptimusSettings.h"
#include "OptimusCoreModule.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SystemTextures.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceHalfEdge)
/*
 * Render resource containing the half edge buffers. 
 */ 
class FHalfEdgeBuffers
{
public:
	FHalfEdgeBuffers() = default;

	void Init(FRDGBuilder& GraphBuilder, const FSkeletalMeshLODRenderData& InLodRenderData)
	{
		TResourceArray<int32> VertexToEdgeData;
		TResourceArray<int32> EdgeToTwinEdgeData;

		SkeletalMeshHalfEdgeUtility::BuildHalfEdgeBuffers(InLodRenderData, VertexToEdgeData, EdgeToTwinEdgeData);	
		
		FRDGBuffer* VertexToEdgeBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), VertexToEdgeData.Num()), TEXT("Optimus.VertexToEdge"));
		GraphBuilder.QueueBufferUpload(VertexToEdgeBuffer, VertexToEdgeData.GetData(), VertexToEdgeData.Num() * sizeof(int32));
		VertexToEdgePooledBuffer = GraphBuilder.ConvertToExternalBuffer(VertexToEdgeBuffer);
			
		FRDGBuffer* EdgeToTwinEdgeBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), EdgeToTwinEdgeData.Num()), TEXT("Optimus.EdgeToTwinEdge"));
		GraphBuilder.QueueBufferUpload(EdgeToTwinEdgeBuffer, EdgeToTwinEdgeData.GetData(), EdgeToTwinEdgeData.Num() * sizeof(int32));
		EdgeToTwinEdgePooledBuffer = GraphBuilder.ConvertToExternalBuffer(EdgeToTwinEdgeBuffer);

		bInitialized = true;
	}

	bool IsInitialized() const { return bInitialized; }
	
	~FHalfEdgeBuffers()
	{
		VertexToEdgePooledBuffer.SafeRelease();
		EdgeToTwinEdgePooledBuffer.SafeRelease();
	}

	TRefCountPtr<FRDGPooledBuffer> GetVertexToEdgeBuffer()
	{
		return VertexToEdgePooledBuffer;
	}

	TRefCountPtr<FRDGPooledBuffer> GetEdgeToTwinBuffer()
	{
		return EdgeToTwinEdgePooledBuffer;
	}

private:
	bool bInitialized = false;
	TRefCountPtr<FRDGPooledBuffer> VertexToEdgePooledBuffer;
	TRefCountPtr<FRDGPooledBuffer> EdgeToTwinEdgePooledBuffer;
};


FString UOptimusHalfEdgeDataInterface::GetDisplayName() const
{
	return TEXT("HalfEdge");
}

TArray<FOptimusCDIPinDefinition> UOptimusHalfEdgeDataInterface::GetPinDefinitions() const
{
	FName Vertex(UOptimusSkinnedMeshComponentSource::Domains::Vertex);
	FName Triangle(UOptimusSkinnedMeshComponentSource::Domains::Triangle);

	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "EdgesPerVertex", "ReadEdge", Vertex, "ReadNumVertices", false});
	Defs.Add({ "TwinEdges", "ReadTwinEdge", Triangle, 3, "ReadNumTriangles", false});
	return Defs;
}

TSubclassOf<UActorComponent> UOptimusHalfEdgeDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}

void UOptimusHalfEdgeDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumTriangles"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadEdge"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTwinEdge"))
		.AddReturnType(EShaderFundamentalType::Int)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FHalfEdgeDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumTriangles)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, VertexToEdgeBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, EdgeToTwinEdgeBuffer)

	SHADER_PARAMETER(uint32, bUseBufferFromRenderData)
	SHADER_PARAMETER_SRV(StructuredBuffer<int>, RenderDataVertexToEdgeBuffer)
	SHADER_PARAMETER_SRV(StructuredBuffer<int>, RenderDataEdgeToTwinEdgeBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusHalfEdgeDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FHalfEdgeDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusHalfEdgeDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceHalfEdge.ush");

TCHAR const* UOptimusHalfEdgeDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusHalfEdgeDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusHalfEdgeDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusHalfEdgeDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusHalfEdgeDataProvider* Provider = NewObject<UOptimusHalfEdgeDataProvider>();

	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);

	if (Provider->SkinnedMesh != nullptr)
	{
		FSkeletalMeshRenderData const* SkeletalMeshRenderData = Provider->SkinnedMesh->GetSkeletalMeshRenderData();
		if (SkeletalMeshRenderData != nullptr)
		{
			// Items in this array are initialized on demand by the render proxy
			Provider->OnDemandHalfEdgeBuffers.SetNum(SkeletalMeshRenderData->LODRenderData.Num());
		}
	}

	return Provider;
}


void UOptimusHalfEdgeDataProvider::BeginDestroy()
{
	Super::BeginDestroy();
	DestroyFence.BeginFence();
}

bool UOptimusHalfEdgeDataProvider::IsReadyForFinishDestroy()
{
	if (Super::IsReadyForFinishDestroy() && DestroyFence.IsFenceComplete())
	{
		OnDemandHalfEdgeBuffers.Reset();
		return true;
	}
	return false;
}

FComputeDataProviderRenderProxy* UOptimusHalfEdgeDataProvider::GetRenderProxy()
{
#if WITH_EDITOR
	if (Optimus::IsAssetValidationEnabled())
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = TWeakObjectPtr(this)]()
			{
				if (UOptimusHalfEdgeDataProvider* DataProvider = Cast<UOptimusHalfEdgeDataProvider>(WeakThis))
				{
					DataProvider->ValidateSkinnedAsset();
				}
			});
	}
#endif
		
	return new FOptimusHalfEdgeDataProviderProxy(SkinnedMesh.Get(), OnDemandHalfEdgeBuffers);
}

void UOptimusHalfEdgeDataProvider::SetOptimusDeformerAssetPath(const FTopLevelAssetPath& InPath)
{
#if WITH_EDITOR
	DeformerAssetPath = InPath;
#endif
}

#if WITH_EDITOR
void UOptimusHalfEdgeDataProvider::ValidateSkinnedAsset()
{
	check(IsInGameThread());

	if (!SkinnedMesh.IsValid())
	{
		return;
	}
	
	if (USkinnedAsset* Asset = SkinnedMesh->GetSkinnedAsset())
	{
		if (!ValidatedAssets.Contains(Asset))
		{
			ValidatedAssets.Add(Asset);
					
			if (FSkeletalMeshObject* MeshObject = SkinnedMesh->MeshObject)
			{
				if (!Asset->HasHalfEdgeBuffer(MeshObject->GetLOD()))
				{
					UE_LOGF(LogOptimusCore, Warning,
						   "Please add Deformer Graph %ls to TargetMeshDeformers of Skeletal Mesh Asset %ls to "
							   "make sure extra data required by the Deformer is cooked for non-editor builds"
							   "This warning can be turned off with %ls 0",
						   *DeformerAssetPath.GetPackageName().ToString(),
						   *Asset->GetPackage()->GetName(),
						   *Optimus::GetEnableAssetValidationCVarName().ToString()
						   );
				}
			}
		}
	}		
}

#endif

FOptimusHalfEdgeDataProviderProxy::FOptimusHalfEdgeDataProviderProxy(
	USkinnedMeshComponent* InSkinnedMeshComponent, 
	TArray<FHalfEdgeBuffers>& InOnDemandHalfEdgeBuffers)
	: OnDemandHalfEdgeBuffers(InOnDemandHalfEdgeBuffers)
{
	if (InSkinnedMeshComponent && InSkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = InSkinnedMeshComponent->GetScene();
		ComponentId = InSkinnedMeshComponent->GetPrimitiveSceneId();
	}
}

bool FOptimusHalfEdgeDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	if (!ensure(SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData.Num() == OnDemandHalfEdgeBuffers.Num()))
	{
		return false;
	}

	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[SkeletalMeshObject->GetLOD()];	
	
	// Invalid if there is no cooked buffer and run time generation isn't possible either
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer = LodRenderData->MultiSizeIndexContainer.GetIndexBuffer();
	const FPositionVertexBuffer& VertexBuffer = LodRenderData->StaticVertexBuffers.PositionVertexBuffer;
	if (!LodRenderData->HalfEdgeBuffer.IsReadyForRendering())
	{
		if (!IndexBuffer->GetNeedsCPUAccess() || !VertexBuffer.GetAllowCPUAccess())
		{
			return false;
		}
	}

	return true;
}

void FOptimusHalfEdgeDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	FallbackSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(int32))));
	
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	if (LodRenderData->HalfEdgeBuffer.IsReadyForRendering())
	{
		bUseBufferFromRenderData = true;

		VertexToEdgeBufferSRV = FallbackSRV;
		EdgeToTwinEdgeBufferSRV = FallbackSRV;
	}
	else
	{
		bUseBufferFromRenderData = false;
		
		if (!OnDemandHalfEdgeBuffers[LodIndex].IsInitialized())
		{
			// fall back to compute half edge on demand
			OnDemandHalfEdgeBuffers[LodIndex].Init(GraphBuilder, *LodRenderData);
		}

		FRDGBuffer* VertexToEdgeBuffer = GraphBuilder.RegisterExternalBuffer(OnDemandHalfEdgeBuffers[LodIndex].GetVertexToEdgeBuffer());
		VertexToEdgeBufferSRV = GraphBuilder.CreateSRV(VertexToEdgeBuffer);
	
		FRDGBuffer* EdgeToTwinEdgeBuffer = GraphBuilder.RegisterExternalBuffer(OnDemandHalfEdgeBuffers[LodIndex].GetEdgeToTwinBuffer());
		EdgeToTwinEdgeBufferSRV = GraphBuilder.CreateSRV(EdgeToTwinEdgeBuffer);	
	}
}

void FOptimusHalfEdgeDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* RenderDataVertexToEdgeBufferSRV = LodRenderData->HalfEdgeBuffer.GetVertexToEdgeBufferSRV();
	FRHIShaderResourceView* RenderDataEdgeToTwinEdgeBufferSRV = LodRenderData->HalfEdgeBuffer.GetEdgeToTwinEdgeBufferSRV();
	
	FRHIShaderResourceView* NullSRVBinding = GEmptyStructuredBufferWithUAV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumTriangles = LodRenderData->GetTotalFaces();
		Parameters.VertexToEdgeBuffer = VertexToEdgeBufferSRV;
		Parameters.EdgeToTwinEdgeBuffer = EdgeToTwinEdgeBufferSRV;

		Parameters.bUseBufferFromRenderData = bUseBufferFromRenderData;
		Parameters.RenderDataVertexToEdgeBuffer = RenderDataVertexToEdgeBufferSRV != nullptr ? RenderDataVertexToEdgeBufferSRV : NullSRVBinding;
		Parameters.RenderDataEdgeToTwinEdgeBuffer = RenderDataEdgeToTwinEdgeBufferSRV != nullptr ? RenderDataEdgeToTwinEdgeBufferSRV : NullSRVBinding;
	}
}
