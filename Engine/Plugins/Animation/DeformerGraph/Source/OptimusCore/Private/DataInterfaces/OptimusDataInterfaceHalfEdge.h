// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerAssetPathAccessor.h"
#include "RenderGraphDefinitions.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "RenderCommandFence.h"
#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceHalfEdge.generated.h"

#define UE_API OPTIMUSCORE_API

class FHalfEdgeDataInterfaceParameters;
class FHalfEdgeBuffers;
class FRDGBuffer;
class FRDGBufferSRV;
class FSceneInterface;
class FSkeletalMeshObject;
class USkinnedMeshComponent;
class USkinnedAsset;


/** Compute Framework Data Interface for reading mesh half edge data. */
/** Half Edge Data Interface provides vertex connectivity info, even across material sections, see ComputeNormalsTangents deformer function for example usage.
 *  Skeletal mesh assets using this data interface should have "BuildHalfEdgeBuffers" turned on under LODInfo settings
 */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusHalfEdgeDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("HalfEdge"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusHalfEdgeDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerAssetPathAccessor
	
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UObject Interface
	void BeginDestroy() override;
	bool IsReadyForFinishDestroy() override;
	//~ End UObject Interface

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	// IOptimusDeformerAssetPathAccessor
	void SetOptimusDeformerAssetPath(const FTopLevelAssetPath& InPath) override;

	TArray<FHalfEdgeBuffers> OnDemandHalfEdgeBuffers;
	FRenderCommandFence DestroyFence;
	
protected:
#if WITH_EDITOR
	void ValidateSkinnedAsset();
	FTopLevelAssetPath DeformerAssetPath;
	TSet<TWeakObjectPtr<USkinnedAsset>> ValidatedAssets;
#endif
};

class FOptimusHalfEdgeDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusHalfEdgeDataProviderProxy(
		USkinnedMeshComponent* InSkinnedMeshComponent, 
		TArray<FHalfEdgeBuffers>& InOnDemandHalfEdgeBuffers);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData);
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FHalfEdgeDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;

	TArray<FHalfEdgeBuffers>& OnDemandHalfEdgeBuffers;
	
	bool bUseBufferFromRenderData = false;
	FRDGBufferSRV* VertexToEdgeBufferSRV = nullptr;
	FRDGBufferSRV* EdgeToTwinEdgeBufferSRV = nullptr;
	
	FRDGBufferSRVRef FallbackSRV = nullptr;
};

#undef UE_API
