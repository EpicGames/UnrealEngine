// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceLoopTerminal.h"

#include "Components/SceneComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceLoopTerminal)


TArray<FOptimusCDIPinDefinition> UOptimusLoopTerminalDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "Index", "ReadIndex", false});
	Defs.Add({ "Count", "ReadCount", false});
	return Defs;	
}

TSubclassOf<UActorComponent> UOptimusLoopTerminalDataInterface::GetRequiredComponentClass() const
{
	return USceneComponent::StaticClass();
}

void UOptimusLoopTerminalDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadIndex"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadCount"))
		.AddReturnType(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FLoopTerminalDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, Index)
	SHADER_PARAMETER(uint32, Count)
END_SHADER_PARAMETER_STRUCT()

void UOptimusLoopTerminalDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FLoopTerminalDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusLoopTerminalDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceLoopTerminal.ush");

TCHAR const* UOptimusLoopTerminalDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusLoopTerminalDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusLoopTerminalDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
 	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusLoopTerminalDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusLoopTerminalDataProvider* Provider = NewObject<UOptimusLoopTerminalDataProvider>();
	Provider->Index = Index;
	Provider->Count = Count;
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusLoopTerminalDataProvider::GetRenderProxy()
{
	return new FOptimusLoopTerminalDataProviderProxy(Index, Count);
}

FOptimusLoopTerminalDataProviderProxy::FOptimusLoopTerminalDataProviderProxy(
	uint32 InIndex,
	uint32 InCount
	) :
	Index(InIndex),
	Count(InCount)
{
}


bool FOptimusLoopTerminalDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return true;
}

void FOptimusLoopTerminalDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.Index = Index;
		Parameters.Count = Count;
	}
}
