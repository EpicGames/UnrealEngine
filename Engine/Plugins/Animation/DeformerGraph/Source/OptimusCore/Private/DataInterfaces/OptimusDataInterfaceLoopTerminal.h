// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "OptimusDataInterfaceLoopTerminal.generated.h"

class FLoopTerminalDataInterfaceParameters;
class USceneComponent;

/** Compute Framework Data Interface for reading loop data. */
UCLASS(Category = ComputeFramework)
class UOptimusLoopTerminalDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	FString GetDisplayName() const override { return TEXT("LoopTerminal"); }
	bool IsVisible() const override { return false; }
	TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("LoopTerminal"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	friend class UOptimusDeformer;
	
	static TCHAR const* TemplateFilePath;

	UPROPERTY()
	uint32 Index = 0;

	UPROPERTY()
	uint32 Count = 0;
};

/** Compute Framework Data Provider for reading loop data. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusLoopTerminalDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	uint32 Index = 0;
	
	uint32 Count = 0;
};

class FOptimusLoopTerminalDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusLoopTerminalDataProviderProxy(uint32 InIndex, uint32 InCount);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FLoopTerminalDataInterfaceParameters;

	uint32 Index = 0;
	uint32 Count = 0;
};
