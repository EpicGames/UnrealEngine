// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceMorphTarget.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceMorphTarget)

FString UOptimusMorphTargetDataInterface::GetDisplayName() const
{
	return TEXT("Morph Target");
}

TArray<FOptimusCDIPinDefinition> UOptimusMorphTargetDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "DeltaPosition", "ReadDeltaPosition", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "DeltaNormal", "ReadDeltaNormal", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}

TSubclassOf<UActorComponent> UOptimusMorphTargetDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}

void UOptimusMorphTargetDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDeltaPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadDeltaNormal"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FMorphTargetDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER_SRV(Buffer<float>, MorphBuffer)
END_SHADER_PARAMETER_STRUCT()

void UOptimusMorphTargetDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FMorphTargetDataInterfaceParameters>(UID);
}

void UOptimusMorphTargetDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_MORPHTARGET"), 2);
}

TCHAR const* UOptimusMorphTargetDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceMorphTarget.ush");

TCHAR const* UOptimusMorphTargetDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusMorphTargetDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusMorphTargetDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusMorphTargetDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusMorphTargetDataProvider* Provider = NewObject<UOptimusMorphTargetDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusMorphTargetDataProvider::GetRenderProxy()
{
	return new FOptimusMorphTargetDataProviderProxy(SkinnedMesh.Get());
}


FOptimusMorphTargetDataProviderProxy::FOptimusMorphTargetDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent != nullptr && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
		FrameNumber = SkinnedMeshComponent->GetScene()->GetFrameNumber() + 1; // +1 matches the logic for FrameNumberToPrepare in FSkeletalMeshObjectGPUSkin::Update()
	}
}

bool FOptimusMorphTargetDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}

	return true;
}

struct FMorphTargetDataInterfacePermutationIds
{
	uint32 EnableDeformerMorphTarget = 0;

	FMorphTargetDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_MORPHTARGET"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerMorphTarget = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusMorphTargetDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();

	FMorphTargetDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		constexpr bool bPreviousFrame = false;
		FRHIShaderResourceView* MorphBufferSRV = FSkeletalMeshDeformerHelpers::GetMorphTargetBufferForReading(SkeletalMeshObject, LodIndex, bPreviousFrame);
		const bool bValidMorph = MorphBufferSRV != nullptr;

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidMorph ? PermutationIds.EnableDeformerMorphTarget : 0);
	}
}

void FOptimusMorphTargetDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		constexpr bool bPreviousFrame = false;
		FRHIShaderResourceView* MorphBufferSRV = FSkeletalMeshDeformerHelpers::GetMorphTargetBufferForReading(SkeletalMeshObject, LodIndex, bPreviousFrame);

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.MorphBuffer = MorphBufferSRV != nullptr ? MorphBufferSRV : NullSRVBinding;
	}
}
