// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"

#include "OptimusDataInterfaceMorphTarget.generated.h"

#define UE_API OPTIMUSCORE_API

class FMorphTargetDataInterfaceParameters;
class FSceneInterface;
class FSkeletalMeshObject;
class USkinnedMeshComponent;

/** Compute Framework Data Interface for reading skeletal mesh. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusMorphTargetDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("MorphTarget"); }
	bool CanSupportUnifiedDispatch() const override {return true;}
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusMorphTargetDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusMorphTargetDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusMorphTargetDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData);
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FMorphTargetDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	uint32 FrameNumber = 0;
};

#undef UE_API
