// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceRawBuffer.h"

#include "OptimusDataTypeRegistry.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDeformerInstance.h"
#include "OptimusExpressionEvaluator.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"
#include "ShaderCompilerCore.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceRawBuffer)


const UOptimusComponentSource* UOptimusRawBufferDataInterface::GetComponentSource() const
{
	if (const UOptimusComponentSourceBinding* ComponentSourceBindingPtr = ComponentSourceBinding.Get())
	{
		return ComponentSourceBindingPtr->GetComponentSource();
	}
	return nullptr;
}

bool UOptimusRawBufferDataInterface::SupportsAtomics() const
{
	TArray<FOptimusDataTypeHandle> Types = FOptimusDataTypeRegistry::Get().GetAllTypesWithAtomicSupport();
	for (FOptimusDataTypeHandle Type : Types)
	{
		if (Type->ShaderValueType == ValueType)
		{
			return true;
		}
	}
	
	return false;
}

FString UOptimusRawBufferDataInterface::GetRawType() const
{
	// Currently the only case for using a raw typed buffer is for vectors with size 3.
	// We _may_ also need something for user structures if we they don't obey alignment restrictions.
	if (ValueType->DimensionType == EShaderFundamentalDimensionType::Vector && ValueType->VectorElemCount == 3)
	{
		return FShaderValueType::Get(ValueType->Type)->ToString();
	}
	return {};
}

int32 UOptimusRawBufferDataInterface::GetRawStride() const
{
	// Currently the only case for using a raw typed buffer is for vectors with size 3.
	// We _may_ also need something for user structures if we they don't obey alignment restrictions.
	if (ValueType->DimensionType == EShaderFundamentalDimensionType::Vector && ValueType->VectorElemCount == 3)
	{
		return 4;
	}
	return 0;
}

TArray<FOptimusCDIPinDefinition> UOptimusRawBufferDataInterface::GetPinDefinitions() const
{
	// FIXME: Multi-level support by proxying through a data interface.
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"ValueIn", "ReadValue", DataDomain.DimensionNames[0], "ReadNumValues"});
	Defs.Add({"ValueOut", "WriteValue", DataDomain.DimensionNames[0], "ReadNumValues"});
	return Defs;
}

TSubclassOf<UActorComponent> UOptimusRawBufferDataInterface::GetRequiredComponentClass() const
{
	return USceneComponent::StaticClass();
}

void UOptimusRawBufferDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	// Functions in order of EOptimusBufferReadType
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumValues"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValue"))
		.AddReturnType(ValueType)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValueUAV"))
		.AddReturnType(ValueType)
		.AddParam(EShaderFundamentalType::Uint);
}

void UOptimusRawBufferDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	// Functions in order of EOptimusBufferWriteType
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteValue"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(ValueType);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicAdd"))
		.AddReturnType(ValueType)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(ValueType);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicMin"))
		.AddReturnType(ValueType)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(ValueType);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteAtomicMax"))
		.AddReturnType(ValueType)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(ValueType);
}

int32 UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType ReadType)
{
	return (int32)ReadType;
}

int32 UOptimusRawBufferDataInterface::GetWriteValueOutputIndex(EOptimusBufferWriteType WriteType)
{
	return (int32)WriteType;
}

BEGIN_SHADER_PARAMETER_STRUCT(FTransientBufferDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, BufferSize)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, BufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, BufferUAV)
END_SHADER_PARAMETER_STRUCT()

void UOptimusTransientBufferDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FTransientBufferDataInterfaceParameters>(UID);
}

BEGIN_SHADER_PARAMETER_STRUCT(FImplicitPersistentBufferDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, BufferSize)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int>, BufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, BufferUAV)
END_SHADER_PARAMETER_STRUCT()

void UOptimusImplicitPersistentBufferDataInterface::GetShaderParameters(TCHAR const* UID,
	FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FImplicitPersistentBufferDataInterfaceParameters>(UID);
}

BEGIN_SHADER_PARAMETER_STRUCT(FPersistentBufferDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, BufferSize)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, BufferUAV)
END_SHADER_PARAMETER_STRUCT()

void UOptimusPersistentBufferDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FPersistentBufferDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusRawBufferDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceRawBuffer.ush");

TCHAR const* UOptimusRawBufferDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusRawBufferDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusRawBufferDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	const FString PublicType = ValueType->ToString();
	const FString RawType = GetRawType();
	const bool bUseRawType = !RawType.IsEmpty();

	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
		{ TEXT("PublicType"), PublicType },
		{ TEXT("BufferType"), bUseRawType ? RawType : PublicType },
		{ TEXT("BufferTypeRaw"), bUseRawType },
		{ TEXT("SupportAtomic"), SupportsAtomics() ? 1 : 0 },
		{ TEXT("SplitReadWrite"), UseSplitBuffers() ? 1 : 0 },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

FString UOptimusTransientBufferDataInterface::GetDisplayName() const
{
	return TEXT("Transient");
}

UComputeDataProvider* UOptimusTransientBufferDataInterface::CreateDataProvider(
	TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask
	) const
{
	UOptimusTransientBufferDataProvider *Provider = CreateProvider<UOptimusTransientBufferDataProvider>(InBinding);
	Provider->bZeroInitForAtomicWrites = bZeroInitForAtomicWrites;
	return Provider;
}


FString UOptimusImplicitPersistentBufferDataInterface::GetDisplayName() const
{
	return TEXT("ImplicitPersistent");
}

UComputeDataProvider* UOptimusImplicitPersistentBufferDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding,
	uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusImplicitPersistentBufferDataProvider *Provider = CreateProvider<UOptimusImplicitPersistentBufferDataProvider>(InBinding);
	Provider->DataInterfaceName = this->GetFName();
	Provider->bZeroInitForAtomicWrites = bZeroInitForAtomicWrites;
	return Provider;
}


FString UOptimusPersistentBufferDataInterface::GetDisplayName() const
{
	return TEXT("Persistent");
}


UComputeDataProvider* UOptimusPersistentBufferDataInterface::CreateDataProvider(
	TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask
	) const
{
	UOptimusPersistentBufferDataProvider *Provider = CreateProvider<UOptimusPersistentBufferDataProvider>(InBinding);
	
	Provider->ResourceName = ResourceName;

	return Provider;
}


bool UOptimusRawBufferDataProvider::GetLodAndInvocationElementCounts(
	int32& OutLodIndex,
	TArray<int32>& OutInvocationElementCounts
	) const
{
	const UOptimusComponentSource* ComponentSourcePtr = ComponentSource.Get();
	if (!ComponentSourcePtr)
	{
		return false;
	}

	const UActorComponent* ComponentPtr = Component.Get();
	if (!ComponentPtr)
	{
		return false;
	}

	OutLodIndex = ComponentSourcePtr->GetLodIndex(ComponentPtr);

	return Optimus::EvaluateExecutionDomainExpressionParseResult(DataDomainExpressionParseResult, ComponentSource, Component, OutInvocationElementCounts);
}

FComputeDataProviderRenderProxy* UOptimusTransientBufferDataProvider::GetRenderProxy()
{
	int32 LodIndex;
	TArray<int32> InvocationCounts;
	
	if (!GetLodAndInvocationElementCounts(LodIndex, InvocationCounts))
	{
		InvocationCounts.Reset();
	}
	
	return new FOptimusTransientBufferDataProviderProxy(InvocationCounts, ElementStride, RawStride, bZeroInitForAtomicWrites);
}

FComputeDataProviderRenderProxy* UOptimusImplicitPersistentBufferDataProvider::GetRenderProxy()
{
	int32 LodIndex;
	TArray<int32> InvocationCounts;

	if (!GetLodAndInvocationElementCounts(LodIndex, InvocationCounts))
	{
		InvocationCounts.Reset();
	}

	return new FOptimusImplicitPersistentBufferDataProviderProxy(
		InvocationCounts, ElementStride, RawStride, bZeroInitForAtomicWrites,
		BufferPool,
		DataInterfaceName,
		LodIndex);
}


FComputeDataProviderRenderProxy* UOptimusPersistentBufferDataProvider::GetRenderProxy()
{
	int32 LodIndex = 0;
	TArray<int32> InvocationCounts;
	if (!GetLodAndInvocationElementCounts(LodIndex, InvocationCounts))
	{
		InvocationCounts.Reset();
	}
	
	return new FOptimusPersistentBufferDataProviderProxy(InvocationCounts, ElementStride, RawStride, BufferPool, ResourceName, LodIndex);
}


FOptimusTransientBufferDataProviderProxy::FOptimusTransientBufferDataProviderProxy(
	TArray<int32> InInvocationElementCounts,
	int32 InElementStride,
	int32 InRawStride,
	bool bInZeroInitForAtomicWrites
	) :
	InvocationElementCounts(InInvocationElementCounts),
	TotalElementCount(0),
	ElementStride(InElementStride),
	RawStride(InRawStride),
	bZeroInitForAtomicWrites(bInZeroInitForAtomicWrites)
{
	for (int32 NumElements : InvocationElementCounts)
	{
		TotalElementCount += NumElements;
	}
}

bool FOptimusTransientBufferDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	
	if (TotalElementCount <= 0)
	{
		return false;
	}

	return true;
}

void FOptimusTransientBufferDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	// If we are using a raw type alias for the buffer then we need to adjust stride and count.
	check(RawStride == 0 || ElementStride % RawStride == 0);
	const int32 Stride = RawStride ? RawStride : ElementStride;
	const int32 ElementStrideMultiplier = RawStride ? ElementStride / RawStride : 1;

	Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(Stride, TotalElementCount * ElementStrideMultiplier), TEXT("TransientBuffer"), ERDGBufferFlags::None);
	BufferSRV = GraphBuilder.CreateSRV(Buffer);
	BufferUAV = GraphBuilder.CreateUAV(Buffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

	if (bZeroInitForAtomicWrites)
	{
		AddClearUAVPass(GraphBuilder, BufferUAV, 0);
	}
}

void FOptimusTransientBufferDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.BufferSize =TotalElementCount;
		Parameters.BufferSRV = BufferSRV;
		Parameters.BufferUAV = BufferUAV;
	}
}

FOptimusImplicitPersistentBufferDataProviderProxy::FOptimusImplicitPersistentBufferDataProviderProxy(
	TArray<int32> InInvocationElementCounts,
	int32 InElementStride,
	int32 InRawStride,
	bool bInZeroInitForAtomicWrites,
	TSharedPtr<FOptimusPersistentBufferPool> InBufferPool,
	FName InDataInterfaceName,
	int32 InLODIndex
	) :
	InvocationElementCounts(InInvocationElementCounts),
	TotalElementCount(0),
	ElementStride(InElementStride),
	RawStride(InRawStride),
	bZeroInitForAtomicWrites(bInZeroInitForAtomicWrites),
	BufferPool(InBufferPool),
	DataInterfaceName(InDataInterfaceName),
	LODIndex(InLODIndex)
{
	for (int32 NumElements : InvocationElementCounts)
	{
		TotalElementCount += NumElements;
	}
}

bool FOptimusImplicitPersistentBufferDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	
	if (TotalElementCount <= 0)
	{
		return false;
	}

	if (DataInterfaceName == NAME_None)
	{
		return false;
	}

	return true;
}

void FOptimusImplicitPersistentBufferDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	TArray<int32> Count;
	Count.Add(TotalElementCount);
	TArray<FRDGBufferRef> Buffers;
	bool bJustAllocated = false;
	BufferPool->GetImplicitPersistentBuffers(GraphBuilder, DataInterfaceName, LODIndex, ElementStride, RawStride, Count, Buffers, bJustAllocated);

	ensure(Buffers.Num() == 1);
	Buffer = Buffers[0];

	BufferSRV = GraphBuilder.CreateSRV(Buffer);
	BufferUAV = GraphBuilder.CreateUAV(Buffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

	if (bZeroInitForAtomicWrites && bJustAllocated)
	{
		AddClearUAVPass(GraphBuilder, BufferUAV, 0);
	}
}

void FOptimusImplicitPersistentBufferDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.BufferSize = TotalElementCount;
		Parameters.BufferSRV = BufferSRV;
		Parameters.BufferUAV = BufferUAV;
	}
}


FOptimusPersistentBufferDataProviderProxy::FOptimusPersistentBufferDataProviderProxy(
	TArray<int32> InInvocationElementCounts,
	int32 InElementStride,
	int32 InRawStride,
	TSharedPtr<FOptimusPersistentBufferPool> InBufferPool,
	FName InResourceName,
	int32 InLODIndex
	) : 
	InvocationElementCounts(InInvocationElementCounts),
	TotalElementCount(0),
	ElementStride(InElementStride),
	RawStride(InRawStride),
	BufferPool(InBufferPool),
	ResourceName(InResourceName),
	LODIndex(InLODIndex)
{
	for (int32 NumElements : InvocationElementCounts)
	{
		TotalElementCount += NumElements;
	}
}

bool FOptimusPersistentBufferDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	
	if (TotalElementCount <= 0)
	{
		return false;
	}

	return true;
}

void FOptimusPersistentBufferDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	TArray<int32> Count;
	Count.Add(TotalElementCount);
	TArray<FRDGBufferRef> Buffers;
	bool bJustAllocated = false;
	BufferPool->GetResourceBuffers(GraphBuilder, ResourceName, LODIndex, ElementStride, RawStride, Count, Buffers, bJustAllocated);

	ensure(Buffers.Num() == 1);
	Buffer = Buffers[0];

	// We want to use SkipBarrier to allow simultaineous execution of any subinvocations, but we want to keep barriers between kernels. 
	// RDG will do this if we use a different UAV object per kernel based on the same underlying buffer. So we create one UAV per kernel here.
	// This may end up being overkill for large graphs, in which case we will only want to create a UAV for each kernel that uses this data provider.
	BufferUAVs.Reserve(InAllocationData.NumGraphKernels);
	for (int32 BufferIndex = 0; BufferIndex < InAllocationData.NumGraphKernels; ++BufferIndex)
	{
		BufferUAVs.Add(GraphBuilder.CreateUAV(Buffer, ERDGUnorderedAccessViewFlags::SkipBarrier));
	}
}

void FOptimusPersistentBufferDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0;  InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.BufferSize = TotalElementCount;
		Parameters.BufferUAV = BufferUAVs[InDispatchData.GraphKernelIndex];
	}
}
