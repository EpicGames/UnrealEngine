// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusPersistentBufferProvider.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusConstant.h"
#include "OptimusDataDomain.h"
#include "OptimusDeformerInstance.h"
#include "OptimusHelpers.h"
#include "RenderGraphFwd.h"

#include "OptimusDataInterfaceRawBuffer.generated.h"

#define UE_API OPTIMUSCORE_API

class FOptimusPersistentBufferPool;
class FTransientBufferDataInterfaceParameters;
class FImplicitPersistentBufferDataInterfaceParameters;
class FPersistentBufferDataInterfaceParameters;
class UOptimusComponentSource;
class UOptimusComponentSourceBinding;
class UOptimusRawBufferDataProvider;

enum class EOptimusBufferReadType : uint8
{
	ReadSize,
	Default,
	ForceUAV
};

/** Write to buffer operation types. */
UENUM()
enum class EOptimusBufferWriteType : uint8
{
	Write UMETA(ToolTip = "Write the value to resource buffer."),
	WriteAtomicAdd UMETA(ToolTip = "AtomicAdd the value to the value in the resource buffer."),
	WriteAtomicMin UMETA(ToolTip = "AtomicMin the value with the value in the resource buffer."),
	WriteAtomicMax UMETA(ToolTip = "AtomicMax the value with the value in the resource buffer."),
	Count UMETA(Hidden),
};

UCLASS(MinimalAPI, Abstract)
class UOptimusRawBufferDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	static UE_API int32 GetReadValueInputIndex(EOptimusBufferReadType ReadType = EOptimusBufferReadType::Default);
	static UE_API int32 GetWriteValueOutputIndex(EOptimusBufferWriteType WriteType);
	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	bool IsVisible() const override	{ return false;	}
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	//~ End UComputeDataInterface Interface

	/** The value type we should be allocating elements for */
	UPROPERTY()
	FShaderValueTypeHandle ValueType;

	/** The data domain this buffer covers */
	UPROPERTY()
	FOptimusDataDomain DataDomain;

	/** The component source to query component domain validity and sizing */
	UPROPERTY()
	TWeakObjectPtr<UOptimusComponentSourceBinding> ComponentSourceBinding;

	UPROPERTY()
	FOptimusConstantIdentifier DomainConstantIdentifier_DEPRECATED;
	
protected:
	virtual bool UseSplitBuffers() const { return true; }

	UE_API int32 GetRawStride() const;

	template<typename U>
	U* CreateProvider(TObjectPtr<UObject> InBinding) const
	{
		U *Provider = NewObject<U>();
		if (UActorComponent* Component = Cast<UActorComponent>(InBinding))
		{
			Provider->Component = Component;
			Provider->ComponentSource = GetComponentSource();
			Provider->ElementStride = ValueType->GetResourceElementSize();
			Provider->RawStride = GetRawStride();
			Provider->DataDomainExpressionParseResult = Optimus::ParseExecutionDomainExpression(DataDomain.AsExpression().Get({}), Provider->ComponentSource);
		}
		return Provider;
	}

private:
	static UE_API TCHAR const* TemplateFilePath;

	UE_API const UOptimusComponentSource* GetComponentSource() const;
	UE_API bool SupportsAtomics() const;
	UE_API FString GetRawType() const;
};


/** Compute Framework Data Interface for a transient buffer. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusTransientBufferDataInterface : public UOptimusRawBufferDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("TransientBuffer"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UPROPERTY()
	bool bZeroInitForAtomicWrites = false;	
};

/** Compute Framework Data Interface for a implicit persistent buffer. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusImplicitPersistentBufferDataInterface : public UOptimusRawBufferDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	//~ End UOptimusComputeDataInterface Interface


	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("ImplicitPersistentBuffer"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UPROPERTY()
	bool bZeroInitForAtomicWrites = false;
};


/** Compute Framework Data Interface for a transient buffer. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusPersistentBufferDataInterface : public UOptimusRawBufferDataInterface
{
	GENERATED_BODY()

public:
	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("PersistentBuffer"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UPROPERTY()
	FName ResourceName;

protected:
	// For persistent buffers, we only provide the UAV, not the SRV.
	bool UseSplitBuffers() const override { return false; } 
};


/** Compute Framework Data Provider for a transient buffer. */
UCLASS(MinimalAPI, Abstract)
class UOptimusRawBufferDataProvider :
	public UComputeDataProvider
{
	GENERATED_BODY()

public:
	/** Helper function to calculate the element count for each section invocation of the given skinned/skeletal mesh
	 *  object and a data domain. Uses the execution domains to compute the information. Returns false if the
	 *  data domain is not valid for computation.
	 */
	UE_API bool GetLodAndInvocationElementCounts(
		int32& OutLodIndex,
		TArray<int32>& OutInvocationElementCounts
		) const;

	/** The skinned mesh component that governs the sizing and LOD of this buffer */
	TWeakObjectPtr<const UActorComponent> Component = nullptr;

	TWeakObjectPtr<const UOptimusComponentSource> ComponentSource = nullptr;

	int32 ElementStride = 4;

	int32 RawStride = 0;

	TVariant<Optimus::Expression::FExpressionObject, Optimus::Expression::FParseError> DataDomainExpressionParseResult;
};


/** Compute Framework Data Provider for a transient buffer. */
UCLASS(MinimalAPI, BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusTransientBufferDataProvider : public UOptimusRawBufferDataProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	UE_API FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	bool bZeroInitForAtomicWrites = false;
};


/** Compute Framework Data Provider for a transient buffer. */
UCLASS(MinimalAPI, BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusImplicitPersistentBufferDataProvider :
	public UOptimusRawBufferDataProvider,
	public IOptimusPersistentBufferProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	UE_API FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin IOptimusPersistentBufferPoolUser Interface
	void SetBufferPool(TSharedPtr<FOptimusPersistentBufferPool> InBufferPool) override { BufferPool = InBufferPool; };
	//~ Begin IOptimusPersistentBufferPoolUser Interface
	
	bool bZeroInitForAtomicWrites = false;
	
	FName DataInterfaceName = NAME_None;
private:
	/** The buffer pool we refer to. Set by UOptimusDeformerInstance::SetupFromDeformer after providers have been
	 *  created
	 */
	TSharedPtr<FOptimusPersistentBufferPool> BufferPool;
};

/** Compute Framework Data Provider for a transient buffer. */
UCLASS(MinimalAPI, BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusPersistentBufferDataProvider :
	public UOptimusRawBufferDataProvider,
	public IOptimusPersistentBufferProvider
{
	GENERATED_BODY()

public:
	//~ Begin UComputeDataProvider Interface
	UE_API FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
	
	//~ Begin IOptimusPersistentBufferPoolUser Interface
	void SetBufferPool(TSharedPtr<FOptimusPersistentBufferPool> InBufferPool) override { BufferPool = InBufferPool; };
	//~ Begin IOptimusPersistentBufferPoolUser Interface
	
	/** The resource this buffer is provider to */
	FName ResourceName;
	
private:
	/** The buffer pool we refer to. Set by UOptimusDeformerInstance::SetupFromDeformer after providers have been
	 *  created
	 */
	TSharedPtr<FOptimusPersistentBufferPool> BufferPool;
};

class FOptimusTransientBufferDataProviderProxy :
	public FComputeDataProviderRenderProxy
{
public:
	FOptimusTransientBufferDataProviderProxy(
		TArray<int32> InInvocationElementCounts,
		int32 InElementStride,
		int32 InRawStride,
		bool bInZeroInitForAtomicWrites
		);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FTransientBufferDataInterfaceParameters;

	const TArray<int32> InvocationElementCounts;
	int32 TotalElementCount;
	const int32 ElementStride;
	const int32 RawStride;
	const bool bZeroInitForAtomicWrites;

	FRDGBufferRef Buffer;
	FRDGBufferSRVRef BufferSRV;
	FRDGBufferUAVRef BufferUAV;
};

class FOptimusImplicitPersistentBufferDataProviderProxy :
	public FComputeDataProviderRenderProxy
{
public:
	FOptimusImplicitPersistentBufferDataProviderProxy(
		TArray<int32> InInvocationElementCounts,
		int32 InElementStride,
		int32 InRawStride,
		bool bInZeroInitForAtomicWrites,
		TSharedPtr<FOptimusPersistentBufferPool> InBufferPool,
		FName InDataInterfaceName,
		int32 InLODIndex
		);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

	private:
	using FParameters = FImplicitPersistentBufferDataInterfaceParameters;

	const TArray<int32> InvocationElementCounts;
	int32 TotalElementCount;
	const int32 ElementStride;
	const int32 RawStride;
	const bool bZeroInitForAtomicWrites;
	
	const TSharedPtr<FOptimusPersistentBufferPool> BufferPool;
	FName DataInterfaceName;
	int32 LODIndex;

	FRDGBufferRef Buffer;
	FRDGBufferSRVRef BufferSRV;
	FRDGBufferUAVRef BufferUAV;
};

class FOptimusPersistentBufferDataProviderProxy :
	public FComputeDataProviderRenderProxy
{
public:
	FOptimusPersistentBufferDataProviderProxy(
		TArray<int32> InInvocationElementCounts,
		int32 InElementStride,
		int32 InRawStride,
		TSharedPtr<FOptimusPersistentBufferPool> InBufferPool,
		FName InResourceName,
		int32 InLODIndex
	);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData);
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FPersistentBufferDataInterfaceParameters;

	const TArray<int32> InvocationElementCounts;
	int32 TotalElementCount;
	const int32 ElementStride;
	const int32 RawStride;
	const TSharedPtr<FOptimusPersistentBufferPool> BufferPool;
	const FName ResourceName;
	const int32 LODIndex;

	FRDGBufferRef Buffer;
	TArray<FRDGBufferUAVRef> BufferUAVs;
};

#undef UE_API
