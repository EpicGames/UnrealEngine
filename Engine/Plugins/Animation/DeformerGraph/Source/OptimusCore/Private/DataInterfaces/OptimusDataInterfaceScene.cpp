// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceScene.h"

#include "Components/SceneComponent.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "CoreGlobals.h"
#include "Engine/World.h"
#include "SceneInterface.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceScene)

FString UOptimusSceneDataInterface::GetDisplayName() const
{
	return TEXT("Scene Data");
}

TArray<FOptimusCDIPinDefinition> UOptimusSceneDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"GameTime", "ReadGameTime"});
	Defs.Add({"GameTimeDelta", "ReadGameTimeDelta"});
	Defs.Add({"FrameNumber", "ReadFrameNumber"});
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSceneDataInterface::GetRequiredComponentClass() const
{
	return USceneComponent::StaticClass();
}


void UOptimusSceneDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadGameTime"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadGameTimeDelta"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadFrameNumber"))
		.AddReturnType(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSceneDataInterfaceParameters, )
	SHADER_PARAMETER(float, GameTime)
	SHADER_PARAMETER(float, GameTimeDelta)
	SHADER_PARAMETER(uint32, FrameNumber)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSceneDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSceneDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusSceneDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceScene.ush");

TCHAR const* UOptimusSceneDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSceneDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSceneDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
 	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSceneDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSceneDataProvider* Provider = NewObject<UOptimusSceneDataProvider>();
	Provider->SceneComponent = Cast<USceneComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusSceneDataProvider::GetRenderProxy()
{
	return new FOptimusSceneDataProviderProxy(SceneComponent.Get());
}


FOptimusSceneDataProviderProxy::FOptimusSceneDataProviderProxy(USceneComponent* SceneComponent)
{
	bIsValid = SceneComponent != nullptr;
	
	bool bUseSceneTime = bIsValid;
#if WITH_EDITOR
	// Don't tick time in Editor unless in PIE.
	if (GIsEditor && bUseSceneTime)
	{
		bUseSceneTime = SceneComponent->GetWorld() != nullptr && SceneComponent->GetWorld()->WorldType != EWorldType::Editor;
	}
#endif
	GameTime = bUseSceneTime ? SceneComponent->GetWorld()->TimeSeconds : 0;
	GameTimeDelta = bUseSceneTime ? SceneComponent->GetWorld()->DeltaTimeSeconds : 0;
	FrameNumber = bUseSceneTime ? SceneComponent->GetScene()->GetFrameNumber() : 0;
}

bool FOptimusSceneDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return bIsValid;
}

void FOptimusSceneDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.GameTime = GameTime;
		Parameters.GameTimeDelta = GameTimeDelta;
		Parameters.FrameNumber = FrameNumber;
	}
}
