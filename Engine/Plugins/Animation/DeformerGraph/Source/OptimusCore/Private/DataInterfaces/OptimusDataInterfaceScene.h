// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "OptimusDataInterfaceScene.generated.h"

class FSceneDataInterfaceParameters;
class USceneComponent;

/** Compute Framework Data Interface for reading general scene data. */
UCLASS(Category = ComputeFramework)
class UOptimusSceneDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	FString GetDisplayName() const override;
	TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("Scene"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	TCHAR const* GetShaderVirtualPath() const override;
	void GetShaderHash(FString& InOutKey) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading general scene data. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSceneDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USceneComponent> SceneComponent = nullptr;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusSceneDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSceneDataProviderProxy(USceneComponent* SceneComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSceneDataInterfaceParameters;

	bool bIsValid;
	float GameTime;
	float GameTimeDelta;
	uint32 FrameNumber;
};
