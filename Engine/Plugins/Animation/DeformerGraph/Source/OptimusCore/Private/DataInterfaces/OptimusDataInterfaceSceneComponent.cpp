// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSceneComponent.h"

#include "Components/SceneComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusHelpers.h"
#include "SceneInterface.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSceneComponent)

FString UOptimusSceneComponentDataInterface::GetDisplayName() const
{
	return TEXT("Scene Component Data");
}

TArray<FOptimusCDIPinDefinition> UOptimusSceneComponentDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "WorldTransform", "ReadWorldTransform" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSceneComponentDataInterface::GetRequiredComponentClass() const
{
	return USceneComponent::StaticClass();
}


void UOptimusSceneComponentDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWorldTransform"))
		.AddReturnType(EShaderFundamentalType::Float, 4, 4);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSceneComponentDataInterfaceParameters, )
	SHADER_PARAMETER(FMatrix44f, WorldTransform)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSceneComponentDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSceneComponentDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusSceneComponentDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSceneComponent.ush");

TCHAR const* UOptimusSceneComponentDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSceneComponentDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSceneComponentDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
 	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSceneComponentDataInterface::CreateDataProvider() const
{
	return NewObject<UOptimusSceneComponentDataProvider>();
}

void UOptimusSceneComponentDataProvider::Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask)
{
	SceneComponent = Cast<USceneComponent>(InBinding);
}

FComputeDataProviderRenderProxy* UOptimusSceneComponentDataProvider::GetRenderProxy()
{
	return new FOptimusSceneComponentDataProviderProxy(SceneComponent.Get());
}


FOptimusSceneComponentDataProviderProxy::FOptimusSceneComponentDataProviderProxy(USceneComponent* SceneComponent)
{
	bIsValid = SceneComponent != nullptr;
	if (bIsValid)
	{
		WorldTransform = SceneComponent->GetComponentTransform();
	}
}

bool FOptimusSceneComponentDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return bIsValid;
}

void FOptimusSceneComponentDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.WorldTransform = Optimus::ConvertFTransformToFMatrix44f(WorldTransform);
	}
}
