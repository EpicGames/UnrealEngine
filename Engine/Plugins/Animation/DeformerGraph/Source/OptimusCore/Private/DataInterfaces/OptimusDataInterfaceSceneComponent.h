// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "OptimusDataInterfaceSceneComponent.generated.h"

class FSceneComponentDataInterfaceParameters;
class USceneComponent;

/** Compute Framework Data Interface for reading general scene component data. */
UCLASS(Category = ComputeFramework)
class UOptimusSceneComponentDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	virtual TCHAR const* GetClassName() const override { return TEXT("SceneComponent"); }
	virtual bool CanSupportUnifiedDispatch() const override { return true; }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider() const override;
	//~ End UComputeDataInterface Interface

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading general scene data. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSceneComponentDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USceneComponent> SceneComponent = nullptr;

	//~ Begin UComputeDataProvider Interface
	virtual void Initialize(const UComputeDataInterface* InDataInterface, UObject* InBinding, uint64 InInputMask, uint64 InOutputMask) override;
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusSceneComponentDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSceneComponentDataProviderProxy(USceneComponent* SceneComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSceneComponentDataInterfaceParameters;

	bool bIsValid = false;
	FTransform WorldTransform = FTransform::Identity;
};
