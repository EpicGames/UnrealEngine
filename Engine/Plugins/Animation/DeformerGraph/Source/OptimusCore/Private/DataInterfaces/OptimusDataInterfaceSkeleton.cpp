// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkeleton.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkeleton)

FString UOptimusSkeletonDataInterface::GetDisplayName() const
{
	return TEXT("Skeleton");
}

TArray<FOptimusCDIPinDefinition> UOptimusSkeletonDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumBones", "ReadNumBones", Optimus::DomainName::Vertex, "ReadNumVertices"});
	Defs.Add({"BoneMatrix", "ReadBoneMatrix", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"BoneWeight", "ReadBoneWeight", {{Optimus::DomainName::Vertex, "ReadNumVertices"}, {Optimus::DomainName::Bone, "ReadNumBones"}}});
	Defs.Add({"WeightedBoneMatrix", "ReadWeightedBoneMatrix", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkeletonDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusSkeletonDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumBones"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadBoneWeight"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadWeightedBoneMatrix"))
		.AddReturnType(EShaderFundamentalType::Float, 3, 4)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkeletonDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumBoneInfluences)
	SHADER_PARAMETER(uint32, InputWeightStride)
	SHADER_PARAMETER(uint32, InputWeightIndexSize)
	SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(BoneMatrices)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightStream)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightLookupStream)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkeletonDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkeletonDataInterfaceParameters>(UID);
}

void UOptimusSkeletonDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	// Need to be able to support these permutations according to the skeletal mesh settings.
	// todo[CF]: I think GPUSKIN_UNLIMITED_BONE_INFLUENCE and GPUSKIN_BONE_INDEX_UINT16/GPUSKIN_BONE_WEIGHTS_UINT16 are mutually exclusive. So we could save permutations here.
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_BONES"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_INDEX_UINT16"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"), 2);
}

TCHAR const* UOptimusSkeletonDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkeleton.ush");

TCHAR const* UOptimusSkeletonDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSkeletonDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSkeletonDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSkeletonDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSkeletonDataProvider* Provider = NewObject<UOptimusSkeletonDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusSkeletonDataProvider::GetRenderProxy()
{
	return new FOptimusSkeletonDataProviderProxy(SkinnedMesh.Get());
}


FOptimusSkeletonDataProviderProxy::FOptimusSkeletonDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent != nullptr && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
	}
	BoneRevisionNumber = SkinnedMeshComponent != nullptr ? SkinnedMeshComponent->GetBoneTransformRevisionNumber() : 0;
}

bool FOptimusSkeletonDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (SkeletalMeshObject->GetSkeletalMeshRenderData().LODRenderData[SkeletalMeshObject->GetLOD()].RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}

	return true;
}

struct FSkeletonDataInterfacePermutationIds
{
	uint32 EnableDeformerBones = 0;
	uint32 UnlimitedBoneInfluence = 0;
	uint32 BoneIndexUint16 = 0;
	uint32 BoneWeightsUint16 = 0;

	FSkeletonDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_BONES"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerBones = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"));
			static uint32 Hash = GetTypeHash(Name);
			UnlimitedBoneInfluence = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_INDEX_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneIndexUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneWeightsUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusSkeletonDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FSkeletonDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];

		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
		check(WeightBuffer != nullptr);
		FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
		const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
		FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;
		const bool bValidBones = (BoneBufferSRV != nullptr) && (SkinWeightBufferSRV != nullptr) && (!bUnlimitedBoneInfluences || InputWeightLookupStreamSRV != nullptr);
		const bool bUse16BitBoneIndex = WeightBuffer->Use16BitBoneIndex();
		const bool bUse16BitBoneWeights = WeightBuffer->Use16BitBoneWeight();

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidBones ? PermutationIds.EnableDeformerBones : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUnlimitedBoneInfluences ? PermutationIds.UnlimitedBoneInfluence : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneIndex ? PermutationIds.BoneIndexUint16 : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneWeights ? PermutationIds.BoneWeightsUint16 : 0);
	}
}

void FOptimusSkeletonDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
	check(WeightBuffer != nullptr);
	FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
	const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
	FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumBoneInfluences = WeightBuffer->GetMaxBoneInfluences();
		Parameters.InputWeightStride = WeightBuffer->GetConstantInfluencesVertexStride();
		Parameters.InputWeightIndexSize = WeightBuffer->GetBoneIndexByteSize() | (WeightBuffer->GetBoneWeightByteSize() << 8);
		Parameters.BoneMatrices = BoneBufferSRV != nullptr ? BoneBufferSRV : NullSRVBinding;
		Parameters.InputWeightStream = SkinWeightBufferSRV != nullptr ? SkinWeightBufferSRV : NullSRVBinding;
		Parameters.InputWeightLookupStream = InputWeightLookupStreamSRV != nullptr ? InputWeightLookupStreamSRV : NullSRVBinding;
	}
}
