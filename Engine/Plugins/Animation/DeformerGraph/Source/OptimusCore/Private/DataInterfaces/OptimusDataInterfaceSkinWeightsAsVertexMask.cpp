// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinWeightsAsVertexMask.h"

#include "OptimusDataDomain.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusHelpers.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"
#include "SystemTextures.h"
#include "ComputeFramework/ComputeKernelPermutationVector.h"
#include "Nodes/OptimusNode_DataInterface.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Animation/SkinWeightProfileManager.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinWeightsAsVertexMask)

FName UOptimusSkinWeightsAsVertexMaskDataInterface::GetSkinWeightProfilePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinWeightsAsVertexMaskDataInterface, SkinWeightProfile);	
}

FName UOptimusSkinWeightsAsVertexMaskDataInterface::GetBoneNamesPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinWeightsAsVertexMaskDataInterface, BoneNames);	
}

FName UOptimusSkinWeightsAsVertexMaskDataInterface::GetExpandTowardsRootPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinWeightsAsVertexMaskDataInterface, ExpandTowardsRoot);
}

FName UOptimusSkinWeightsAsVertexMaskDataInterface::GetExpandTowardsLeafPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinWeightsAsVertexMaskDataInterface, ExpandTowardsLeaf);
}

FName UOptimusSkinWeightsAsVertexMaskDataInterface::GetDebugDrawIncludedBonesPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinWeightsAsVertexMaskDataInterface, bDebugDrawIncludedBones);
}

FString UOptimusSkinWeightsAsVertexMaskDataInterface::GetDisplayName() const
{
	return TEXT("Skin Weights as Vertex Mask");
}

TArray<FOptimusCDIPinDefinition> UOptimusSkinWeightsAsVertexMaskDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"Mask", "ReadMask", Optimus::DomainName::Vertex, "ReadNumVertices" });

	return Defs;
}

TArray<FOptimusCDIPropertyPinDefinition> UOptimusSkinWeightsAsVertexMaskDataInterface::GetPropertyPinDefinitions() const
{
	TArray<FOptimusCDIPropertyPinDefinition> Defs;

	const FOptimusDataTypeHandle NameType = FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass());
	const FOptimusDataTypeHandle NameArrayType = FOptimusDataTypeRegistry::Get().FindArrayType(*FNameProperty::StaticClass());
	const FOptimusDataTypeHandle IntType = FOptimusDataTypeRegistry::Get().FindType(*FIntProperty::StaticClass());
	const FOptimusDataTypeHandle BoolType = FOptimusDataTypeRegistry::Get().FindType(*FBoolProperty::StaticClass());
	
	Defs.Add({GetSkinWeightProfilePropertyName(), NameType});
	Defs.Add({GetBoneNamesPropertyName(), NameArrayType});
	Defs.Add({GetExpandTowardsRootPropertyName(), IntType});
	Defs.Add({GetExpandTowardsLeafPropertyName(), IntType});
	Defs.Add({GetDebugDrawIncludedBonesPropertyName(), BoolType});

	return Defs;
}

TSubclassOf<UActorComponent> UOptimusSkinWeightsAsVertexMaskDataInterface::GetRequiredComponentClass() const
{
	return USkeletalMeshComponent::StaticClass();
}

void UOptimusSkinWeightsAsVertexMaskDataInterface::RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode)
{
	if(InNode)
	{
		OnPinDefinitionChangedDelegate.BindUObject(InNode, &UOptimusNode::RecreatePinsFromPinDefinitions);
		OnPinDefinitionRenamedDelegate.BindUObject(InNode, &UOptimusNode::RenamePinFromPinDefinition);
		OnDisplayNameChangedDelegate.BindUObject(InNode, &UOptimusNode::UpdateDisplayNameFromDataInterface);
	}
}

void UOptimusSkinWeightsAsVertexMaskDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadMask"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkinWeightsAsVertexMaskDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumBoneInfluences)
	SHADER_PARAMETER(uint32, InputWeightStride)
	SHADER_PARAMETER(uint32, InputWeightIndexSize)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightStream)
	SHADER_PARAMETER_SRV(Buffer<uint>, InputWeightLookupStream)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, BoneIsSelected)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkinWeightsAsVertexMaskDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder,
	FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkinWeightsAsVertexMaskDataInterfaceParameters>(UID);
}

void UOptimusSkinWeightsAsVertexMaskDataInterface::GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const
{
	// Need to be able to support these permutations according to the skeletal mesh settings.
	// todo[CF]: I think GPUSKIN_UNLIMITED_BONE_INFLUENCE and GPUSKIN_BONE_INDEX_UINT16/GPUSKIN_BONE_WEIGHTS_UINT16 are mutually exclusive. So we could save permutations here.
	OutPermutationVector.AddPermutation(TEXT("ENABLE_DEFORMER_BONES"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_INDEX_UINT16"), 2);
	OutPermutationVector.AddPermutation(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"), 2);
}

TCHAR const* UOptimusSkinWeightsAsVertexMaskDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkinWeightsAsVertexMask.ush");

void UOptimusSkinWeightsAsVertexMaskDataInterface::GetShaderHash(FString& InOutKey) const
{
	return GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

TCHAR const* UOptimusSkinWeightsAsVertexMaskDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;	
}

void UOptimusSkinWeightsAsVertexMaskDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSkinWeightsAsVertexMaskDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask,
	uint64 InOutputMask) const
{
	UOptimusSkinWeightsAsVertexMaskDataProvider* Provider = NewObject<UOptimusSkinWeightsAsVertexMaskDataProvider>();
	Provider->Init(this, Cast<USkeletalMeshComponent>(InBinding));
	return Provider;
}


void UOptimusSkinWeightsAsVertexMaskDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}

void UOptimusSkinWeightsAsVertexMaskDataProvider::Init(const UOptimusSkinWeightsAsVertexMaskDataInterface* InDataInterface, USkeletalMeshComponent* InSkeletalMesh)
{
	WeakDataInterface = InDataInterface;
	SkeletalMesh = InSkeletalMesh;

	SkinWeightProfile = InDataInterface->SkinWeightProfile;
	BoneNames = InDataInterface->BoneNames;
	ExpandTowardsRoot = InDataInterface->ExpandTowardsRoot;
	ExpandTowardsLeaf = InDataInterface->ExpandTowardsLeaf;
	bDebugDrawIncludedBones = InDataInterface->bDebugDrawIncludedBones;
	DebugDrawColor = InDataInterface->DebugDrawColor;

	// Request profiles early here for all the LODs that can run this deformer so that we can avoid requesting profiles
	// during gameplay or LOD transitions as it can cause T-Posing for a few frames due to GPU readback delay
	Optimus::RequestSkinWeightProfileForDeformer(InSkeletalMesh, SkinWeightProfile);
}

FComputeDataProviderRenderProxy* UOptimusSkinWeightsAsVertexMaskDataProvider::GetRenderProxy()
{
	FOptimusSkinWeightsAsVertexMaskDataProviderProxy* Proxy = new FOptimusSkinWeightsAsVertexMaskDataProviderProxy();

	if (SkeletalMesh.Get() && SkeletalMesh->MeshObject)
	{
		const UOptimusSkinWeightsAsVertexMaskDataInterface* DataInterface = WeakDataInterface.Get();
		UOptimusDeformerInstance* DeformerInstance = WeakDeformerInstance.Get();
		if (DeformerInstance && DataInterface)
		{
			{
				const FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride( DataInterface, UOptimusSkinWeightsAsVertexMaskDataInterface::GetSkinWeightProfilePropertyName() );

				TValueOrError<FName, EPropertyBagResult> Value = ValueContainer.Value.GetValueName(FOptimusValueContainerStruct::ValuePropertyName);
				if (Value.HasValue())
				{
					SkinWeightProfile = Value.GetValue();
				}
			}

			{
				const FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride( DataInterface, UOptimusSkinWeightsAsVertexMaskDataInterface::GetBoneNamesPropertyName());

				TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> Value = ValueContainer.Value.GetArrayRef(FOptimusValueContainerStruct::ValuePropertyName);
				if (Value.HasValue())
				{
					TArray<FName> NewBoneNames;
					NewBoneNames.SetNum(Value.GetValue().Num());
					for (int32 Index = 0 ; Index < NewBoneNames.Num(); Index++)
					{
						NewBoneNames[Index] = Value.GetValue().GetValueName(Index).GetValue();
					}

					if (NewBoneNames != BoneNames)
					{
						bIsInitialized = false;
						BoneNames = MoveTemp(NewBoneNames);
					}
				}
			}

			{
				const FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride( DataInterface, UOptimusSkinWeightsAsVertexMaskDataInterface::GetExpandTowardsRootPropertyName());

				TValueOrError<int32 , EPropertyBagResult> Value = ValueContainer.Value.GetValueInt32(FOptimusValueContainerStruct::ValuePropertyName);
				if (Value.HasValue())
				{
					if (ExpandTowardsRoot != Value.GetValue())
					{
						bIsInitialized = false;
						ExpandTowardsRoot = Value.GetValue();
					}
				}
			}

			{
				const FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride( DataInterface, UOptimusSkinWeightsAsVertexMaskDataInterface::GetExpandTowardsLeafPropertyName());

				TValueOrError<int32 , EPropertyBagResult> Value = ValueContainer.Value.GetValueInt32(FOptimusValueContainerStruct::ValuePropertyName);
				if (Value.HasValue())
				{
					if (ExpandTowardsLeaf != Value.GetValue())
					{
						bIsInitialized = false;
						ExpandTowardsLeaf = Value.GetValue();
					}
				}
			}

			{
				const FOptimusValueContainerStruct ValueContainer =
					DeformerInstance->GetDataInterfacePropertyOverride( DataInterface, UOptimusSkinWeightsAsVertexMaskDataInterface::GetDebugDrawIncludedBonesPropertyName());

				TValueOrError<bool, EPropertyBagResult> Value = ValueContainer.Value.GetValueBool(FOptimusValueContainerStruct::ValuePropertyName);
				if (Value.HasValue())
				{
					if (bDebugDrawIncludedBones != Value.GetValue())
					{
						bDebugDrawIncludedBones = Value.GetValue();
					}
				}
			}	
		}
		
		ExpandTowardsRoot = FMath::Max(0, ExpandTowardsRoot);
		ExpandTowardsLeaf = FMath::Max(0, ExpandTowardsLeaf);
		
		FSkeletalMeshObject* SkeletalMeshObject = SkeletalMesh->MeshObject;
		// Not a good idea to get lod from mesh object, which is from last frame and not updated until later by render thread.
		// So using predicted lod level here, which mirrors the way we get lod during SendRenderDynamicData_Concurrent
		const int32 CurrentLodIndex = SkeletalMesh->GetPredictedLODLevel();
		FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		FSkeletalMeshLODRenderData const& CurrentLodRenderData = SkeletalMeshRenderData.LODRenderData[CurrentLodIndex];

		if (!Optimus::IsSkinWeightProfileAvailable(CurrentLodRenderData, SkinWeightProfile))
		{
			Optimus::RequestSkinWeightProfileForDeformer(SkeletalMesh.Get(), SkinWeightProfile, CurrentLodIndex);
		}
		else
		{
			if (!bIsInitialized)
			{
				bIsInitialized = true;

				CachedSelectedBones.Reset();
				CachedBoneIsSelectedPerSectionPerLod.Reset();
				
				const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetSkinnedAsset()->GetRefSkeleton();
				const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();

				// Negative distance : Towards Root
				// Positive distance : Towards Leaf
				TArray<TOptional<int32>> BoneDistance;
				BoneDistance.SetNum(RefSkeleton.GetNum());

				// Set distance for all parents of the seed bones
				for (const FName& BoneName : BoneNames)
				{
					int32 Index = RefSkeleton.FindBoneIndex(BoneName);

					int32 Distance = 0;
					while (Index != INDEX_NONE && FMath::Abs(Distance) <= ExpandTowardsRoot)
					{
						if (BoneDistance[Index].IsSet() && BoneDistance[Index].GetValue() <= Distance)
						{
							// Skip if the rest of this chain has been visited by a closer seed bone
							break;
						}
							
						CachedSelectedBones.Add(Index);
						BoneDistance[Index] = Distance;
						Distance--;
						Index = RefSkeleton.GetParentIndex(Index);
					}
				}

				// Indices are sorted from parent to children, so distance value can flood from parent to children
				for (int32 Index = 0; Index < RefSkeleton.GetNum(); Index++)
				{
					if (!BoneDistance[Index].IsSet())
					{
						int32 Parent = RefSkeleton.GetParentIndex(Index);
						if (Parent != INDEX_NONE)
						{
							if (BoneDistance[Parent].IsSet())
							{
								int32 ParentDist = BoneDistance[Parent].GetValue();
								
								// Seed bone has to get to this bone by going towards the root
								if (ParentDist < 0)
								{
									if (FMath::Abs(ParentDist) < ExpandTowardsRoot)
									{
										BoneDistance[Index] = ParentDist - 1;
										CachedSelectedBones.Add(Index);
									}
								}
								else
								{
									// Seed bone has to get to this bone by going towards the leaf 
									if (ParentDist < ExpandTowardsLeaf)
									{
										BoneDistance[Index] = ParentDist + 1;
										CachedSelectedBones.Add(Index);
									}	
								}
							}
						}
					}
				}

				for (int32 LODIndex = 0; LODIndex < SkeletalMeshRenderData.LODRenderData.Num(); LODIndex++)
				{
					FSkeletalMeshLODRenderData const& LocalLodRenderData = SkeletalMeshRenderData.LODRenderData[LODIndex];
					
					TArray<TArray<uint32>>& CachedBoneIsSelectedPerSectionRef = CachedBoneIsSelectedPerSectionPerLod.AddDefaulted_GetRef();
					for (int32 SectionIndex = 0; SectionIndex < LocalLodRenderData.RenderSections.Num(); ++SectionIndex)
					{
						TArray<uint32>& BoneIsSelectedRef = CachedBoneIsSelectedPerSectionRef.AddDefaulted_GetRef();
				
						FSkelMeshRenderSection const& RenderSection = LocalLodRenderData.RenderSections[SectionIndex];
						const int32 NumBones = RenderSection.BoneMap.Num();
						BoneIsSelectedRef.Init(false, NumBones);

						for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
						{
							int32 FinalBoneIndex = RenderSection.BoneMap[BoneIndex];
							if (CachedSelectedBones.Contains(FinalBoneIndex))
							{
								BoneIsSelectedRef[BoneIndex] = true;
							}
						}

						// Only process first section when unified.
						if (bUnifiedBoneMap)
						{
							break;
						}
					}

					// Unified bone map is cross-LOD, so LOD 0 / Section 0 covers all bones. Break the LOD loop too.
					if (bUnifiedBoneMap)
					{
						break;
					}
				}
			}

			if (bDebugDrawIncludedBones)
			{
#if WITH_EDITOR
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [ WeakThis = TWeakObjectPtr(this)]()
					{
						if (WeakThis.IsValid())
						{
							if (USkeletalMeshComponent* Mesh = WeakThis->SkeletalMesh.Get())
							{
								const FReferenceSkeleton& RefSkeleton = Mesh->GetSkinnedAsset()->GetRefSkeleton();

								const TArray<FTransform>& CurrentComponentSpaceBoneMatrix = Mesh->GetComponentSpaceTransforms();

								for (int32 Index = 0; Index < CurrentComponentSpaceBoneMatrix.Num(); Index++)
								{
									int32 ParentIndex = RefSkeleton.GetParentIndex(Index);
									if (ParentIndex != INDEX_NONE && WeakThis->CachedSelectedBones.Contains(ParentIndex))
									{
										const FTransform& Child = CurrentComponentSpaceBoneMatrix[Index] * Mesh->GetComponentTransform();
										const FTransform& Parent = CurrentComponentSpaceBoneMatrix[ParentIndex] * Mesh->GetComponentTransform();
										DrawDebugLine(Mesh->GetWorld(), Child.GetLocation(), Parent.GetLocation(), WeakThis->DebugDrawColor, false, -1, SDPG_Foreground);
									}
								}
							}
						}
					});	
#endif
			}
			
			
			if (SkeletalMesh.IsValid() && SkeletalMesh->IsRegistered())
			{
				Proxy->SceneInterface = SkeletalMesh->GetScene();
				Proxy->ComponentId = SkeletalMesh->GetPrimitiveSceneId();
			}
			Proxy->SkinWeightProfile = SkinWeightProfile;
			Proxy->BoneIsSelectedPerSectionPerLod = CachedBoneIsSelectedPerSectionPerLod;
		}
	}

	return Proxy;
}



bool FOptimusSkinWeightsAsVertexMaskDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	// This data interface does not support unified dispatch (because bone maps can be per-section) 
	// so the invocation count has to match the render sections
	if (LodRenderData->RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}
	
	FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
	if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
	{
		const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
		WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
	}
	
	if (WeightBuffer == nullptr)
	{
		return false;
	}

	if (BoneIsSelectedPerSectionPerLod.IsEmpty())
	{
		return false;
	}

	return true;	
}

void FOptimusSkinWeightsAsVertexMaskDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();
	const int32 NumUniqueSections = bUnifiedBoneMap ? 1 : LodRenderData->RenderSections.Num();

	BoneIsSelectedBuffersPerSection.AddDefaulted(NumUniqueSections);
	BoneIsSelectedBufferSRVsPerSection.AddDefaulted(NumUniqueSections);

	for (int32 InvocationIndex = 0 ; InvocationIndex < NumUniqueSections; InvocationIndex++)
	{
		const TArray<uint32>& BoneIsSelected = bUnifiedBoneMap ? BoneIsSelectedPerSectionPerLod[0][0] : BoneIsSelectedPerSectionPerLod[LodIndex][InvocationIndex];

		// When using unified bone map, the BoneIsSelected array was built from LOD 0's bone map.
		// The current LOD may have a different bone count. Use the smaller of the two to avoid OOB.
		int32 NumBones = FMath::Min(LodRenderData->RenderSections[InvocationIndex].BoneMap.Num(), BoneIsSelected.Num());

		int32 Stride = sizeof(uint32);
		int32 ByteSize = NumBones * Stride;
		
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(Stride,  NumBones), TEXT("BoneIsSelectedBuffer"), ERDGBufferFlags::None);
		FRDGBufferSRVRef BufferSRV = GraphBuilder.CreateSRV(Buffer);

		BoneIsSelectedBuffersPerSection[InvocationIndex] = Buffer;
		BoneIsSelectedBufferSRVsPerSection[InvocationIndex] = BufferSRV;

		GraphBuilder.QueueBufferUpload(Buffer, BoneIsSelected.GetData(), ByteSize, ERDGInitialDataFlags::None);	
	}
}

struct FSkinWeightsAsVertexMaskDataInterfacePermutationIds
{
	uint32 EnableDeformerBones = 0;
	uint32 UnlimitedBoneInfluence = 0;
	uint32 BoneIndexUint16 = 0;
	uint32 BoneWeightsUint16 = 0;

	FSkinWeightsAsVertexMaskDataInterfacePermutationIds(FComputeKernelPermutationVector const& PermutationVector)
	{
		{
			static FString Name(TEXT("ENABLE_DEFORMER_BONES"));
			static uint32 Hash = GetTypeHash(Name);
			EnableDeformerBones = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_UNLIMITED_BONE_INFLUENCE"));
			static uint32 Hash = GetTypeHash(Name);
			UnlimitedBoneInfluence = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_INDEX_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneIndexUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
		{
			static FString Name(TEXT("GPUSKIN_BONE_WEIGHTS_UINT16"));
			static uint32 Hash = GetTypeHash(Name);
			BoneWeightsUint16 = PermutationVector.GetPermutationBits(Name, Hash, 1);
		}
	}
};

void FOptimusSkinWeightsAsVertexMaskDataProviderProxy::GatherPermutations(FPermutationData& InOutPermutationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FSkinWeightsAsVertexMaskDataInterfacePermutationIds PermutationIds(InOutPermutationData.PermutationVector);
	for (int32 InvocationIndex = 0; InvocationIndex < InOutPermutationData.NumInvocations; ++InvocationIndex)
	{
		const bool bPreviousFrame = false;
		FRHIShaderResourceView* BoneBufferSRV = FSkeletalMeshDeformerHelpers::GetBoneBufferForReading(SkeletalMeshObject, LodIndex, InvocationIndex, bPreviousFrame);
		
		FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
		if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
		{
			const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
			WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
		}
		
		check(WeightBuffer != nullptr);
		FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
		const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
		FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;
		const bool bValidBones = (BoneBufferSRV != nullptr) && (SkinWeightBufferSRV != nullptr) && (!bUnlimitedBoneInfluences || InputWeightLookupStreamSRV != nullptr);
		const bool bUse16BitBoneIndex = WeightBuffer->Use16BitBoneIndex();
		const bool bUse16BitBoneWeights = WeightBuffer->Use16BitBoneWeight();

		InOutPermutationData.PermutationIds[InvocationIndex] |= (bValidBones ? PermutationIds.EnableDeformerBones : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUnlimitedBoneInfluences ? PermutationIds.UnlimitedBoneInfluence : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneIndex ? PermutationIds.BoneIndexUint16 : 0);
		InOutPermutationData.PermutationIds[InvocationIndex] |= (bUse16BitBoneWeights ? PermutationIds.BoneWeightsUint16 : 0);
	}
}

void FOptimusSkinWeightsAsVertexMaskDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	const bool bUnifiedBoneMap = SkeletalMeshRenderData.HasUnifiedBoneMap();

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();
	
	FSkinWeightVertexBuffer const* WeightBuffer = LodRenderData->GetSkinWeightVertexBuffer();
	if (LodRenderData->SkinWeightProfilesData.ContainsProfile(SkinWeightProfile))
	{
		const FSkinWeightProfileStack ProfileStack{SkinWeightProfile};
		WeightBuffer = LodRenderData->SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
	}
	check(WeightBuffer != nullptr);
	FRHIShaderResourceView* SkinWeightBufferSRV = WeightBuffer->GetDataVertexBuffer()->GetSRV();
	const bool bUnlimitedBoneInfluences = WeightBuffer->GetBoneInfluenceType() == GPUSkinBoneInfluenceType::UnlimitedBoneInfluence;
	FRHIShaderResourceView* InputWeightLookupStreamSRV = bUnlimitedBoneInfluences ? WeightBuffer->GetLookupVertexBuffer()->GetSRV() : nullptr;

	
	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < InDispatchData.NumInvocations; ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumBoneInfluences = WeightBuffer->GetMaxBoneInfluences();
		Parameters.InputWeightStride = WeightBuffer->GetConstantInfluencesVertexStride();
		Parameters.InputWeightIndexSize = WeightBuffer->GetBoneIndexByteSize() | (WeightBuffer->GetBoneWeightByteSize() << 8);
		Parameters.InputWeightStream = SkinWeightBufferSRV != nullptr ? SkinWeightBufferSRV : NullSRVBinding;
		Parameters.InputWeightLookupStream = InputWeightLookupStreamSRV != nullptr ? InputWeightLookupStreamSRV : NullSRVBinding;
		Parameters.BoneIsSelected = BoneIsSelectedBufferSRVsPerSection[bUnifiedBoneMap ? 0 : InvocationIndex];
	}
}
