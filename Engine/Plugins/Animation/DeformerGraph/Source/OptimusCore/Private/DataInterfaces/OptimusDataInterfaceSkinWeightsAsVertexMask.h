// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDataType.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceSkinWeightsAsVertexMask.generated.h"

#define UE_API OPTIMUSCORE_API

class FSceneInterface;
class FSkinWeightsAsVertexMaskDataInterfaceParameters;
class FSkeletalMeshObject;
class USkeletalMeshComponent;
class FSkeletalMeshLODRenderData;
struct FReferenceSkeleton;


/** Compute Framework Data Interface for merging skin weights of one or more bones into a per-vertex float map. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusSkinWeightsAsVertexMaskDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()
	
public:
	
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;

	UE_API void RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode) override;
	//~ End UOptimusComputeDataInterface Interface
	

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkinWeightMask"); }
	bool CanSupportUnifiedDispatch() const override {return false;};
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface
	
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	FName SkinWeightProfile = NAME_None;

	// Weights of selected bones are combined to form a single a vertex weight map 
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	TArray<FName> BoneNames = {TEXT("Root")};
	
	// Include all bones within the radius by expanding selection towards the root
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	int32 ExpandTowardsRoot = 0;

	// Include children up to the specified depth
	UPROPERTY(EditAnywhere, Category = "Data Interface")
	int32 ExpandTowardsLeaf = 999;

	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bDebugDrawIncludedBones = true;

	UPROPERTY(EditAnywhere, Category = "Debug")
	FColor DebugDrawColor = FColor::Green;
	
private:
	friend class UOptimusSkinWeightsAsVertexMaskDataProvider;
	static UE_API FName GetSkinWeightProfilePropertyName();
	static UE_API FName GetBoneNamesPropertyName();
	static UE_API FName GetExpandTowardsRootPropertyName();
	static UE_API FName GetExpandTowardsLeafPropertyName();
	static UE_API FName GetDebugDrawIncludedBonesPropertyName();
	
	static UE_API TCHAR const* TemplateFilePath;
	FOnPinDefinitionChanged OnPinDefinitionChangedDelegate;
	FOnPinDefinitionRenamed OnPinDefinitionRenamedDelegate;
	FSimpleDelegate OnDisplayNameChangedDelegate;
};


/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkinWeightsAsVertexMaskDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor

{
	GENERATED_BODY()

public:
	void Init(const UOptimusSkinWeightsAsVertexMaskDataInterface* InDataInterface, USkeletalMeshComponent* InSkeletalMesh);
	
	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;

	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMesh = nullptr;

	FName SkinWeightProfile = NAME_None;
	
	TArray<FName> BoneNames;
	int32 ExpandTowardsRoot = 0;
	int32 ExpandTowardsLeaf = 0;
	bool bDebugDrawIncludedBones = false;
	FColor DebugDrawColor = FColor::Green;

	bool bIsInitialized = false;
	TSet<int32> CachedSelectedBones;
	TArray<TArray<TArray<uint32>>> CachedBoneIsSelectedPerSectionPerLod;
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface


private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance;

	TWeakObjectPtr<const UOptimusSkinWeightsAsVertexMaskDataInterface> WeakDataInterface;
};

class FOptimusSkinWeightsAsVertexMaskDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSkinWeightsAsVertexMaskDataProviderProxy() = default;

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherPermutations(FPermutationData& InOutPermutationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

	using FParameters = FSkinWeightsAsVertexMaskDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;

	FName SkinWeightProfile = NAME_None;

	TArray<TArray<TArray<uint32>>> BoneIsSelectedPerSectionPerLod;
	TArray<FRDGBufferRef> BoneIsSelectedBuffersPerSection;
	TArray<FRDGBufferSRVRef> BoneIsSelectedBufferSRVsPerSection;
};

#undef UE_API
