// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinnedMesh.h"

#include "Components/SkinnedMeshComponent.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinnedMesh)


FString UOptimusSkinnedMeshDataInterface::GetDisplayName() const
{
	return TEXT("Skinned Mesh");
}

TArray<FOptimusCDIPinDefinition> UOptimusSkinnedMeshDataInterface::GetPinDefinitions() const
{
	FName Vertex(UOptimusSkinnedMeshComponentSource::Domains::Vertex);
	FName Triangle(UOptimusSkinnedMeshComponentSource::Domains::Triangle);
	
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumVertices", "ReadNumVertices", false});
	Defs.Add({"Position", "ReadPosition", Vertex, "ReadNumVertices", false});
	Defs.Add({"TangentX", "ReadTangentX", Vertex, "ReadNumVertices", false});
	Defs.Add({"TangentZ", "ReadTangentZ", Vertex, "ReadNumVertices", false});
	Defs.Add({"NumUVChannels", "ReadNumUVChannels", false});
	Defs.Add({"UV", "ReadUV", {{Vertex, "ReadNumVertices"}, {Optimus::DomainName::UVChannel, "ReadNumUVChannels"}}, false});
	Defs.Add({"Color", "ReadColor", Vertex, "ReadColor", false});
	Defs.Add({"NumTriangles", "ReadNumTriangles", false});
	Defs.Add({"IndexBuffer", "ReadIndexBuffer", Triangle, 3, "ReadNumTriangles", false});
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkinnedMeshDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusSkinnedMeshDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumTriangles"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumUVChannels"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadIndexBuffer"))
		.AddReturnType(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTangentX"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTangentZ"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadUV"))
		.AddReturnType(EShaderFundamentalType::Float, 2)
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadColor"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkinnedMeshDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, NumTriangles)
	SHADER_PARAMETER(uint32, NumUVChannels)
	SHADER_PARAMETER_SRV(Buffer<uint>, IndexBuffer)
	SHADER_PARAMETER_SRV(Buffer<float>, PositionInputBuffer)
	SHADER_PARAMETER_SRV(Buffer<SNORM float4>, TangentInputBuffer)
	SHADER_PARAMETER_SRV(Buffer<float2>, UVInputBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, ColorInputBuffer)
	SHADER_PARAMETER(uint32, ColorIndexMask)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkinnedMeshDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkinnedMeshDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusSkinnedMeshDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMesh.ush");

TCHAR const* UOptimusSkinnedMeshDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSkinnedMeshDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSkinnedMeshDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSkinnedMeshDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSkinnedMeshDataProvider* Provider = NewObject<UOptimusSkinnedMeshDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusSkinnedMeshDataProvider::GetRenderProxy()
{
	return new FOptimusSkinnedMeshDataProviderProxy(SkinnedMesh.Get());
}


FOptimusSkinnedMeshDataProviderProxy::FOptimusSkinnedMeshDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent)
{
	if (SkinnedMeshComponent && SkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = SkinnedMeshComponent->GetScene();
		ComponentId = SkinnedMeshComponent->GetPrimitiveSceneId();
	}
}

bool FOptimusSkinnedMeshDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return true;
}

void FOptimusSkinnedMeshDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRawStaticIndexBuffer16or32Interface* IndexBufferInterface = LodRenderData->MultiSizeIndexContainer.GetIndexBuffer();
	
	IndexBufferSRV = IndexBufferInterface->GetOrCreateSRV(GraphBuilder.RHICmdList);
	TangentsSRV = LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetOrCreateTangentsSRV(GraphBuilder.RHICmdList);	
	TexCoordsSRV = LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetOrCreateTexCoordsSRV(GraphBuilder.RHICmdList);
	
	// Color SRV is either created by default or not available at all
	ColorsSRV = LodRenderData->StaticVertexBuffers.ColorVertexBuffer.GetColorComponentsSRV();
}

void FOptimusSkinnedMeshDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FRHIShaderResourceView* MeshIndexBufferSRV = IndexBufferSRV;
		FRHIShaderResourceView* MeshVertexBufferSRV = LodRenderData->StaticVertexBuffers.PositionVertexBuffer.GetSRV();
		FRHIShaderResourceView* MeshTangentBufferSRV = TangentsSRV;
		FRHIShaderResourceView* MeshUVBufferSRV = TexCoordsSRV;
		FRHIShaderResourceView* MeshColorBufferSRV = ColorsSRV;

		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.NumTriangles =  LodRenderData->GetTotalFaces();
		Parameters.NumUVChannels = LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
		Parameters.IndexBuffer = MeshIndexBufferSRV != nullptr ? MeshIndexBufferSRV : NullSRVBinding;
		Parameters.PositionInputBuffer = MeshVertexBufferSRV != nullptr ? MeshVertexBufferSRV : NullSRVBinding;
		Parameters.TangentInputBuffer = MeshTangentBufferSRV != nullptr ? MeshTangentBufferSRV : NullSRVBinding;
		Parameters.UVInputBuffer = MeshUVBufferSRV != nullptr ? MeshUVBufferSRV : NullSRVBinding;
		Parameters.ColorInputBuffer = MeshColorBufferSRV != nullptr ? MeshColorBufferSRV : NullSRVBinding;
		
		// Basically when we are accessing GWhiteVertexBufferWithSRV(NullSRVBinding),
		// we should not access beyond index 0 since the buffer is only a few bytes
		
		// See FGPUSkinPassthroughVertexFactory::UpdateUniformBuffer() and LocalVertexFactory.ush :: GetVertexFactoryIntermediates()
		// Ideally we should be getting this value from the GPUBaseSkinVertexFactory but the need for
		// section index make it tricky when we are doing unified dispatch
		Parameters.ColorIndexMask = MeshColorBufferSRV != nullptr ? ~0u : 0;
	}
}
