// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "RHIFwd.h"

#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceSkinnedMesh.generated.h"

#define UE_API OPTIMUSCORE_API

class FSceneInterface;
class FSkeletalMeshObject;
class FSkinnedMeshDataInterfaceParameters;
class USkinnedMeshComponent;

/** Compute Framework Data Interface for reading data from the skeletal mesh asset */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusSkinnedMeshDataInterface : public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkinnedMesh"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkinnedMeshDataProvider : public UComputeDataProvider
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

class FOptimusSkinnedMeshDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSkinnedMeshDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent);

	//~ Begin FComputeDataProviderRenderProxy Interface
	virtual bool IsValid(FValidationData const& InValidationData) const override;
	virtual void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	virtual void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSkinnedMeshDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;

	FShaderResourceViewRHIRef IndexBufferSRV;
	FShaderResourceViewRHIRef TangentsSRV;
	FShaderResourceViewRHIRef TexCoordsSRV;
	FShaderResourceViewRHIRef ColorsSRV;
};

#undef UE_API
