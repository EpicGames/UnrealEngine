// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinnedMeshExec.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinnedMeshExec)

FString UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetDisplayName() const
{
	return TEXT("Execute Skinned Mesh");
}

FName UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetCategory() const
{
	return CategoryName::ExecutionDataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "NumThreads", "ReadNumThreads" });
	return Defs;
}


TSubclassOf<UActorComponent> UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumThreads"))
		.AddReturnType(EShaderFundamentalType::Int, 3);
}


BEGIN_SHADER_PARAMETER_STRUCT(FSkinedMeshExecDataInterfaceParameters, )
	SHADER_PARAMETER(FIntVector, NumThreads)
END_SHADER_PARAMETER_STRUCT()

void UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkinedMeshExecDataInterfaceParameters>(UID);
}

TCHAR const* UDEPRECATED_OptimusSkinnedMeshExecDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMeshExec.ush");

TCHAR const* UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UDEPRECATED_OptimusSkinnedMeshExecDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	return nullptr;
}

FName UDEPRECATED_OptimusSkinnedMeshExecDataInterface::GetSelectedExecutionDomainName() const
{
	if (Domain == EOptimusSkinnedMeshExecDomain::Vertex)
	{
		return UOptimusSkinnedMeshComponentSource::Domains::Vertex;
	}

	if (Domain == EOptimusSkinnedMeshExecDomain::Triangle)
	{
		return UOptimusSkinnedMeshComponentSource::Domains::Triangle;
	}

	return NAME_None;
}

