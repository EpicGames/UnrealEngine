// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeprecatedExecutionDataInterface.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"

#include "OptimusDataInterfaceSkinnedMeshExec.generated.h"

#define UE_API OPTIMUSCORE_API

class FRDGBuffer;
class FRDGBufferUAV;
class FSkeletalMeshObject;
class FSkinedMeshExecDataInterfaceParameters;
class USkinnedMeshComponent;

UENUM()
enum class EOptimusSkinnedMeshExecDomain : uint8
{
	None = 0 UMETA(Hidden),
	/** Run kernel with one thread per vertex. */
	Vertex = 1,
	/** Run kernel with one thread per triangle. */
	Triangle,
};

/** Compute Framework Data Interface for executing kernels over a skinned mesh domain. */
UCLASS(MinimalAPI, Deprecated, Category = ComputeFramework, meta = (DeprecationMessage ="This execution interface has been replaced with kernel specific execution data interface, see UOptimusCustomComputeKernelDataInterface"))
class UDEPRECATED_OptimusSkinnedMeshExecDataInterface :
	public UOptimusComputeDataInterface,
	public IOptimusDeprecatedExecutionDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API FName GetCategory() const override;
	bool IsVisible() const override {return false;};
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkinnedMeshExec"); }
	bool IsExecutionInterface() const override { return true; }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	//~ Begin IOptimusDeprecatedExecutionDataInterface Interface
	UE_API FName GetSelectedExecutionDomainName() const override;
	//~ End IOptimusDeprecatedExecutionDataInterface Interface
	
	UPROPERTY(EditAnywhere, Category = Execution)
	EOptimusSkinnedMeshExecDomain Domain = EOptimusSkinnedMeshExecDomain::Vertex;

private:
	static UE_API TCHAR const* TemplateFilePath;
};


#undef UE_API
