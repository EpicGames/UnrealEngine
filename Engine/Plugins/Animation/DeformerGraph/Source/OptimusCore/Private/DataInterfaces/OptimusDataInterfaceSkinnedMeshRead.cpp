// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinnedMeshRead.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "OptimusDeformerInstance.h"
#include "OptimusSkinnedMeshTrackingExtension.h"
#include "Animation/MeshDeformerInstance.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinnedMeshRead)

FString UOptimusSkinnedMeshReadDataInterface::GetDisplayName() const
{
	return TEXT("Read Skinned Mesh");
}

FName UOptimusSkinnedMeshReadDataInterface::GetCategory() const
{
	return CategoryName::DataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusSkinnedMeshReadDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "Position", "ReadPosition", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "TangentX", "ReadTangentX", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "TangentZ", "ReadTangentZ", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "Color", "ReadColor", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkinnedMeshReadDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}

// Should be kept in sync with GetSupportedInput
enum class ESkinnedMeshReadDataInterfaceInputSelectorMask : uint64
{
	NumVertices = 1 << 0,
	Position = 1 << 1,
	TangentX = 1 << 2,
	TangentZ = 1 << 3,
	Color = 1 << 4,
};
ENUM_CLASS_FLAGS(ESkinnedMeshReadDataInterfaceInputSelectorMask);

void UOptimusSkinnedMeshReadDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);
	
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPosition"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTangentX"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadTangentZ"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadColor"))
		.AddReturnType(EShaderFundamentalType::Float, 4)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkinnedMeshReadDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, PositionBufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<SNORM float4>, TangentBufferSRV)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<UNORM float4>, ColorBufferSRV)
	SHADER_PARAMETER_SRV(Buffer<float>, PositionStaticBuffer)
	SHADER_PARAMETER_SRV(Buffer<SNORM float4>, TangentStaticBuffer)
	SHADER_PARAMETER_SRV(Buffer<float4>, ColorStaticBuffer)
	SHADER_PARAMETER(uint32, ColorIndexMask)
	SHADER_PARAMETER(uint32, ReadableOutputBuffers)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkinnedMeshReadDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkinnedMeshReadDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusSkinnedMeshReadDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMeshRead.ush");

TCHAR const* UOptimusSkinnedMeshReadDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSkinnedMeshReadDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSkinnedMeshReadDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

void UOptimusSkinnedMeshReadDataInterface::GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const
{
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(
		TEXT("OPTIMUS_SKINNED_MESH_READ_POSITION"),
		FString::FromInt(static_cast<uint32>(EMeshDeformerOutputBuffer::SkinnedMeshPosition))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(
		TEXT("OPTIMUS_SKINNED_MESH_READ_TANGENTS"),
		FString::FromInt(static_cast<uint32>(EMeshDeformerOutputBuffer::SkinnedMeshTangents))));
	OutDefinitionSet.Defines.Add(FComputeKernelDefinition(
		TEXT("OPTIMUS_SKINNED_MESH_READ_COLOR"),
		FString::FromInt(static_cast<uint32>(EMeshDeformerOutputBuffer::SkinnedMeshVertexColor))));
}

UComputeDataProvider* UOptimusSkinnedMeshReadDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSkinnedMeshReadDataProvider* Provider = NewObject<UOptimusSkinnedMeshReadDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	Provider->InputMask = InInputMask;
	return Provider;
}

FComputeDataProviderRenderProxy* UOptimusSkinnedMeshReadDataProvider::GetRenderProxy()
{
	bool bIsPrimary = true;
	FName ExecGroup;

	if (WeakDeformerInstance.IsValid())
	{
		UOptimusDeformerInstance* Instance = WeakDeformerInstance.Get();
		bIsPrimary = (SkinnedMesh.Get() == Instance->GetMeshComponent());
		ExecGroup = Instance->CurrentExecutionGroupName;
	}

	return new FOptimusSkinnedMeshReadDataProviderProxy(SkinnedMesh.Get(), InputMask, bIsPrimary, ExecGroup);
}

void UOptimusSkinnedMeshReadDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}

FOptimusSkinnedMeshReadDataProviderProxy::FOptimusSkinnedMeshReadDataProviderProxy(USkinnedMeshComponent* InSkinnedMeshComponent, uint64 InInputMask, bool bInIsPrimaryComponent, FName InReaderExecutionGroup)
{
	if (InSkinnedMeshComponent && InSkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = InSkinnedMeshComponent->GetScene();
		ComponentId = InSkinnedMeshComponent->GetPrimitiveSceneId();
	}
	InputMask = InInputMask;
	bIsPrimaryComponent = bInIsPrimaryComponent;
	ReaderExecutionGroup = InReaderExecutionGroup;
}

bool FOptimusSkinnedMeshReadDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}

	if (FSkeletalMeshDeformerHelpers::GetIndexOfFirstAvailableSection(SkeletalMeshObject, SkeletalMeshObject->GetLOD()) == INDEX_NONE)
	{
		return false;
	}

	return true;
}

void FOptimusSkinnedMeshReadDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	
	TangentsSRV = LodRenderData->StaticVertexBuffers.StaticMeshVertexBuffer.GetOrCreateTangentsSRV(GraphBuilder.RHICmdList);
	
	// Color SRV is either created by default or not available at all
	ColorsSRV = LodRenderData->StaticVertexBuffers.ColorVertexBuffer.GetColorComponentsSRV();

	// Try to get deformed buffers. If they exist (allocated by a write proxy), create SRV from them.
	// If not, fall back to a white buffer. The shader will use ReadableOutputBuffers
	// (set in GatherDispatchData) to decide which buffer to actually read from at runtime.
	FRDGBuffer* White = GraphBuilder.RegisterExternalBuffer(GWhiteVertexBufferWithRDG->Buffer);

	FRDGBuffer* DeformedPositionBuffer = (InputMask & static_cast<uint64>(ESkinnedMeshReadDataInterfaceInputSelectorMask::Position))
		? FSkeletalMeshDeformerHelpers::GetAllocatedPositionBuffer(GraphBuilder, SkeletalMeshObject, LodIndex)
		: nullptr;
	PositionBufferSRV = GraphBuilder.CreateSRV(DeformedPositionBuffer ? DeformedPositionBuffer : White, PF_R32_FLOAT);

	// OpenGL ES does not support writing to RGBA16_SNORM images, instead pack data into SINT in the shader
	const EPixelFormat TangentsFormat = IsOpenGLPlatform(GMaxRHIShaderPlatform) ? PF_R16G16B16A16_SINT : PF_R16G16B16A16_SNORM;

	FRDGBuffer* DeformedTangentBuffer = ((InputMask & static_cast<uint64>(ESkinnedMeshReadDataInterfaceInputSelectorMask::TangentX)) ||
		(InputMask & static_cast<uint64>(ESkinnedMeshReadDataInterfaceInputSelectorMask::TangentZ)))
		? FSkeletalMeshDeformerHelpers::GetAllocatedTangentBuffer(GraphBuilder, SkeletalMeshObject, LodIndex)
		: nullptr;
	TangentBufferSRV = GraphBuilder.CreateSRV(DeformedTangentBuffer ? DeformedTangentBuffer : White, TangentsFormat);

	FRDGBuffer* DeformedColorBuffer = (InputMask & static_cast<uint64>(ESkinnedMeshReadDataInterfaceInputSelectorMask::Color))
		? FSkeletalMeshDeformerHelpers::GetAllocatedColorBuffer(GraphBuilder, SkeletalMeshObject, LodIndex)
		: nullptr;
	ColorBufferSRV = GraphBuilder.CreateSRV(DeformedColorBuffer ? DeformedColorBuffer : White, PF_R8G8B8A8);
}

void FOptimusSkinnedMeshReadDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	// Query tracking extension for which deformed buffers are readable at this point in dispatch order.
	// SkinnedMeshWrite's PostSubmit calls MarkDeformed after each kernel dispatch in sorted order,
	// so this sees only outputs from kernels that have already dispatched.
	const EMeshDeformerOutputBuffer ReadableBuffers = FOptimusSkinnedMeshTrackingExtension::GetReadableDeformedBuffers(
		SceneInterface, ComponentId, ReaderExecutionGroup, bIsPrimaryComponent);

	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();

		Parameters.PositionBufferSRV = PositionBufferSRV;
		Parameters.TangentBufferSRV = TangentBufferSRV;
		Parameters.ColorBufferSRV = ColorBufferSRV;
		
		FRHIShaderResourceView* MeshVertexBufferSRV = LodRenderData->StaticVertexBuffers.PositionVertexBuffer.GetSRV();
		FRHIShaderResourceView* MeshTangentBufferSRV = TangentsSRV;
		FRHIShaderResourceView* MeshColorBufferSRV = ColorsSRV;

		Parameters.PositionStaticBuffer = MeshVertexBufferSRV != nullptr ? MeshVertexBufferSRV : NullSRVBinding;
		Parameters.TangentStaticBuffer = MeshTangentBufferSRV != nullptr ? MeshTangentBufferSRV : NullSRVBinding;
		Parameters.ColorStaticBuffer = MeshColorBufferSRV != nullptr ? MeshColorBufferSRV : NullSRVBinding;
		
		// Basically when we are accessing GWhiteVertexBufferWithSRV(NullSRVBinding),
		// we should not access beyond index 0 since the buffer is only a few bytes
		
		// See FGPUSkinPassthroughVertexFactory::UpdateUniformBuffer() and LocalVertexFactory.ush :: GetVertexFactoryIntermediates()
		// Ideally we should be getting this value from the GPUBaseSkinVertexFactory but the need for
		// section index make it tricky when we are doing unified dispatch
		Parameters.ColorIndexMask = MeshColorBufferSRV != nullptr ? ~0u : 0;

		Parameters.ReadableOutputBuffers = static_cast<uint32>(ReadableBuffers);;
	}
}
