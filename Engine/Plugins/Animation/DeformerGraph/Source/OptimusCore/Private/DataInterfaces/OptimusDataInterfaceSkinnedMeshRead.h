// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "RHIFwd.h"

#include "OptimusDataInterfaceSkinnedMeshRead.generated.h"

#define UE_API OPTIMUSCORE_API

class FRDGBuffer;
class FRDGBufferSRV;
class FSceneInterface;
class FSkeletalMeshObject;
class FSkinnedMeshReadDataInterfaceParameters;
class USkinnedMeshComponent;
enum class EMeshDeformerOutputBuffer : uint8;

/** Compute Framework Data Interface for reading the current state of skinned mesh, which may have been deformed by deformers run earlier */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusSkinnedMeshReadDataInterface :
	public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API FName GetCategory() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkinnedMeshRead"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API void GetDefines(FComputeKernelDefinitionSet& OutDefinitionSet) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for writing skinned mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkinnedMeshReadDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	uint64 InputMask = 0;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	//~ Begin IOptimusDeformerInstanceAccessor Interface
	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;
	//~ End IOptimusDeformerInstanceAccessor Interface

private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance = nullptr;
};

class FOptimusSkinnedMeshReadDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSkinnedMeshReadDataProviderProxy(USkinnedMeshComponent* InSkinnedMeshComponent, uint64 InInputMask, bool bInIsPrimaryComponent, FName InReaderExecutionGroup);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSkinnedMeshReadDataInterfaceParameters;

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	uint64 InputMask = 0;
	bool bIsPrimaryComponent = true;
	FName ReaderExecutionGroup;

	FShaderResourceViewRHIRef TangentsSRV;
	FShaderResourceViewRHIRef ColorsSRV;
	
	FRDGBufferSRV* PositionBufferSRV = nullptr;
	FRDGBufferSRV* TangentBufferSRV = nullptr;
	FRDGBufferSRV* ColorBufferSRV = nullptr;
};

#undef UE_API
