// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinnedMeshVertexAttribute.h"

#include "Components/SkinnedMeshComponent.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusValueContainerStruct.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Rendering/SkeletalMeshAttributeVertexBuffer.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinnedMeshVertexAttribute)

FName UOptimusSkinnedMeshVertexAttributeDataInterface::GetAttributeNamePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinnedMeshVertexAttributeDataInterface, AttributeName);
}

FName UOptimusSkinnedMeshVertexAttributeDataInterface::GetDefaultValuePropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UOptimusSkinnedMeshVertexAttributeDataInterface, DefaultValue);
}

FString UOptimusSkinnedMeshVertexAttributeDataInterface::GetDisplayName() const
{
	return TEXT("Skinned Mesh Vertex Attribute");
}


TArray<FOptimusCDIPinDefinition> UOptimusSkinnedMeshVertexAttributeDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({"NumVertices", "ReadNumVertices", false});
	Defs.Add({"Value", "ReadValue", UOptimusSkinnedMeshComponentSource::Domains::Vertex, "ReadNumVertices", false});
	return Defs;
}

TArray<FOptimusCDIPropertyPinDefinition> UOptimusSkinnedMeshVertexAttributeDataInterface::GetPropertyPinDefinitions() const
{
	TArray<FOptimusCDIPropertyPinDefinition> Defs;
	const FOptimusDataTypeHandle NameType = FOptimusDataTypeRegistry::Get().FindType(*FNameProperty::StaticClass());
	Defs.Add({GetAttributeNamePropertyName(), NameType});
	Defs.Add({GetDefaultValuePropertyName(), FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass())});

	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkinnedMeshVertexAttributeDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusSkinnedMeshVertexAttributeDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadValue"))
		.AddReturnType(EShaderFundamentalType::Float)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkinnedMeshVertexAttributeDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(uint32, bIsValid)
	SHADER_PARAMETER(float, DefaultValue)
	SHADER_PARAMETER_SRV(Buffer<float>, ValueBuffer)
END_SHADER_PARAMETER_STRUCT()


void UOptimusSkinnedMeshVertexAttributeDataInterface::GetShaderParameters(
	TCHAR const* InUID,
	FShaderParametersMetadataBuilder& InOutBuilder, 
	FShaderParametersMetadataAllocations& InOutAllocations
	) const
{
	InOutBuilder.AddNestedStruct<FSkinnedMeshVertexAttributeDataInterfaceParameters>(InUID);
}


void UOptimusSkinnedMeshVertexAttributeDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMeshVertexAttribute.ush"), EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}


void UOptimusSkinnedMeshVertexAttributeDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMeshVertexAttribute.ush"), EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}


UComputeDataProvider* UOptimusSkinnedMeshVertexAttributeDataInterface::CreateDataProvider(
	TObjectPtr<UObject> InBinding,
	uint64 InInputMask,
	uint64 InOutputMask
	) const
{
	UOptimusSkinnedMeshVertexAttributeDataProvider* Provider = NewObject<UOptimusSkinnedMeshVertexAttributeDataProvider>();
	Provider->SkinnedMeshComponent = Cast<USkinnedMeshComponent>(InBinding);
	Provider->AttributeName = AttributeName;
	Provider->DefaultValue = DefaultValue;
	Provider->WeakDataInterface = this;

	return Provider;
}


FComputeDataProviderRenderProxy* UOptimusSkinnedMeshVertexAttributeDataProvider::GetRenderProxy()
{
	const UOptimusSkinnedMeshVertexAttributeDataInterface* DataInterface = WeakDataInterface.Get();
	UOptimusDeformerInstance* DeformerInstance = WeakDeformerInstance.Get();
	if (DeformerInstance && DataInterface)
	{
		{
			FOptimusValueContainerStruct ValueContainer =
				DeformerInstance->GetDataInterfacePropertyOverride(
					DataInterface,
					UOptimusSkinnedMeshVertexAttributeDataInterface::GetAttributeNamePropertyName()
					);

			TValueOrError<FName, EPropertyBagResult> Value = ValueContainer.Value.GetValueName(FOptimusValueContainerStruct::ValuePropertyName);
			if (Value.HasValue())
			{
				AttributeName = Value.GetValue();
			}
		}

		{
			FOptimusValueContainerStruct ValueContainer =
				DeformerInstance->GetDataInterfacePropertyOverride(
					DataInterface,
					UOptimusSkinnedMeshVertexAttributeDataInterface::GetDefaultValuePropertyName()
				);

			TValueOrError<float, EPropertyBagResult> Value = ValueContainer.Value.GetValueFloat(FOptimusValueContainerStruct::ValuePropertyName);
			if (Value.HasValue())
			{
				DefaultValue = Value.GetValue();
			}
		}
	}
	
	return new FOptimusSkinnedMeshVertexAttributeDataProviderProxy(SkinnedMeshComponent.Get(), AttributeName, DefaultValue);
}

void UOptimusSkinnedMeshVertexAttributeDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}


FOptimusSkinnedMeshVertexAttributeDataProviderProxy::FOptimusSkinnedMeshVertexAttributeDataProviderProxy(
	USkinnedMeshComponent* InSkinnedMeshComponent, 
	FName InAttributeName,
	float InDefaultValue
	)
{
	if (InSkinnedMeshComponent && InSkinnedMeshComponent->IsRegistered())
	{
		SceneInterface = InSkinnedMeshComponent->GetScene();
		ComponentId = InSkinnedMeshComponent->GetPrimitiveSceneId();
	}
	AttributeName = InAttributeName;
	DefaultValue = InDefaultValue;
}


bool FOptimusSkinnedMeshVertexAttributeDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	return true;
}


void FOptimusSkinnedMeshVertexAttributeDataProviderProxy::GatherDispatchData(
	FDispatchData const& InDispatchData
	)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];
	
	const TStridedView<FSkinnedMeshVertexAttributeDataInterfaceParameters> ParameterArray =
		MakeStridedParameterView<FSkinnedMeshVertexAttributeDataInterfaceParameters>(InDispatchData);
	
	FRHIShaderResourceView* NullSRVBinding = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI.GetReference();
	const FSkeletalMeshAttributeVertexBuffer* AttributeBuffer = LodRenderData->VertexAttributeBuffers.GetAttributeBuffer(AttributeName);
	FRHIShaderResourceView *AttributeSRV = AttributeBuffer ? AttributeBuffer->GetSRV() : NullSRVBinding;

	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FSkinnedMeshVertexAttributeDataInterfaceParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices = LodRenderData->GetNumVertices();
		Parameters.ValueBuffer = AttributeSRV;
		Parameters.bIsValid = AttributeBuffer != nullptr;
		Parameters.DefaultValue = DefaultValue;
	}
}
