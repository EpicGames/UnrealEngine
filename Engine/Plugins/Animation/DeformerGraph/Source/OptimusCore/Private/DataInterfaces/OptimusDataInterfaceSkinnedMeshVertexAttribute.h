// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "OptimusComputeDataInterface.h"

#include "ComputeFramework/ComputeDataProvider.h"

#include "PrimitiveComponentId.h"
#include "OptimusDataInterfaceSkinnedMeshVertexAttribute.generated.h"


class FSceneInterface;
class FSkeletalMeshObject;
class FSkinnedMeshVertexAttributeDataInterfaceParameters;


UCLASS(Category = ComputeFramework)
class UOptimusSkinnedMeshVertexAttributeDataInterface :
	public UOptimusComputeDataInterface
{
	GENERATED_BODY()
	
	//~ Begin UOptimusComputeDataInterface Interface
	FString GetDisplayName() const override;
	TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const override;
	TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface

	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("VertexAttribute"); }
	virtual bool CanSupportUnifiedDispatch() const override { return true; };
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	void GetShaderHash(FString& InOutKey) const override;
	UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

	UPROPERTY(EditAnywhere, Category="Vertex Attribute")
	FName AttributeName;

	UPROPERTY(EditAnywhere, Category="Vertex Attribute")
	float DefaultValue;
private:
	friend class UOptimusSkinnedMeshVertexAttributeDataProvider;
	static FName GetAttributeNamePropertyName();
	static FName GetDefaultValuePropertyName();
};


/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkinnedMeshVertexAttributeDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMeshComponent = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VertexAttribute)
	FName AttributeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VertexAttribute)
	float DefaultValue;

    TWeakObjectPtr<const UOptimusSkinnedMeshVertexAttributeDataInterface> WeakDataInterface;
	
	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface

	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;

	
private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance;
};

class FOptimusSkinnedMeshVertexAttributeDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	FOptimusSkinnedMeshVertexAttributeDataProviderProxy(
		USkinnedMeshComponent* InSkinnedMeshComponent,
		FName InAttributeName,
		float InDefaultValue);

	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	//~ End FComputeDataProviderRenderProxy Interface

private:
	using FParameters = FSkinnedMeshVertexAttributeDataInterfaceParameters;
	
	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	FName AttributeName;
	float DefaultValue;
};
