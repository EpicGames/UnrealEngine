// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataInterfaceSkinnedMeshWrite.h"

#include "Components/SkinnedMeshComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ComputeWorkerInterface.h"
#include "OptimusSkinnedMeshTrackingExtension.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Animation/MeshDeformerInstance.h"
#include "OptimusDeformerInstance.h"

#if WITH_EDITORONLY_DATA
#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"
#include "MeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "Async/ParallelFor.h"
#include "OptimusGeometryReadbackProcessor.h"
#include "Animation/MeshDeformerGeometryReadback.h"
#endif // WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataInterfaceSkinnedMeshWrite)


FString UOptimusSkinnedMeshWriteDataInterface::GetDisplayName() const
{
	return TEXT("Write Skinned Mesh");
}

FName UOptimusSkinnedMeshWriteDataInterface::GetCategory() const
{
	return CategoryName::OutputDataInterfaces;
}

TArray<FOptimusCDIPinDefinition> UOptimusSkinnedMeshWriteDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "Position", "WritePosition", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "TangentX", "WriteTangentX", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "TangentZ", "WriteTangentZ", Optimus::DomainName::Vertex, "ReadNumVertices" });
	Defs.Add({ "Color", "WriteColor", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}


TSubclassOf<UActorComponent> UOptimusSkinnedMeshWriteDataInterface::GetRequiredComponentClass() const
{
	return USkinnedMeshComponent::StaticClass();
}


void UOptimusSkinnedMeshWriteDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);
}

// Should be kept in sync with GetSupportedOutputs
enum class ESkinnedMeshWriteDataInterfaceOutputSelectorMask : uint64
{
	Position = 1 << 0,
	TangentX = 1 << 1,
	TangentZ = 1 << 2,
	Color = 1 << 3,
};
ENUM_CLASS_FLAGS(ESkinnedMeshWriteDataInterfaceOutputSelectorMask);


void UOptimusSkinnedMeshWriteDataInterface::GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WritePosition"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 3);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteTangentX"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 4);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteTangentZ"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 4);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("WriteColor"))
		.AddParam(EShaderFundamentalType::Uint)
		.AddParam(EShaderFundamentalType::Float, 4);
}

BEGIN_SHADER_PARAMETER_STRUCT(FSkinedMeshWriteDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, PositionBufferUAV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<SNORM float4>, TangentBufferUAV)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<UNORM float4>, ColorBufferUAV)
END_SHADER_PARAMETER_STRUCT()

void UOptimusSkinnedMeshWriteDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FSkinedMeshWriteDataInterfaceParameters>(UID);
}

TCHAR const* UOptimusSkinnedMeshWriteDataInterface::TemplateFilePath = TEXT("/Plugin/Optimus/Private/DataInterfaceSkinnedMeshWrite.ush");

TCHAR const* UOptimusSkinnedMeshWriteDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UOptimusSkinnedMeshWriteDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UOptimusSkinnedMeshWriteDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UOptimusSkinnedMeshWriteDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UOptimusSkinnedMeshWriteDataProvider* Provider = NewObject<UOptimusSkinnedMeshWriteDataProvider>();
	Provider->SkinnedMesh = Cast<USkinnedMeshComponent>(InBinding);
	Provider->OutputMask = InOutputMask;
	return Provider;
}

EMeshDeformerOutputBuffer UOptimusSkinnedMeshWriteDataInterface::GetOutputBuffer(int32 InBoundOutputFunctionIndex) const
{
	// Maps to the index of functions in GetSupportedOutputs
	if (InBoundOutputFunctionIndex == 0)
	{
		return EMeshDeformerOutputBuffer::SkinnedMeshPosition;
	}
	if (InBoundOutputFunctionIndex == 1 || InBoundOutputFunctionIndex == 2)
	{
		return EMeshDeformerOutputBuffer::SkinnedMeshTangents;
	}
	if (InBoundOutputFunctionIndex == 3)
	{
		return EMeshDeformerOutputBuffer::SkinnedMeshVertexColor;
	}

	return EMeshDeformerOutputBuffer::None;
}


UOptimusSkinnedMeshWriteDataProvider::UOptimusSkinnedMeshWriteDataProvider(FVTableHelper& Helper)
	: Super(Helper)
{
}

void UOptimusSkinnedMeshWriteDataProvider::InitializePerKernelInputOutputMasks(TMap<int32, FInputOutputMask>&& InPerKernelMasks)
{
	PerKernelMasks = MoveTemp(InPerKernelMasks);
}

FComputeDataProviderRenderProxy* UOptimusSkinnedMeshWriteDataProvider::GetRenderProxy()
{
	FOptimusSkinnedMeshWriteDataProviderProxy* Proxy = new FOptimusSkinnedMeshWriteDataProviderProxy();

	if (SkinnedMesh.IsValid() && SkinnedMesh->IsRegistered())
	{
		Proxy->SceneInterface = SkinnedMesh->GetScene();
		Proxy->ComponentId = SkinnedMesh->GetPrimitiveSceneId();
	}
	Proxy->OutputMask = OutputMask;
	Proxy->PerKernelMasks = PerKernelMasks;

	if (WeakDeformerInstance.IsValid())
	{
		Proxy->ExecutionGroup = WeakDeformerInstance->CurrentExecutionGroupName;
	}

#if WITH_EDITORONLY_DATA
	Proxy->FrameNumber = GFrameNumber;
	Proxy->SkeletalMeshAsset = Cast<USkeletalMesh>(SkinnedMesh->GetSkinnedAsset());
	for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
	{
		Proxy->GeometryReadbackRequests.Add(MoveTemp(Request));
	}
	
	GeometryReadbackRequests.Empty();
#endif
	
	return Proxy;
}
#if WITH_EDITORONLY_DATA
bool UOptimusSkinnedMeshWriteDataProvider::RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest)
{
	GeometryReadbackRequests.Add(MoveTemp(InRequest));

	return true;
}
#endif // WITH_EDITORONLY_DATA

void UOptimusSkinnedMeshWriteDataProvider::SetDeformerInstance(UOptimusDeformerInstance* InInstance)
{
	WeakDeformerInstance = InInstance;
}

bool FOptimusSkinnedMeshWriteDataProviderProxy::IsValid(FValidationData const& InValidationData) const
{
	if (InValidationData.ParameterStructSize != sizeof(FParameters))
	{
		return false;
	}

	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return false;
	}
	
	if (FSkeletalMeshDeformerHelpers::GetIndexOfFirstAvailableSection(SkeletalMeshObject, SkeletalMeshObject->GetLOD()) == INDEX_NONE)
	{
		return false;
	}
	
	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	// this mismatch can happen during map load for one frame sometimes, when predicted lod != actual lod
	// TODO: avoid the mismatch in the first place, use actual lod to query NumInvocations
	if (LodRenderData->RenderSections.Num() != InValidationData.NumInvocations)
	{
		return false;
	}

	return true;
}

void FOptimusSkinnedMeshWriteDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	// Allocate required buffers
	const int32 LodIndex = SkeletalMeshObject->GetLOD();

	if (OutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::Position))
	{
		PositionBuffer = FSkeletalMeshDeformerHelpers::AllocateVertexFactoryPositionBuffer(GraphBuilder, InAllocationData.ExternalAccessQueue, SkeletalMeshObject, LodIndex, TEXT("OptimusSkinnedMeshPosition"));
		PositionBufferUAV = GraphBuilder.CreateUAV(PositionBuffer, PF_R32_FLOAT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (GeometryReadbackRequests.Num() > 0)
		{
			PositionBuffer = FSkeletalMeshDeformerHelpers::GetAllocatedPositionBuffer(GraphBuilder, SkeletalMeshObject, LodIndex);
		}
#endif // WITH_EDITORONLY_DATA
		
		// Unused UAV
		PositionBufferUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalBuffer(GWhiteVertexBufferWithRDG->Buffer), PF_R32_FLOAT);
	}

	// OpenGL ES does not support writing to RGBA16_SNORM images, instead pack data into SINT in the shader
	const EPixelFormat TangentsFormat = IsOpenGLPlatform(GMaxRHIShaderPlatform) ? PF_R16G16B16A16_SINT : PF_R16G16B16A16_SNORM;

	if ((OutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::TangentX)) ||
		(OutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::TangentZ)))
	{
		TangentBuffer = FSkeletalMeshDeformerHelpers::AllocateVertexFactoryTangentBuffer(GraphBuilder, InAllocationData.ExternalAccessQueue, SkeletalMeshObject, LodIndex, TEXT("OptimusSkinnedMeshTangent"));
		TangentBufferUAV = GraphBuilder.CreateUAV(TangentBuffer, TangentsFormat, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (GeometryReadbackRequests.Num() > 0)
		{
			TangentBuffer = FSkeletalMeshDeformerHelpers::GetAllocatedTangentBuffer(GraphBuilder, SkeletalMeshObject, LodIndex);
		}
#endif // WITH_EDITORONLY_DATA

		// Unused UAV
		TangentBufferUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalBuffer(GWhiteVertexBufferWithRDG->Buffer), TangentsFormat);
	}

	if (OutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::Color))
	{
		ColorBuffer = FSkeletalMeshDeformerHelpers::AllocateVertexFactoryColorBuffer(GraphBuilder, InAllocationData.ExternalAccessQueue, SkeletalMeshObject, LodIndex, TEXT("OptimusSkinnedMeshColor"));
		// using RGBA here and do a manual fetch swizzle in shader instead of BGRA directly because some Mac does not support it. See GMetalBufferFormats[PF_B8G8R8A8] 
		ColorBufferUAV = GraphBuilder.CreateUAV(ColorBuffer, PF_R8G8B8A8, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (GeometryReadbackRequests.Num() > 0)
		{
			ColorBuffer = FSkeletalMeshDeformerHelpers::GetAllocatedColorBuffer(GraphBuilder, SkeletalMeshObject, LodIndex);	
		}
#endif // WITH_EDITORONLY_DATA
	
		// Unused UAV
		ColorBufferUAV = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalBuffer(GWhiteVertexBufferWithRDG->Buffer), PF_A32B32G32R32F);
	}

}

void FOptimusSkinnedMeshWriteDataProviderProxy::PostSubmit(FComputeContext& Context) const
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
	if (!SkeletalMeshObject)
	{
		return;
	}

	const uint64 KernelOutputMask = PerKernelMasks.FindChecked(Context.GraphKernelIndex).OutputMask;

	EMeshDeformerOutputBuffer DeformerOutputBuffers = EMeshDeformerOutputBuffer::None;

	if (KernelOutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::Position))
	{
		DeformerOutputBuffers |= EMeshDeformerOutputBuffer::SkinnedMeshPosition;
	}
	if ((KernelOutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::TangentX)) ||
		(KernelOutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::TangentZ)))
	{
		DeformerOutputBuffers |= EMeshDeformerOutputBuffer::SkinnedMeshTangents;
	}
	if (KernelOutputMask & static_cast<uint64>(ESkinnedMeshWriteDataInterfaceOutputSelectorMask::Color))
	{
		DeformerOutputBuffers |= EMeshDeformerOutputBuffer::SkinnedMeshVertexColor;
	}

	FOptimusSkinnedMeshTrackingExtension::MarkDeformed(Context.GraphBuilder, SceneInterface, ComponentId, SkeletalMeshObject, DeformerOutputBuffers, ExecutionGroup);
}

void FOptimusSkinnedMeshWriteDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
{
	FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);

	const int32 LodIndex = SkeletalMeshObject->GetLOD();
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[LodIndex];

	const TStridedView<FParameters> ParameterArray = MakeStridedParameterView<FParameters>(InDispatchData);
	for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
	{
		FParameters& Parameters = ParameterArray[InvocationIndex];
		Parameters.NumVertices =LodRenderData->GetNumVertices();
		Parameters.PositionBufferUAV = PositionBufferUAV;
		Parameters.TangentBufferUAV = TangentBufferUAV;
		Parameters.ColorBufferUAV = ColorBufferUAV;
	}
}

void FOptimusSkinnedMeshWriteDataProviderProxy::GetReadbackData(TArray<FReadbackData>& OutReadbackData) const
{
#if WITH_EDITORONLY_DATA
	if (GeometryReadbackRequests.Num() > 0)
	{
		// Each proxy allocates a FGeometryReadback which stores the result of its readback and is processed async at a later time
		using FGeometryReadback = FOptimusGeometryReadbackProcessor::FGeometryReadback;
		using FBufferReadback = FOptimusGeometryReadbackProcessor::FBufferReadback;
		using FReadbackProcessor = FOptimusGeometryReadbackProcessor;
		
		TSharedPtr<FGeometryReadback> GeometryReadback = MakeShared<FGeometryReadback>();

		GeometryReadback->FrameNumber = FrameNumber;
		for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
		{
			GeometryReadback->GeometryReadbackRequests.Add(MoveTemp(Request));
		}
		GeometryReadbackRequests.Reset();
		GeometryReadback->SkeletalMesh = SkeletalMeshAsset;
		FSkeletalMeshObject* ReadbackMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		GeometryReadback->LodIndex = ReadbackMeshObject ? ReadbackMeshObject->GetLOD() : 0;

		auto SetupBufferReadback = [&OutReadbackData, this](FRDGBuffer* GPUBuffer, FBufferReadback& BufferReadback)
		{
			if (GPUBuffer)
			{
				BufferReadback.bShouldReadback = true;
				BufferReadback.OnReadbackCompleted_RenderThread = [&ReadbackData = BufferReadback.ReadbackData](const void* InData, int InNumBytes)
				{
					if (ensure(InNumBytes > 0))
					{
						check(ReadbackData.Num() == 0);
						ReadbackData.SetNumUninitialized(InNumBytes);
						FPlatformMemory::Memcpy(ReadbackData.GetData(), InData, InNumBytes);
					}
					
					FReadbackProcessor::Get().ProcessCompletedGeometryReadback_RenderThread();
				};

				// Raw buffer readback request to the compute worker
				FComputeDataProviderRenderProxy::FReadbackData& BufferReadbackRequestRef = OutReadbackData.AddDefaulted_GetRef();
				BufferReadbackRequestRef.Buffer = GPUBuffer;
				BufferReadbackRequestRef.NumBytes = GPUBuffer->Desc.GetSize();
				BufferReadbackRequestRef.ReadbackCallback_RenderThread = &BufferReadback.OnReadbackCompleted_RenderThread;
			}
		};

		SetupBufferReadback(PositionBuffer, GeometryReadback->Position);
		SetupBufferReadback(TangentBuffer, GeometryReadback->Tangent);
		SetupBufferReadback(ColorBuffer, GeometryReadback->Color);
		
		FReadbackProcessor::Get().Add(GeometryReadback);	
	}
#endif // WITH_EDITORONLY_DATA
}
