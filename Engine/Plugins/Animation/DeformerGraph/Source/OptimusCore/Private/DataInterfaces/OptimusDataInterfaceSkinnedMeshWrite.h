// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeformerInstanceAccessor.h"
#include "IOptimusDeformerGeometryReadbackProvider.h"
#include "IOptimusOutputBufferWriter.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"


#include "OptimusDataInterfaceSkinnedMeshWrite.generated.h"

#define UE_API OPTIMUSCORE_API

class FRDGBuffer;
class FRDGBufferUAV;
class FSceneInterface;
class FSkeletalMeshObject;
class FSkinedMeshWriteDataInterfaceParameters;
class USkinnedMeshComponent;
enum class EMeshDeformerOutputBuffer : uint8;

#if WITH_EDITORONLY_DATA
class USkeletalMesh;
#endif // WITH_EDITORONLY_DATA

/** Compute Framework Data Interface for writing skinned mesh. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UOptimusSkinnedMeshWriteDataInterface :
	public UOptimusComputeDataInterface,
	public IOptimusOutputBufferWriter
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API FName GetCategory() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("SkinnedMeshWrite"); }
	bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetSupportedOutputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	bool GetRequiresReadback() const override { return true; }
	bool GetRequiresPostSubmitCall() const override { return true; }
	// This DI can be bound to several kernels writing different pins. PostSubmit needs the per-kernel
	// breakdown so MarkDeformed only tags buffers that the submitted kernel actually wrote.
	bool GetRequiresPerKernelInputOutputMasks() const override { return true; }
	//~ End UComputeDataInterface Interface

	//~ Begin IOptimusOutputBufferWriter Interface
	UE_API EMeshDeformerOutputBuffer GetOutputBuffer(int32 InBoundOutputFunctionIndex) const override;
	//~ End IOptimusOutputBufferWriter Interface 
private:
	static UE_API TCHAR const* TemplateFilePath;
};



/** Compute Framework Data Provider for writing skinned mesh. */
UCLASS(BlueprintType, editinlinenew, Category = ComputeFramework)
class UOptimusSkinnedMeshWriteDataProvider :
	public UComputeDataProvider,
	public IOptimusDeformerInstanceAccessor,
	public IOptimusDeformerGeometryReadbackProvider
{
	GENERATED_BODY()

public:
	UOptimusSkinnedMeshWriteDataProvider() = default;
	~UOptimusSkinnedMeshWriteDataProvider() = default;
	UOptimusSkinnedMeshWriteDataProvider(FVTableHelper& Helper);
	
	TWeakObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	uint64 OutputMask = 0;

	TMap<int32, FInputOutputMask> PerKernelMasks;

	//~ Begin UComputeDataProvider Interface
	FComputeDataProviderRenderProxy* GetRenderProxy() override;
	void InitializePerKernelInputOutputMasks(TMap<int32, FInputOutputMask>&& InPerKernelMasks) override;
	//~ End UComputeDataProvider Interface

	//~ Begin IOptimusDeformerInstanceAccessor Interface
	void SetDeformerInstance(UOptimusDeformerInstance* InInstance) override;
	//~ End IOptimusDeformerInstanceAccessor Interface
	
#if WITH_EDITORONLY_DATA	
	//~ Begin IOptimusDeformerGeometryReadbackProvider Interface
	bool RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest) override;
	//~ End IOptimusDeformerGeometryReadbackProvider Interface
#endif // WITH_EDITORONLY_DATA	
private:
	TWeakObjectPtr<UOptimusDeformerInstance> WeakDeformerInstance = nullptr;

#if WITH_EDITORONLY_DATA
	/** Readback requests for the current frame */
	TArray<TUniquePtr<FMeshDeformerGeometryReadbackRequest>> GeometryReadbackRequests;
#endif // WITH_EDITORONLY_DATA
};

class FOptimusSkinnedMeshWriteDataProviderProxy : public FComputeDataProviderRenderProxy
{
public:
	//~ Begin FComputeDataProviderRenderProxy Interface
	bool IsValid(FValidationData const& InValidationData) const override;
	void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
	void GatherDispatchData(FDispatchData const& InDispatchData) override;
	void PostSubmit(FComputeContext& Context) const override;
	void GetReadbackData(TArray<FReadbackData>& OutReadbackData) const override;
	//~ End FComputeDataProviderRenderProxy Interface

	FSceneInterface* SceneInterface = nullptr;
	FPrimitiveComponentId ComponentId;
	uint64 OutputMask = 0;
	TMap<int32, UComputeDataProvider::FInputOutputMask> PerKernelMasks;
	FName ExecutionGroup;

#if WITH_EDITORONLY_DATA

	int32 FrameNumber = 0;

	USkeletalMesh* SkeletalMeshAsset = nullptr;
	/** Readback requests for the current frame */
	mutable TArray<TUniquePtr<FMeshDeformerGeometryReadbackRequest>> GeometryReadbackRequests;
	
	TFunction<
		void(
			TArray<FReadbackData>&,
			FSkeletalMeshObject*,
			FRDGBuffer*,
			FRDGBuffer*,
			FRDGBuffer*
		)> RequestGeometryReadback_RenderThread;
#endif // WITH_EDITORONLY_DATA
	
private:
	using FParameters = FSkinedMeshWriteDataInterfaceParameters;

	FRDGBuffer* PositionBuffer = nullptr;
	FRDGBufferUAV* PositionBufferUAV = nullptr;
	FRDGBuffer* TangentBuffer = nullptr;
	FRDGBufferUAV* TangentBufferUAV = nullptr;
	FRDGBuffer* ColorBuffer = nullptr;
	FRDGBufferUAV* ColorBufferUAV = nullptr;
};

#undef UE_API
