// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_AnimAttributeDataInterface.h"

#include "OptimusCoreModule.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "DataInterfaces/OptimusDataInterfaceAnimAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_AnimAttributeDataInterface)


UOptimusNode_AnimAttributeDataInterface::UOptimusNode_AnimAttributeDataInterface() :
	Super()
{
	EnableDynamicPins();
}

void UOptimusNode_AnimAttributeDataInterface::SetDataInterfaceClass(
	TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass)
{
	Super::SetDataInterfaceClass(InDataInterfaceClass);
	
	if (UOptimusAnimAttributeDataInterface* Interface = Cast<UOptimusAnimAttributeDataInterface>(DataInterfaceData))
	{
		// Add a default attribute so that the node is ready to be used
		Interface->AddAnimAttribute(TEXT("EmptyName"), NAME_None,
			FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()) );
	}

	// Undo support
	DataInterfaceData->SetFlags(RF_Transactional);
}

#if WITH_EDITOR
void UOptimusNode_AnimAttributeDataInterface::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified)
	{
		static FProperty* NameProperty =
			FOptimusAnimAttributeDescription::StaticStruct()->FindPropertyByName(
				GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeDescription, Name)
			);
		
		static FProperty* BoneNameProperty =
			FOptimusAnimAttributeDescription::StaticStruct()->FindPropertyByName(
				GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeDescription, BoneName)
			);
		
		static FProperty* DataTypeNameProperty =
			FOptimusDataTypeRef::StaticStruct()->FindPropertyByName(
				GET_MEMBER_NAME_STRING_CHECKED(FOptimusDataTypeRef, TypeName)
			);

		if (PropertyChangedEvent.PropertyChain.Contains(NameProperty)||
			PropertyChangedEvent.PropertyChain.Contains(BoneNameProperty) || 
			PropertyChangedEvent.PropertyChain.Contains(DataTypeNameProperty))
		{
			UpdatePinNames();
		}

		if (PropertyChangedEvent.PropertyChain.Contains(DataTypeNameProperty))
		{
			UpdatePinTypes();
		}

		if (PropertyChangedEvent.PropertyChain.GetTail()->GetValue()->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusAnimAttributeArray, InnerArray))
		{
			RefreshOutputPins();
		}
	}
}

#endif

void UOptimusNode_AnimAttributeDataInterface::OnDataTypeChanged(FName InTypeName)
{
	Super::OnDataTypeChanged(InTypeName);

	UOptimusAnimAttributeDataInterface* Interface = Cast<UOptimusAnimAttributeDataInterface>(DataInterfaceData);
	Interface->OnDataTypeChanged(InTypeName);
}

void UOptimusNode_AnimAttributeDataInterface::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);
}


void UOptimusNode_AnimAttributeDataInterface::UpdatePinTypes()
{
	UOptimusAnimAttributeDataInterface* Interface = Cast<UOptimusAnimAttributeDataInterface>(DataInterfaceData);
	const int32 NumAttributes = Interface->AttributeArray.Num(); 

	// Let's try and figure out which pin got changed.
	TArrayView<UOptimusNodePin* const> NodePins = GetPins();
	
	const TArray<UOptimusNodePin*> OutputPins = NodePins.FilterByPredicate(
		[](UOptimusNodePin* const InPin)
		{
			if (InPin->GetDirection() == EOptimusNodePinDirection::Output)
			{
				return true;
			}
			
			return false;
		}
	);	
	
	if (ensure(NumAttributes == OutputPins.Num()))
	{
		for (int32 Index = 0; Index < OutputPins.Num(); Index++)
		{
			if (OutputPins[Index]->GetDataType() != Interface->AttributeArray[Index].DataType.Resolve())
			{
				SetPinDataType(OutputPins[Index], Interface->AttributeArray[Index].DataType);
			}
		}
	}
}


void UOptimusNode_AnimAttributeDataInterface::UpdatePinNames()
{
	UOptimusAnimAttributeDataInterface* Interface = Cast<UOptimusAnimAttributeDataInterface>(DataInterfaceData);
	
	TArray<FOptimusCDIPinDefinition> PinDefinitions = Interface->GetPinDefinitions();
	
	// Let's try and figure out which pin got changed.
	TArrayView<UOptimusNodePin* const> NodePins = GetPins();

	const TArray<UOptimusNodePin*> OutputPins = NodePins.FilterByPredicate(
		[](UOptimusNodePin* const InPin)
		{
			if (InPin->GetDirection() == EOptimusNodePinDirection::Output)
			{
				return true;
			}
			
			return false;
		}
	);
	
	if (ensure(PinDefinitions.Num() == OutputPins.Num()))
	{
		for (int32 Index = 0; Index < OutputPins.Num(); Index++)
		{
			if (OutputPins[Index]->GetFName() != PinDefinitions[Index].PinName)
			{
				SetPinName(OutputPins[Index], PinDefinitions[Index].PinName);
			}
		}
	}
}


void UOptimusNode_AnimAttributeDataInterface::ClearOutputPins()
{
	for (UOptimusNodePin* Pin : GetPinsByDirection(EOptimusNodePinDirection::Output, false))
	{
		RemovePin(Pin);
	}
}

void UOptimusNode_AnimAttributeDataInterface::RefreshOutputPins()
{
	// Save the links and readd them later when new pins are created
	TMap<FName, TArray<UOptimusNodePin*>> ConnectedPinsMap;

	for (const UOptimusNodePin* Pin : GetPins())
	{
		if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			ConnectedPinsMap.Add(Pin->GetFName()) = Pin->GetConnectedPins();
		}
	}	

	ClearOutputPins();

	UOptimusAnimAttributeDataInterface* Interface = Cast<UOptimusAnimAttributeDataInterface>(DataInterfaceData);
	for (const FOptimusAnimAttributeDescription& Attribute : Interface->AttributeArray)
	{
		AddPin(Attribute.PinName, EOptimusNodePinDirection::Output, {}, Attribute.DataType);
	}

	for (UOptimusNodePin* AddedPin : GetPins())
	{
		if (AddedPin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			if (TArray<UOptimusNodePin*>* ConnectedPins = ConnectedPinsMap.Find(AddedPin->GetFName()))
			{
				for (UOptimusNodePin* ConnectedPin : *ConnectedPins)
				{
					GetOwningGraph()->AddLink(AddedPin, ConnectedPin);
				}
			}	
		}
	}
}
