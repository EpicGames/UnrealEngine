// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode_DataInterface.h"

#include "OptimusNode_AnimAttributeDataInterface.generated.h"

// Deprecated
UCLASS(Hidden)
class UOptimusNode_AnimAttributeDataInterface :
	public UOptimusNode_DataInterface

{
public:
	GENERATED_BODY()
	UOptimusNode_AnimAttributeDataInterface();

	// UOptimusNode_DataInterface overrides
	virtual void SetDataInterfaceClass(TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass) override;

#if WITH_EDITOR
	void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif


	virtual void OnDataTypeChanged(FName InTypeName) override;

protected:
	// UObject overrides
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	
private:
	void UpdatePinTypes();
	void UpdatePinNames();

	void ClearOutputPins();

	void RefreshOutputPins();
};
