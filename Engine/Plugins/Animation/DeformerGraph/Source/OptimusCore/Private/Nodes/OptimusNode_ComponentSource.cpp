// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_ComponentSource.h"

#include "OptimusNodePin.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "Actions/OptimusAction.h"
#include "Actions/OptimusComponentBindingActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_ComponentSource)


void UOptimusNode_ComponentSource::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);
	
	if (!GetOwningGraph())
	{
		return;
	}
	
	const UOptimusDeformer* NewBindingOwner = Cast<UOptimusDeformer>(GetOwningGraph()->GetCollectionRoot());

	if (!NewBindingOwner)
	{
		return;
	}
	
	if (Binding)
	{
		const UOptimusDeformer* OldBindingOwner = Binding->GetOwningDeformer();
	
		// No action needed if we are copying/pasting within the same deformer asset 
		if (OldBindingOwner == NewBindingOwner)
		{
			return;
		}

		// Refresh the binding so that we don't hold a reference to a binding in another deformer asset
		Binding = NewBindingOwner->ResolveComponentBinding(Binding->GetFName());
	}	
	else if (!DuplicationInfo.BindingName.IsNone())
	{
		// Resolve the variable description from the duplication information that likely came from a copy/paste operation
		// where the pointer didn't survive and the variable didn't exist and we had to set it up in PreDuplicateRequirementActions
		Binding = NewBindingOwner->ResolveComponentBinding(DuplicationInfo.BindingName);

		if (Binding)
		{
			// Re-create the pins. Don't call PostCreateNode, since it won't notify the graph that the pin layout has
			// changed.
			check(GetPins().IsEmpty());
			ConstructNode();
		}

		// Empty the duplication info before we carry on.
		DuplicationInfo = FOptimusNode_ComponentSource_DuplicationInfo();
	}
}

void UOptimusNode_ComponentSource::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	if (ensure(Binding))
	{
		FString ExportText;

		Out.Logf(TEXT("%sCustomProperties ComponentSourceBinding BindingName=%s ComponentType=%s\n"),
			FCString::Spc(Indent), *Binding->BindingName.ToString(), *Binding->ComponentType->GetStructPathName().ToString());
	}
}


void UOptimusNode_ComponentSource::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	if (FParse::Command(&SourceText, TEXT("ComponentSourceBinding")))
	{
		FName BindingName;
		if (!FParse::Value(SourceText, TEXT("BindingName="), BindingName))
		{
			return;
		}

		FString ComponentTypeClassPath;
		if (!FParse::Value(SourceText, TEXT("ComponentType="), ComponentTypeClassPath))
		{
			return;
		}

		UClass* ComponentSourceClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), FTopLevelAssetPath(ComponentTypeClassPath), EFindObjectFlags::None));
		if (!ComponentSourceClass)
		{
			return;
		}

		DuplicationInfo.BindingName=BindingName;
		DuplicationInfo.ComponentType = ComponentSourceClass;
	}
}

void UOptimusNode_ComponentSource::SetComponentSourceBinding(
	UOptimusComponentSourceBinding* InBinding
	)
{
	Binding = InBinding;

	SetDisplayName(FText::FromName(Binding->BindingName));
}


FName UOptimusNode_ComponentSource::GetNodeCategory() const
{
	static FName ComponentCategory("Component");
	return ComponentCategory;
}


void UOptimusNode_ComponentSource::ConstructNode()
{
	const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
	FOptimusDataTypeRef ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());

	// Binding can be empty during copy paste. It is assigned a value in post duplicate and ConstructNode is called again.
	if (ensure(ComponentSourceType.IsValid()) &&
		Binding &&
		ensure(Binding->ComponentType))
	{
		AddPinDirect(Binding->GetComponentSource()->GetBindingName(), EOptimusNodePinDirection::Output, {}, ComponentSourceType);
	}
}

void UOptimusNode_ComponentSource::PreDuplicateRequirementActions(const UOptimusNodeGraph* InTargetGraph, FOptimusCompoundAction* InCompoundAction)
{
	if (DuplicationInfo.BindingName.IsNone())
	{
		return;
	}

	// Check if the deformer has a variable that matches the name and type.
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InTargetGraph->GetCollectionRoot());
	
	for (const UOptimusComponentSourceBinding* ExistingBinding: Deformer->GetComponentBindings())
	{
		if (ExistingBinding->BindingName == DuplicationInfo.BindingName &&
			ExistingBinding->ComponentType == DuplicationInfo.ComponentType)
		{
			// Note: We don't care about component tags in this context.
			return;
		}
	}

	// Note: This will fail if there are multiple overlapping names. A better system is needed.
	bool bFoundDuplicate = false;
	do
	{
		bFoundDuplicate = false;
		for (const UOptimusComponentSourceBinding* ExistingBinding: Deformer->GetComponentBindings())
		{
			if (ExistingBinding->BindingName == DuplicationInfo.BindingName)
			{
				DuplicationInfo.BindingName.SetNumber(DuplicationInfo.BindingName.GetNumber() + 1);
				bFoundDuplicate = true;
			}
		}
	}
	while(bFoundDuplicate);

	InCompoundAction->AddSubAction<FOptimusComponentBindingAction_AddBinding>(DuplicationInfo.ComponentType->GetDefaultObject<UOptimusComponentSource>(), DuplicationInfo.BindingName);
}

UOptimusNodePin* UOptimusNode_ComponentSource::GetComponentPin() const
{
	if (ensure(!GetPins().IsEmpty()))
	{
		return GetPins()[0];
	}
	return nullptr;
}
