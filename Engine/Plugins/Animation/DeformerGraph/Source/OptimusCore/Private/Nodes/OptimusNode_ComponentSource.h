// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusComponentBindingProvider.h"
#include "IOptimusNonCollapsibleNode.h"
#include "OptimusNode.h"

#include "OptimusComponentSource.h"

#include "OptimusNode_ComponentSource.generated.h"

USTRUCT()
struct FOptimusNode_ComponentSource_DuplicationInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName BindingName;

	UPROPERTY()
	TSubclassOf<UOptimusComponentSource> ComponentType;
};

UCLASS(Hidden)
class UOptimusNode_ComponentSource :
	public UOptimusNode,
	public IOptimusComponentBindingProvider,
	public IOptimusNonCollapsibleNode
{
	GENERATED_BODY()
public:
	void SetComponentSourceBinding(
		UOptimusComponentSourceBinding* InBinding
		);

	// UOptimusNode overrides
	FName GetNodeCategory() const override;

	// IOptimusComponentSourceBindingProvider implementation
	UOptimusComponentSourceBinding* GetComponentBinding(const FOptimusPinTraversalContext& InContext = {}) const override
	{
		return Binding;
	}

protected:
	friend class UOptimusDeformer;

	// UOptimusNode overrides
	void ConstructNode() override;
	void PreDuplicateRequirementActions(const UOptimusNodeGraph* InTargetGraph, FOptimusCompoundAction* InCompoundAction) override;

	/** Accessor to allow the UOptimusDeformer class to hook up data interface nodes to binding nodes automatically
	 *  for backcomp.
	 */
	UOptimusNodePin* GetComponentPin() const;

	// UObject overrides
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	
private:
	UPROPERTY()
	TObjectPtr<UOptimusComponentSourceBinding> Binding;

	// Duplication data across graphs/assets
	UPROPERTY(DuplicateTransient)
	FOptimusNode_ComponentSource_DuplicationInfo DuplicationInfo;	
};
