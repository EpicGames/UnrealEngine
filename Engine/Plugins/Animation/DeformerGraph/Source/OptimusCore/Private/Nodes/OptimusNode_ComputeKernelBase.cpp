// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_ComputeKernelBase.h"

#include "IOptimusComputeKernelDataInterface.h"
#include "IOptimusDataInterfaceProvider.h"
#include "OptimusCoreModule.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"

#include "DataInterfaces/OptimusDataInterfaceGraph.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "DataInterfaces/OptimusDataInterfaceLoopTerminal.h"
#include "IOptimusValueProvider.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "OptimusKernelSource.h"
#include "OptimusExecutionDomain.h"
#include "OptimusNode_LoopTerminal.h"
#include "OptimusNode_ResourceAccessorBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_ComputeKernelBase)


#define LOCTEXT_NAMESPACE "OptimusKernelBase"


static FString GetShaderParamPinValueString(
	const UOptimusNodePin *InPin
	)
{
	return InPin->GetDataType()->ShaderValueType->GetZeroValueAsString();
#if 0
	// FIXME: Need property storage.
	const FShaderValueType& ValueType = *InPin->GetDataType()->ShaderValueType;
	TArrayView<UOptimusNodePin* const> SubPins = InPin->GetSubPins();
	TArray<FString> ValueArray;
	if (SubPins.IsEmpty())
	{
		ValueArray.Add(InPin->GetValueAsString());
	}
	
	// FIXME: Support all types properly. Should probably be moved to a better place.
	return FString::Printf(TEXT("%s(%s)"), *ValueType.ToString(), *InPin->GetValueAsString());
#endif
}


static void CopyValueType(FShaderValueTypeHandle InValueType,  FShaderParamTypeDefinition& OutParamDef)
{
	OutParamDef.ValueType = InValueType;
	OutParamDef.ArrayElementCount = 0;
	OutParamDef.ResetTypeDeclaration();
}

#if WITH_EDITOR
FOptimus_ComputeKernelResult UOptimusNode_ComputeKernelBase::CreateComputeKernel(
	UObject *InKernelSourceOuter,
	const FOptimusPinTraversalContext& InTraversalContext,
	const FOptimus_KernelInputMap& InKernelInputs,
	const FOptimus_KernelOutputMap& InKernelOutputs,
	UComputeDataInterface* InOutKernelDataInterface,
	FOptimus_InterfaceBindingMap& OutInputDataBindings,
	FOptimus_InterfaceBindingMap& OutOutputDataBindings
) const
{
	// Maps friendly name to unique name for each struct type
	TMap<FName, FName> StructTypeDefs;

	auto ReturnError = [](FText InErrorText) -> FOptimus_ComputeKernelResult
	{
		return FOptimus_ComputeKernelResult(TInPlaceType<FText>(), InErrorText);
	};

	auto GetStructTypeDefFromPin = [&StructTypeDefs](const UOptimusNodePin* InPin) -> TOptional<FText>
	{
		const FOptimusDataTypeHandle TypeHandle = InPin->GetDataType();
		const FShaderValueTypeHandle ShaderValueType = TypeHandle->ShaderValueType;

		// Only process shader struct types
		if (ShaderValueType->Type != EShaderFundamentalType::Struct)
		{
			return {};
		}
			
		TArray<FShaderValueTypeHandle> StructTypes = ShaderValueType->GetMemberStructTypes();
		StructTypes.Add(ShaderValueType);

		for (const FShaderValueTypeHandle StructType : StructTypes)
		{
			const FOptimusDataTypeHandle StructOptimusType = FOptimusDataTypeRegistry::Get().FindType(StructType->Name);
			if (ensure(StructOptimusType))
			{
				if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(StructOptimusType->TypeObject))
				{
					const FName UniqueName = StructType->Name;
					const FName FriendlyNameForKernel = Optimus::GetTypeName(ScriptStruct, false);

					if (FName* FoundUniqueName = StructTypeDefs.Find(FriendlyNameForKernel))
					{
						// The same friendly name cannot be claimed by two unique types;
						if (*FoundUniqueName != UniqueName)
						{
							return FText::Format(LOCTEXT("InvalidPinFriendlyName", "Invalid unique friendly name on pin '{0}'"), FText::FromName(InPin->GetUniqueName()));
						}

						// Type is already in the map, no more actions needed
						return {};
					}

					StructTypeDefs.Add(FriendlyNameForKernel) = UniqueName;
				}
			}
		}
		return {};
	};

	for (const UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->IsGroupingPin())
		{
			for (const UOptimusNodePin* SubPin: Pin->GetSubPins())
			{
				if (TOptional<FText> Result = GetStructTypeDefFromPin(SubPin); Result.IsSet())
				{
					return ReturnError(Result.GetValue());
				}
			}
		}
		else 
		{
			if (TOptional<FText> Result = GetStructTypeDefFromPin(Pin); Result.IsSet())
			{
				return ReturnError(Result.GetValue());
			}
		}
	}

	TSet<UOptimusComponentSourceBinding*> PrimaryBindings = GetPrimaryGroupPin()->GetComponentSourceBindingsRecursively(InTraversalContext);
	
	// ValidateForCompile() should guaranteed that we don't hit this error
	if (!ensure(PrimaryBindings.Num() == 1))
	{
		return ReturnError(LOCTEXT("ZeroOrMultiplePrimaryBindings", "Primary Group has zero or more than one Component Bindings"));
	}
	
	UOptimusComponentSourceBinding* PrimaryBinding = *(PrimaryBindings.CreateIterator());
	
	UOptimusKernelSource* KernelSource = NewObject<UOptimusKernelSource>(InKernelSourceOuter);

	// Wrap functions for unconnected resource pins (or value pins) that return default values
	// (for reads) or do nothing (for writes).
	TArray<FString> GeneratedFunctions;

	bool bSupportUnifiedDispatchOnly = false;
	
	check(InOutKernelDataInterface)
	{
		TOptional<FText> Result = BindKernelDataInterfaceForComputeKernel(
			PrimaryBinding,
			InOutKernelDataInterface,
			KernelSource,
			OutInputDataBindings,
			GeneratedFunctions,
			bSupportUnifiedDispatchOnly
			);
	
		if (Result.IsSet())
		{
			return ReturnError(Result.GetValue());	
		}	
	}
	

	bool bIsUnifiedDispatch = true;
	
	int32 GroupIndex = 0;
	for (const UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->IsGroupingPin())
		{
			for (const UOptimusNodePin* SubPin: Pin->GetSubPins())
			{
				TOptional<FText> Result = ProcessInputPinForComputeKernel(
					bSupportUnifiedDispatchOnly,
					InTraversalContext, SubPin,
					GroupIndex == 0 ? FString() : Pin->GetName(),
					InKernelInputs, KernelSource,  GeneratedFunctions, OutInputDataBindings, bIsUnifiedDispatch
					);
				if (Result.IsSet())
				{
					return ReturnError(Result.GetValue());
				}
			}
			
			GroupIndex++;
		}
		else if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			TOptional<FText> Result = ProcessOutputPinForComputeKernel(
				bSupportUnifiedDispatchOnly,
				InTraversalContext, Pin,
				InKernelOutputs,
				KernelSource, GeneratedFunctions, OutInputDataBindings, OutOutputDataBindings, bIsUnifiedDispatch
				);
				
			if (Result.IsSet())
			{
				return ReturnError(Result.GetValue());
			}
		}
	}



	FString CookedSource;
	CookedSource += FString::Join(GeneratedFunctions, TEXT("\n"));
	CookedSource += "\n\n";
	CookedSource += GetKernelSourceText(bIsUnifiedDispatch);
	
	KernelSource->SetSource(CookedSource);
	KernelSource->EntryPoint = GetKernelHlslName();
	KernelSource->GroupSize = GetGroupSize();
	KernelSource->AdditionalSources = GetAdditionalSources();

	for (const TTuple<FName, FName>& NamePair : StructTypeDefs)
	{
		if (NamePair.Key != NamePair.Value)
		{
			KernelSource->DefinitionsSet.Defines.Add(FComputeKernelDefinition(NamePair.Key.ToString(), NamePair.Value.ToString()));
		}
	}

#if 0
	UE_LOGF(LogOptimusCore, Log, "Kernel: %ls [%ls]", *GetNodePath(), *GetNodeName().ToString());
	UE_LOGF(LogOptimusCore, Log, "Cooked Source:\n%ls\n", *CookedSource);
	if (!OutInputDataBindings.IsEmpty())
	{
		UE_LOGF(LogOptimusCore, Log, "Input Bindings:");
		for (const TPair<int32, FOptimus_InterfaceBinding>& Binding: OutInputDataBindings)
		{
			TArray<FShaderFunctionDefinition> Defs;
			Binding.Value.DataInterface->GetSupportedInputs(Defs);
			UE_LOGF(LogOptimusCore, Log, "  K[%d] %ls -> %ls@%d [%ls]", Binding.Key, *Binding.Value.BindingFunctionName,
				*Binding.Value.DataInterface->GetName(), Binding.Value.DataInterfaceBindingIndex,
				Defs.IsValidIndex(Binding.Value.DataInterfaceBindingIndex) ? *Defs[Binding.Value.DataInterfaceBindingIndex].Name : TEXT("<undefined>"));
		}
	}
	if (!OutOutputDataBindings.IsEmpty())
	{
		UE_LOGF(LogOptimusCore, Log, "Output Bindings:");
		for (const TPair<int32, FOptimus_InterfaceBinding>& Binding: OutOutputDataBindings)
		{
			TArray<FShaderFunctionDefinition> Defs;
			Binding.Value.DataInterface->GetSupportedOutputs(Defs);
			UE_LOGF(LogOptimusCore, Log, "  K[%d] %ls -> %ls@%d [%ls]", Binding.Key, *Binding.Value.BindingFunctionName,
				*Binding.Value.DataInterface->GetName(), Binding.Value.DataInterfaceBindingIndex,
				Defs.IsValidIndex(Binding.Value.DataInterfaceBindingIndex) ? *Defs[Binding.Value.DataInterfaceBindingIndex].Name : TEXT("<undefined>"));
		}
	}
#endif
	
	return FOptimus_ComputeKernelResult(TInPlaceType<UOptimusKernelSource*>(), KernelSource);
}
#endif // WITH_EDITOR

TOptional<FText> UOptimusNode_ComputeKernelBase::ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
{
	if (!GetExecutionDomain().IsDefined())
	{
		return LOCTEXT("NoExecutionDomain", "Kernel has undefined Execution Domain, please use the Details panel to select a valid domain");
	}
	
	auto GetStructTypeDefFromPin = [](const UOptimusNodePin* InPin) -> TOptional<FText>
	{
		const FOptimusDataTypeHandle TypeHandle = InPin->GetDataType();
		if (!TypeHandle.IsValid())
		{
			return FText::Format(LOCTEXT("InvalidPinType", "Invalid data type on pin '{0}'"), FText::FromName(InPin->GetUniqueName()));
		}

		const FShaderValueTypeHandle ShaderValueType = TypeHandle->ShaderValueType;
		if (!ShaderValueType.IsValid())
		{
			return FText::Format(LOCTEXT("InvalidPinValueType", "Invalid shader value type on pin '{0}'"), FText::FromName(InPin->GetUniqueName()));
		}

		return {};
	};

	// Collect struct types and ensure all groups have the same component source (or none) within each group
	auto VerifyPinBindings = [InContext](TSet<UOptimusComponentSourceBinding*> &InCollectedBindings, const UOptimusNodePin* InPin)-> TOptional<FText>
	{
		// Check for all pins, even if the data domain is singleton, it can have different values per invocation
		// UOptimusNode_DataInterface::ValidateForCompile also double checks
		const TSet<UOptimusComponentSourceBinding*> PinBindings = InPin->GetComponentSourceBindings(InContext);
		if (PinBindings.Num() > 1)
		{
			return FText::Format(LOCTEXT("MultipleBindingsOnPin", "Multiple component bindings arriving into pin '{0}'"), FText::FromName(InPin->GetUniqueName()));
		}
		if (InCollectedBindings.IsEmpty())
		{
			InCollectedBindings = PinBindings;
		}
		else if (!PinBindings.IsEmpty())
		{
			if (InCollectedBindings.Num() != PinBindings.Num() || !InCollectedBindings.Includes(PinBindings))
			{
				return FText::Format(LOCTEXT("IncompatibleBindingsPinGroup", "Component binding for pin '{0}' is different from the component bindings of the other pins in its group"), FText::FromName(InPin->GetUniqueName()));
			}
		}
		return {};
	};

	TArray<TSet<UOptimusComponentSourceBinding*>> BindingsPerGroup;
	
	for (const UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->IsGroupingPin())
		{
			TSet<UOptimusComponentSourceBinding*>& Bindings = BindingsPerGroup.AddDefaulted_GetRef();

			if(TOptional<FText> Result = VerifyPinBindings(Bindings, Pin); Result.IsSet())
			{
				return Result;
			}
			
			for (const UOptimusNodePin* SubPin: Pin->GetSubPins())
			{
				if(TOptional<FText> Result = VerifyPinBindings(Bindings, SubPin); Result.IsSet())
				{
					return Result;
				}

				if (TOptional<FText> Result = GetStructTypeDefFromPin(SubPin); Result.IsSet())
				{
					return Result;
				}
			}
		}
	}

	// We should have at least the primary group, which needs to have a unique component binding
	if (ensure(BindingsPerGroup.Num() >= 1) && BindingsPerGroup[0].Num() == 0)
	{
		return LOCTEXT("NoPrimaryBinding", "Primary Group has no Component Binding");
	}
	
	return {};
}






FString UOptimusNode_ComputeKernelBase::GetAtomicWriteFunctionName(
	EOptimusBufferWriteType InWriteType,
	const FString& InBindingName
	)
{
	FString FunctionName;
	
	switch(InWriteType)
	{
	case EOptimusBufferWriteType::WriteAtomicAdd:
		FunctionName = FString::Printf(TEXT("AtomicAdd%s"), *InBindingName);
		break;
	case EOptimusBufferWriteType::WriteAtomicMin:
		FunctionName = FString::Printf(TEXT("AtomicMin%s"), *InBindingName);
		break;
	case EOptimusBufferWriteType::WriteAtomicMax:
		FunctionName = FString::Printf(TEXT("AtomicMax%s"), *InBindingName);
		break;
	default:
		checkNoEntry();
		break;
	}

	return FunctionName;
}

FString UOptimusNode_ComputeKernelBase::GetReadFunctionName(const FString& InBindingName)
{
	return FString::Printf(TEXT("Read%s"), *InBindingName);
}

const TCHAR* UOptimusNode_ComputeKernelBase::GetNumThreadsReservedName()
{
	static const TCHAR* NumThreadsReservedName = TEXT("NumThreads");
	return NumThreadsReservedName;
}

#if WITH_EDITOR
TOptional<FText> UOptimusNode_ComputeKernelBase::ProcessInputPinForComputeKernel(
	bool bSupportUnifiedDispatchOnly,
	const FOptimusPinTraversalContext& InTraversalContext,
	const UOptimusNodePin* InInputPin,
	const FString& InGroupName,
	const FOptimus_KernelInputMap& InKernelInputs,
	UOptimusKernelSource* InKernelSource,
	TArray<FString>& OutGeneratedFunctions,
	FOptimus_InterfaceBindingMap& OutInputDataBindings,
	bool& bOutIsUnifiedDispatch
	) const
{
	
	FShaderParamTypeDefinition IndexParamDef;
	CopyValueType(FShaderValueType::Get(EShaderFundamentalType::Uint), IndexParamDef);

	const FShaderValueTypeHandle ValueType = InInputPin->GetDataType()->ShaderValueType;

	if (const FOptimus_KernelConnection* InputConnection = InKernelInputs.Find(InInputPin))
	{
		const UOptimusNodePin* OutputPin = InputConnection->ConnectedPin;
		const UOptimusNode* OutputNode = OutputPin->GetOwningNode();
		UComputeDataInterface* DataInterface = InputConnection->DataInterface;

		if (!DataInterface->CanSupportUnifiedDispatch() )
		{
			if (!InGroupName.IsEmpty())
			{
				return FText::Format(LOCTEXT("SecondaryGroupOnlySupportsDataInterfaceUnifiedDispatch", "Cannot connect Secondary group input binding pin '{0}' to a data interface that does not support unified dispatch, consider inserting a kernel inbetween"), FText::FromName(InInputPin->GetUniqueName()));
			}

			if (bSupportUnifiedDispatchOnly)
			{
				return FText::Format(LOCTEXT("InvalidInput_KernelOnlySupportsDataInterfaceUnifiedDispatch", "Cannot connect input binding pin '{0}' to a data interface that does not support unified dispatch, consider using a simple kernel execution domain or inserting a kernel inbetween"), FText::FromName(InInputPin->GetUniqueName()));
			}

			bOutIsUnifiedDispatch = false;
		}

		int32 DataInterfaceFuncIndex = INDEX_NONE;

		if (Cast<const IOptimusComputeKernelProvider>(OutputNode))
		{
			if (ensure(Cast<UOptimusRawBufferDataInterface>(DataInterface)))
			{
				DataInterfaceFuncIndex = UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType::Default);
			}
		}
		else if (const IOptimusDataInterfaceProvider* InterfaceProvider = Cast<const IOptimusDataInterfaceProvider>(OutputNode)) 
		{
			if (ensure(Cast<UOptimusComputeDataInterface>(DataInterface)))
			{
				DataInterfaceFuncIndex = InterfaceProvider->GetDataFunctionIndexFromPin(OutputPin);
			}
		}
		else if (const IOptimusValueProvider* ValueProvider = Cast<const IOptimusValueProvider>(OutputNode))
		{
			UOptimusGraphDataInterface* GraphDataInterface = Cast<UOptimusGraphDataInterface>(DataInterface);
			if (ensure(GraphDataInterface))
			{
				DataInterfaceFuncIndex = GraphDataInterface->FindFunctionIndex(ValueProvider->GetValueIdentifier());
				check(DataInterfaceFuncIndex != INDEX_NONE);
			}
		}
		else if (Cast<const UOptimusNode_LoopTerminal>(OutputNode))
		{
			if (ensure(Cast<const UOptimusLoopTerminalDataInterface>(DataInterface)))
			{
				DataInterfaceFuncIndex = UOptimusNode_LoopTerminal::GetDataFunctionIndexFromPin(OutputPin);
			}
		}

		TArray<FShaderFunctionDefinition> ReadFunctions;
		DataInterface->GetSupportedInputs(ReadFunctions);
		FString DataFunctionName = ReadFunctions[DataInterfaceFuncIndex].Name;
		
		// The shader function definition that exposes the function that we use to
		// read values to input into the kernel.
		FShaderFunctionDefinition FuncDef;
		FuncDef.Name = DataFunctionName;
		FuncDef.bHasReturnType = true;
		
		FShaderParamTypeDefinition ParamDef;
		CopyValueType(ValueType, ParamDef);
		FuncDef.ParamTypes.Emplace(ParamDef);

		// For resources we need the index parameter.
		if (!InInputPin->GetDataDomain().IsSingleton())
		{
			for (int32 Count = 0; Count < InInputPin->GetDataDomain().NumDimensions(); Count++)
			{
				FuncDef.ParamTypes.Add(IndexParamDef);
			}
		}

		FOptimus_InterfaceBinding InterfaceBinding;
		InterfaceBinding.DataInterface = DataInterface;
		InterfaceBinding.DataInterfaceBindingIndex = DataInterfaceFuncIndex;
		InterfaceBinding.BindingFunctionName = FString::Printf(TEXT("Read%s"), *InInputPin->GetName());
		InterfaceBinding.BindingFunctionNamespace = InGroupName;
		
		OutInputDataBindings.Add(InKernelSource->ExternalInputs.Num(), InterfaceBinding);  
		
		InKernelSource->ExternalInputs.Emplace(FuncDef);
	}
	else
	{
		// Nothing connected. Get the default value (for now).
		FString ValueStr;
		FString OptionalParamStr;
		if (InInputPin->GetDataDomain().IsSingleton())
		{
			// it is not easy to provide a default/zero value for a struct type variable, so we error out here
			if (ValueType->Type == EShaderFundamentalType::Struct)
			{
				return FText::Format(LOCTEXT("DisconnectedStructPinUnsupported", "Disconnected struct type pin '{0}' is not supported"), FText::FromName(InInputPin->GetUniqueName()));
			}
			
			ValueStr = GetShaderParamPinValueString(InInputPin);
		}
		else
		{
			ValueStr = InInputPin->GetDataType()->ShaderValueType->GetZeroValueAsString();

			// No output connections, leave a stub function. The compiler will be in charge
			// of optimizing out anything that causes us to ends up here.
			TArray<FString> StubIndexes;
			
			for (const FString &IndexName: GetIndexNamesFromDataDomain(InInputPin->GetDataDomain()))
			{
				StubIndexes.Add(*FString::Printf(TEXT("uint %s"), *IndexName));
			}
		
			OptionalParamStr = *FString::Join(StubIndexes, TEXT(", "));
		}

		FString NamespacePrefix, NamespaceSuffix;
		if (!InGroupName.IsEmpty())
		{
			NamespacePrefix = FString::Printf(TEXT("namespace %s { "), *InGroupName);
			NamespaceSuffix = TEXT(" }");
		}

		OutGeneratedFunctions.Add(
			FString::Printf(TEXT("%s%s Read%s(%s) { return %s; }%s"),
				*NamespacePrefix, *ValueType->ToString(), *InInputPin->GetName(), *OptionalParamStr, *ValueStr, *NamespaceSuffix));	
	}
	
	return {};
}


TOptional<FText> UOptimusNode_ComputeKernelBase::ProcessOutputPinForComputeKernel(
	bool bSupportUnifiedDispatchOnly,
	const FOptimusPinTraversalContext& InTraversalContext,
	const UOptimusNodePin* InOutputPin,
	const FOptimus_KernelOutputMap& InKernelOutputs,
	UOptimusKernelSource* InKernelSource,
	TArray<FString>& OutGeneratedFunctions,
	FOptimus_InterfaceBindingMap& OutInputDataBindings,
	FOptimus_InterfaceBindingMap& OutOutputDataBindings,
	bool& bOutIsUnifiedDispatch
	) const
{
	FShaderParamTypeDefinition IndexParamDef;
	CopyValueType(FShaderValueType::Get(EShaderFundamentalType::Uint), IndexParamDef);

	TArray<FString> IndexNames = GetIndexNamesFromDataDomain(InOutputPin->GetDataDomain());

	const FShaderValueTypeHandle ValueType = InOutputPin->GetDataType()->ShaderValueType;

	if (const TArray<FOptimus_KernelConnection>* KernelConnections = InKernelOutputs.Find(InOutputPin))
	{
		// If we have an output connection going to multiple data interfaces, then we
		// have to wrap them all up in a single proxy function to make it still transparent
		// to the kernel writer.
		struct FWriteConnectionDef
		{
			const UComputeDataInterface* DataInterface = nullptr;
			FString DataFunctionName;
			FString WriteToName;
		};
		TArray<FWriteConnectionDef> WriteConnectionDefs;

		// All connected kernels should share the same data interface
		UComputeDataInterface* SharedRawBufferDI = nullptr;

		for (const FOptimus_KernelConnection& KernelConnection: *KernelConnections)
		{
			const UOptimusNodePin* ConnectedPin = KernelConnection.ConnectedPin;
			const UOptimusNode *ConnectedNode = ConnectedPin ? ConnectedPin->GetOwningNode() : nullptr;
			UComputeDataInterface* DataInterface = KernelConnection.DataInterface; 

			TArray<FShaderFunctionDefinition> WriteFunctions;
			DataInterface->GetSupportedOutputs(WriteFunctions);

			if (!DataInterface->CanSupportUnifiedDispatch())
			{
				if (bSupportUnifiedDispatchOnly)
				{
					return FText::Format(LOCTEXT("InvalidOutput_KernelOnlySupportsDataInterfaceUnifiedDispatch", "Cannot connect output binding pin '{0}' to a data interface that does not support unified dispatch, consider using to a simple kernel execution domain or inserting a kernel inbetween"), FText::FromName(InOutputPin->GetUniqueName()));
				}

				bOutIsUnifiedDispatch = false;
			}
			
			if (!ConnectedNode)
			{
				// No connected node indicates the kernel output is cached in a generated raw buffer
				if (const UOptimusRawBufferDataInterface* RawBufferDataInterface = Cast<const UOptimusRawBufferDataInterface>(DataInterface);
					ensure(RawBufferDataInterface))
				{
					const int32 OutputIndex = UOptimusRawBufferDataInterface::GetWriteValueOutputIndex(EOptimusBufferWriteType::Write);
					WriteConnectionDefs.Add({RawBufferDataInterface, WriteFunctions[OutputIndex].Name, TEXT("RawBuffer")});
				}
			}
			else
			{
				if (const IOptimusDataInterfaceProvider* InterfaceProvider = Cast<const IOptimusDataInterfaceProvider>(ConnectedNode);
					ensure(InterfaceProvider))
				{
					int32 DataFunctionIndex = InterfaceProvider->GetDataFunctionIndexFromPin(ConnectedPin);
					TArray<FShaderFunctionDefinition> FunctionDefinitions;
					DataInterface->GetSupportedOutputs(FunctionDefinitions);
					FString DataFunctionName = FunctionDefinitions[DataFunctionIndex].Name;

					WriteConnectionDefs.Add({DataInterface, DataFunctionName, ConnectedPin->GetName()});
				}	
			}
		}

		TArray<FString> WrapFunctionNameCalls;

		for (int32 WriteConnectionIndex = 0; WriteConnectionIndex < WriteConnectionDefs.Num(); ++WriteConnectionIndex)
		{
			const FWriteConnectionDef& WriteConnectionDef = WriteConnectionDefs[WriteConnectionIndex];
			const FString DataFunctionName = WriteConnectionDef.DataFunctionName;
			FShaderFunctionDefinition FuncDef;
			FuncDef.Name = DataFunctionName;
			FuncDef.bHasReturnType = false;
		
			FShaderParamTypeDefinition ParamDef;
			CopyValueType(ValueType, ParamDef);
			for (int32 Count = 0; Count < IndexNames.Num(); Count++)
			{
				FuncDef.ParamTypes.Add(IndexParamDef);
			}
			FuncDef.ParamTypes.Emplace(ParamDef);

			TArray<FShaderFunctionDefinition> WriteFunctions;
			WriteConnectionDef.DataInterface->GetSupportedOutputs(WriteFunctions);
			int32 DataInterfaceFuncIndex = WriteFunctions.IndexOfByPredicate([DataFunctionName](const FShaderFunctionDefinition &InDef) { return DataFunctionName == InDef.Name; });
		
			FString WrapFunctionName;
			if (WriteConnectionDefs.Num() > 1)
			{
				WrapFunctionName = FString::Printf(TEXT("Write%sTo%s%d"), *InOutputPin->GetName(), *WriteConnectionDef.WriteToName, WriteConnectionIndex);
				WrapFunctionNameCalls.Add(FString::Printf(TEXT("    %s(%s, Value)"), *WrapFunctionName, *FString::Join(IndexNames, TEXT(", "))));
			}
			else
			{
				WrapFunctionName = FString::Printf(TEXT("Write%s"), *InOutputPin->GetName());
			}
			
			FOptimus_InterfaceBinding InterfaceBinding;
			InterfaceBinding.DataInterface = WriteConnectionDef.DataInterface;
			InterfaceBinding.DataInterfaceBindingIndex = DataInterfaceFuncIndex;
			InterfaceBinding.BindingFunctionName = WrapFunctionName;
			
			OutOutputDataBindings.Add(InKernelSource->ExternalOutputs.Num(), InterfaceBinding);
			InKernelSource->ExternalOutputs.Emplace(FuncDef);
		}

		if (!WrapFunctionNameCalls.IsEmpty())
		{
			TArray<FString> IndexParamNames;
			for (const FString& IndexName: IndexNames)
			{
				IndexParamNames.Add(FString::Printf(TEXT("uint %s"), *IndexName));
			}
			
			// Add a wrapper function that calls all the write functions in one shot.
			OutGeneratedFunctions.Add(
				FString::Printf(TEXT("void Write%s(%s, %s Value)\n{\n%s;\n}"),
					*InOutputPin->GetName(), *FString::Join(IndexParamNames, TEXT(", ")), *ValueType->ToString(), *FString::Join(WrapFunctionNameCalls, TEXT(";\n"))));
		}	
		
		if (DoesOutputPinSupportAtomic(InOutputPin) || DoesOutputPinSupportRead(InOutputPin))
		{
			// Read access on output pins guarantees existence of raw buffer data interface
			if (ensure(KernelConnections->Num() > 0))
			{
				// We may have multiple connections but they should all be tie to the same raw buffer data interface, so just use the first connection
				if (UOptimusRawBufferDataInterface* RawBufferDataInterface = Cast<UOptimusRawBufferDataInterface>((*KernelConnections)[0].DataInterface))
				{
					if (DoesOutputPinSupportAtomic(InOutputPin))
					{
						TArray<FShaderFunctionDefinition> WriteFunctions;
						RawBufferDataInterface->GetSupportedOutputs(WriteFunctions);
					
						for (uint8 WriteType = uint8(EOptimusBufferWriteType::Write) + 1; WriteType < uint8(EOptimusBufferWriteType::Count); WriteType++)
						{

							FString WrapFunctionName = GetAtomicWriteFunctionName((EOptimusBufferWriteType)WriteType, InOutputPin->GetName());

							const int32 OutputIndex = UOptimusRawBufferDataInterface::GetWriteValueOutputIndex((EOptimusBufferWriteType)WriteType);
						
							FShaderFunctionDefinition FuncDef = WriteFunctions[OutputIndex];
							for (int32 Index = 0; Index < FuncDef.ParamTypes.Num(); Index++)
							{
								FuncDef.ParamTypes[Index].ResetTypeDeclaration();
							}
					
							FOptimus_InterfaceBinding InterfaceBinding;
							InterfaceBinding.DataInterface = RawBufferDataInterface;
							InterfaceBinding.DataInterfaceBindingIndex = OutputIndex;
							InterfaceBinding.BindingFunctionName = WrapFunctionName;
					
							OutOutputDataBindings.Add(InKernelSource->ExternalOutputs.Num(), InterfaceBinding);
							InKernelSource->ExternalOutputs.Emplace(FuncDef);	
						}
					}

					if (DoesOutputPinSupportRead(InOutputPin))
					{
						TArray<FShaderFunctionDefinition> ReadFunctions;
						RawBufferDataInterface->GetSupportedInputs(ReadFunctions);
						int32 InputIndex = UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType::ForceUAV);

						FShaderFunctionDefinition FuncDef = ReadFunctions[InputIndex];
						for (int32 Index = 0; Index < FuncDef.ParamTypes.Num(); Index++)
						{
							FuncDef.ParamTypes[Index].ResetTypeDeclaration();
						}
					
						FOptimus_InterfaceBinding InterfaceBinding;
						InterfaceBinding.DataInterface = RawBufferDataInterface;
						InterfaceBinding.DataInterfaceBindingIndex = InputIndex;
						InterfaceBinding.BindingFunctionName = GetReadFunctionName(InOutputPin->GetName());
					
						OutInputDataBindings.Add(InKernelSource->ExternalInputs.Num(), InterfaceBinding);
						InKernelSource->ExternalInputs.Emplace(FuncDef);		
					}
				}
			}
		}
	}
	else
	{
		// No output connections, leave a stub function. The compiler will be in charge
		// of optimizing out anything that causes us to ends up here.
		TArray<FString> StubIndexes;

		for (const FString &IndexName: IndexNames)
		{
			StubIndexes.Add(*FString::Printf(TEXT("uint %s"), *IndexName));
		}
		
		OutGeneratedFunctions.Add(
			FString::Printf(TEXT("void Write%s(%s, %s Value) { }"),
				*InOutputPin->GetName(), *FString::Join(StubIndexes, TEXT(", ")), *ValueType->ToString()));
	}

	return {};
}

TOptional<FText> UOptimusNode_ComputeKernelBase::BindKernelDataInterfaceForComputeKernel(
	const UOptimusComponentSourceBinding* InKernelPrimaryComponentSourceBinding,
	UComputeDataInterface* InOutKernelDataInterface,
	UOptimusKernelSource* InKernelSource,
	FOptimus_InterfaceBindingMap& OutInputDataBindings,
	TArray<FString>& OutGeneratedFunctions,
	bool& bOutUnifiedDispatchOnly
	) const
{
	check(InOutKernelDataInterface);
	check(InKernelPrimaryComponentSourceBinding);
	
	IOptimusComputeKernelDataInterface* KernelDataInterface = Cast<IOptimusComputeKernelDataInterface>(InOutKernelDataInterface);
	check(KernelDataInterface);
	FString NumThreadsExpression = GetExecutionDomain().AsExpression();
	KernelDataInterface->SetExecutionDomain(NumThreadsExpression);
	KernelDataInterface->SetComponentBinding(InKernelPrimaryComponentSourceBinding);
	
	TVariant<bool, FText> UnifiedDispatchResult = Optimus::IsExecutionDomainUnifiedDispatchOnly(NumThreadsExpression,InKernelPrimaryComponentSourceBinding->GetComponentSource());

	if (UnifiedDispatchResult.IsType<FText>())
	{
		return UnifiedDispatchResult.Get<FText>();
	}

	bOutUnifiedDispatchOnly = UnifiedDispatchResult.Get<bool>();
	
	TArray<FShaderFunctionDefinition> ReadFunctions;
	InOutKernelDataInterface->GetSupportedInputs(ReadFunctions);	

	// Simply grab everything the kernel data interface has to offer
	for (int32 FuncIndex = 0; FuncIndex < ReadFunctions.Num(); FuncIndex++)
	{
		FShaderFunctionDefinition FuncDef = ReadFunctions[FuncIndex];

		for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
		{
			// Making sure parameter has type declaration generated
			CopyValueType(ParamType.ValueType, ParamType);
		}

		FOptimus_InterfaceBinding InterfaceBinding;
		InterfaceBinding.DataInterface = InOutKernelDataInterface;
		InterfaceBinding.DataInterfaceBindingIndex = FuncIndex;
		InterfaceBinding.BindingFunctionName = FuncDef.Name;
		InterfaceBinding.BindingFunctionNamespace = Optimus::GetKernelInternalNamespaceName() ;
				
		OutInputDataBindings.Add(InKernelSource->ExternalInputs.Num(), InterfaceBinding);  
				
		InKernelSource->ExternalInputs.Emplace(FuncDef);	
	}

	OutGeneratedFunctions.Add(FString::Printf(
			TEXT("uint3 Read%s() { return %s::%s(); }"),
			GetNumThreadsReservedName(), Optimus::GetKernelInternalNamespaceName(), KernelDataInterface->GetReadNumThreadsFunctionName() ));

	return {};
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
