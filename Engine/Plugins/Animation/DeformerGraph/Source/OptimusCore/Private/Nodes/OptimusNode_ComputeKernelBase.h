// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusComputeKernelProvider.h"
#include "OptimusDataDomain.h"
#include "OptimusDataType.h"
#include "OptimusExecutionDomain.h"
#include "OptimusNode.h"

#include "OptimusNode_ComputeKernelBase.generated.h"

enum class EOptimusBufferWriteType : uint8;
struct FOptimusKernelConstantContainer;
class UComputeSource;


UCLASS(Abstract)
class UOptimusNode_ComputeKernelBase :
	public UOptimusNode, 
	public IOptimusComputeKernelProvider
{
	GENERATED_BODY()
	
public:
	/** Implement this to return the HLSL kernel's function name */
	virtual FString GetKernelHlslName() const PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::GetKernelHlslName, return FString();)

	/** Implement this to return the HLSL kernel's function name */
	virtual FIntVector GetGroupSize() const PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::GetGroupSize, return FIntVector();)

	/** Implement this to return the complete HLSL code for this kernel */
	virtual FString GetKernelSourceText(bool bInIsUnifiedDispatch) const PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::GetKernelSourceText, return FString();)

	/** Implement this to return additional source code that the kernel requires. */
	virtual TArray<TObjectPtr<UComputeSource>> GetAdditionalSources() const { return {}; }
	
#if WITH_EDITOR
	// IOptimusComputeKernelProvider
	FOptimus_ComputeKernelResult CreateComputeKernel(
		UObject* InKernelSourceOuter,
		const FOptimusPinTraversalContext& InTraversalContext,
		const FOptimus_KernelInputMap& InKernelInputs,
		const FOptimus_KernelOutputMap& InKernelOutputs,
		UComputeDataInterface* InOutKernelDataInterface,
		FOptimus_InterfaceBindingMap& OutInputDataBindings,
		FOptimus_InterfaceBindingMap& OutOutputDataBindings
	) const override;
#endif

	FOptimusExecutionDomain GetExecutionDomain() const override PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::GetExecutionDomain, return {}; );
	const UOptimusNodePin* GetPrimaryGroupPin() const override PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::GetPrimaryGroupPin, return {}; );  
	UComputeDataInterface* MakeKernelDataInterface(UObject* InOuter) const override PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::MakeKernelDataInterface, return {}; );
	bool DoesOutputPinSupportAtomic(const UOptimusNodePin* InPin) const override PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::DoesOutputPinSupportAtomic, return false; );
	bool DoesOutputPinSupportRead(const UOptimusNodePin* InPin) const override PURE_VIRTUAL(UOptimusNode_ComputeKernelBase::DoesOutputPinSupportRead, return false; );
	
	// -- UOptimusNode overrides
	TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const override;
	
protected:
	static TArray<FString> GetIndexNamesFromDataDomainLevels(
		const TArray<FName> &InLevelNames
		)
	{
		TArray<FString> IndexNames;
	
		for (FName DomainName: InLevelNames)
		{
			IndexNames.Add(FString::Printf(TEXT("%sIndex"), *DomainName.ToString()));
		}
		return IndexNames;
	}

	static TArray<FString> GetIndexNamesFromDataDomain(
		const FOptimusDataDomain &InDomain
		)
	{
		TArray<FString> IndexNames;
	
		if (InDomain.Type == EOptimusDataDomainType::Dimensional)
		{
			TArray<FName> LevelNames = InDomain.DimensionNames;
			IndexNames = GetIndexNamesFromDataDomainLevels(LevelNames);
		}
		else if (InDomain.Type == EOptimusDataDomainType::Expression)
		{
			IndexNames = {TEXT("Index")};
		}
		else
		{
			checkNoEntry();
		}

		return IndexNames;
	}
	
	static FString GetAtomicWriteFunctionName(EOptimusBufferWriteType InWriteType, const FString& InBindingName);
	static FString GetReadFunctionName(const FString& InBindingName);
	static const TCHAR* GetNumThreadsReservedName();
	
#if WITH_EDITOR
private:
	TOptional<FText> BindKernelDataInterfaceForComputeKernel(
		const UOptimusComponentSourceBinding* InKernelPrimaryComponentSourceBinding,
		UComputeDataInterface* InOutKernelDataInterface,
		UOptimusKernelSource* InKernelSource,
		FOptimus_InterfaceBindingMap& OutInputDataBindings,
		TArray<FString>& OutGeneratedFunctions,
		bool& bOutUnifiedDispatchOnly
		) const;
	
	TOptional<FText> ProcessInputPinForComputeKernel(
		bool bSupportUnifiedDispatchOnly,
		const FOptimusPinTraversalContext& InTraversalContext,
		const UOptimusNodePin* InInputPin,
		const FString& InGroupName,
		const FOptimus_KernelInputMap& InKernelInputs,
		UOptimusKernelSource* InKernelSource,
		TArray<FString>& OutGeneratedFunctions,
		FOptimus_InterfaceBindingMap& OutInputDataBindings,
		bool& bOutIsUnifiedDispatch
		) const;

	TOptional<FText> ProcessOutputPinForComputeKernel(
	bool bSupportUnifiedDispatchOnly,
		const FOptimusPinTraversalContext& InTraversalContext,
		const UOptimusNodePin* InOutputPin,
		const FOptimus_KernelOutputMap& InKernelOutputs,
		UOptimusKernelSource* InKernelSource,
		TArray<FString>& OutGeneratedFunctions,
		FOptimus_InterfaceBindingMap& OutInputDataBindings,
		FOptimus_InterfaceBindingMap& OutOutputDataBindings,
		bool& bOutIsUnifiedDispatch
		) const;
#endif // WITH_EDITOR
};

