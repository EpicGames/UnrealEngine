// Copyright Epic Games, Inc. All Rights Reserved.


#include "OptimusNode_ComputeKernelFunction.h"

#include "OptimusHelpers.h"
#include "OptimusNodePin.h"
#include "OptimusShaderText.h"
#include "OptimusBindingTypes.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_ComputeKernelFunction)

UClass* UOptimusNode_ComputeKernelFunctionGeneratorClass::CreateNodeClass(
	UObject* InPackage,
	FName InCategory,
	FName InKernelName,
	FIntVector InGroupSize,
	const TArray<FOptimusParameterBinding>& InInputBindings,
	const TArray<FOptimusParameterBinding>& InOutputBindings,
	const FString& InShaderSource
	)
{
	if (!ensure(InPackage) || !ensure(!InCategory.IsNone()) || !ensure(!InKernelName.IsNone()) ||
		!ensure(InGroupSize.GetMin() > 0) || !ensure(!InShaderSource.IsEmpty()))
	{
		return nullptr;
	}

	// We have to have at least one input binding and one output binding.
	if (!ensure(InInputBindings.Num() >= 1) || !ensure(InOutputBindings.Num() >= 1))
	{
		return nullptr;
	}

	for (const FOptimusParameterBinding& Binding: InInputBindings)
	{
		if (!ensure(Binding.IsValid()))
		{
			return nullptr;
		}
	}
	for (const FOptimusParameterBinding& Binding: InOutputBindings)
	{
		if (!ensure(Binding.IsValid()))
		{
			return nullptr;
		}
	}
	
	FName ClassName(TEXT("Optimus_ComputeKernel_") + InKernelName.ToString());

	ClassName = Optimus::GetUniqueNameForScope(InPackage, ClassName);

	UClass *ParentClass = UOptimusNode_ComputeKernelFunction::StaticClass();
	
	UOptimusNode_ComputeKernelFunctionGeneratorClass* KernelClass =
		NewObject<UOptimusNode_ComputeKernelFunctionGeneratorClass>(InPackage, ClassName, RF_Standalone|RF_Public);
	KernelClass->SetSuperStruct(ParentClass);
	KernelClass->PropertyLink = ParentClass->PropertyLink;

	// Copy in the static state
	KernelClass->Category = InCategory;
	KernelClass->KernelName = InKernelName;
	KernelClass->GroupSize = InGroupSize;
	KernelClass->InputBindings = InInputBindings;
	KernelClass->OutputBindings = InOutputBindings;
	KernelClass->ShaderSource = InShaderSource;

	// Append new properties to the existing ones.
	FField::FLinkedListBuilder PropertyListBuilder(&KernelClass->ChildProperties);
	PropertyListBuilder.MoveToEnd();

	//TMap<const FProperty*, const TArray<uint8>*> PropertyValues;
	for (const FOptimusParameterBinding& InputBinding: InInputBindings)
	{
		if (InputBinding.DataDomain.IsSingleton())
		{
			FProperty *Property = InputBinding.DataType->CreateProperty(KernelClass, InputBinding.Name);

			// Update the property so that it is editable in 
			Property->PropertyFlags |= CPF_Edit;
#if WITH_EDITOR
			Property->SetMetaData(TEXT("Category"), InKernelName.ToString() + TEXT(" Settings"));
#endif

			/** FIXME: When parameter evaluation is available, update this to copy the 
			 *  evaluated value from the binding connection.
			if (!ParameterBinding.RawValue.IsEmpty())
			{
				PropertyValues.Add(Property, &ParameterBinding.RawValue);
			}
			*/

			PropertyListBuilder.AppendNoTerminate(*Property);
		}
	}

	// Finalize the class
	KernelClass->Bind();
	KernelClass->StaticLink(true);
	KernelClass->AddToRoot();

	// Grab the CDO and update the default values based on the raw values in the value bindings.
	/*UOptimusNode_ComputeKernelFunction *KernelCDO = Cast<UOptimusNode_ComputeKernelFunction>(KernelClass->GetDefaultObject());

	// Copy the default values from the incoming properties.
	for (const TPair<const FProperty*, const TArray<uint8>*>& Item: PropertyValues)
	{
		const FProperty* Property = Item.Key;
		const TArray<uint8>& RawValue = *Item.Value;

		Property->CopyCompleteValue(Property->ContainerPtrToValuePtr<UOptimusNode_ComputeKernelFunction>(KernelCDO), RawValue.GetData());
	}*/
	
	return KernelClass;
}


// This gets called when a new UObject gets constructed from this class, once all properties have
// been zero-intialized. There doesn't seem to be any other clean mechanism to simply copy from 
// the CDO during construction.
void UOptimusNode_ComputeKernelFunctionGeneratorClass::InitPropertiesFromCustomList(
	uint8* InObjectPtr,
	const uint8* InCDOPtr
	)
{
	if (!InObjectPtr || !InCDOPtr || !ensure(InObjectPtr != static_cast<const void*>(GetDefaultObject())))
	{
		return;
	}

	// We want to copy all properties.
	for (TFieldIterator<FProperty> PropIt(this); PropIt; ++PropIt)
	{
		const FProperty* Property = *PropIt;

		Property->CopyCompleteValue_InContainer(InObjectPtr, InCDOPtr);
	}
}


void UOptimusNode_ComputeKernelFunctionGeneratorClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	UClass::Link(Ar, bRelinkExistingProperties);

	// Force assembly of the reference token stream so that we can be properly handled by the
	// garbage collector.
	AssembleReferenceTokenStream(/*bForce=*/true);
}


UOptimusNode_ComputeKernelFunction::UOptimusNode_ComputeKernelFunction()
{
}


FText UOptimusNode_ComputeKernelFunction::GetDisplayName() const
{
	return FText::FromString(FName::NameToDisplayString(GetGeneratorClass()->KernelName.ToString(), false));
}


FName UOptimusNode_ComputeKernelFunction::GetNodeCategory() const
{
	return GetGeneratorClass()->Category;
}


FString UOptimusNode_ComputeKernelFunction::GetKernelHlslName() const
{
	return GetGeneratorClass()->KernelName.ToString();
}


FIntVector UOptimusNode_ComputeKernelFunction::GetGroupSize() const
{
	return GetGeneratorClass()->GroupSize;
}


FString UOptimusNode_ComputeKernelFunction::GetKernelSourceText(bool bInIsUnifiedDispatch) const
{
	checkNoEntry();
	return {};
}


void UOptimusNode_ComputeKernelFunction::ConstructNode()
{
	UOptimusNode_ComputeKernelFunctionGeneratorClass *NodeClass = GetGeneratorClass();  
	
	for (const FOptimusParameterBinding& Binding: NodeClass->InputBindings)
	{
		AddPinDirect(Binding.Name, EOptimusNodePinDirection::Input, Binding.DataDomain, Binding.DataType);
	}
	for (const FOptimusParameterBinding& Binding: NodeClass->OutputBindings)
	{
		AddPinDirect(Binding.Name, EOptimusNodePinDirection::Output, Binding.DataDomain, Binding.DataType);
	}
}


FOptimusExecutionDomain UOptimusNode_ComputeKernelFunction::GetExecutionDomain() const
{
	return GetGeneratorClass()->ExecutionDomain;
}


UOptimusNode_ComputeKernelFunctionGeneratorClass* UOptimusNode_ComputeKernelFunction::GetGeneratorClass() const
{
	return Cast<UOptimusNode_ComputeKernelFunctionGeneratorClass>(GetClass());
}
