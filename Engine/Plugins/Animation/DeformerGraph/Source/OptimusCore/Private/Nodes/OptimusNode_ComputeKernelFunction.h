// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode_ComputeKernelBase.h"
#include "OptimusBindingTypes.h"
#include "OptimusExecutionDomain.h"

#include "OptimusNode_ComputeKernelFunction.generated.h"

#define UE_API OPTIMUSCORE_API


UCLASS()
class UOptimusNode_ComputeKernelFunctionGeneratorClass :
	public UClass
{
	GENERATED_BODY()
public:
	static UClass *CreateNodeClass(
		UObject* InPackage,
		FName InCategory,
		FName InKernelName,
		FIntVector InGroupSize,
		const TArray<FOptimusParameterBinding>& InInputBindings,
		const TArray<FOptimusParameterBinding>& InOutputBindings,
		const FString& InShaderSource
		);

	// UClass overrides
	void InitPropertiesFromCustomList(uint8* InObjectPtr, const uint8* InCDOPtr) override;
	void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	
	UPROPERTY()
	FName Category;

	UPROPERTY()
	FName KernelName;

	UPROPERTY()
	FOptimusExecutionDomain ExecutionDomain;

	UPROPERTY()
	FIntVector GroupSize;

	UPROPERTY()
	TArray<FOptimusParameterBinding> InputBindings;

	UPROPERTY()
	TArray<FOptimusParameterBinding> OutputBindings;

	UPROPERTY()
	FString ShaderSource;
};


/**
 * 
 */
UCLASS(MinimalAPI, Hidden)
class UOptimusNode_ComputeKernelFunction :
	public UOptimusNode_ComputeKernelBase
{
	GENERATED_BODY()

public:
	UE_API UOptimusNode_ComputeKernelFunction();

	// UOptimusNode overrides
	UE_API FText GetDisplayName() const override;
	UE_API FName GetNodeCategory() const override; 

	// UOptimusNode_ComputeKernelBase overrides
	UE_API FString GetKernelHlslName() const override;
	UE_API FIntVector GetGroupSize() const override;
	UE_API FString GetKernelSourceText(bool bInIsUnifiedDispatch) const override;

	UE_API void ConstructNode() override;
	
	// IOptimusComputeKernelProvider
	UE_API FOptimusExecutionDomain GetExecutionDomain() const override;
	const UOptimusNodePin* GetPrimaryGroupPin() const override { return {}; }
	UComputeDataInterface* MakeKernelDataInterface(UObject* InOuter) const override { return nullptr; };
	bool DoesOutputPinSupportAtomic(const UOptimusNodePin* InPin) const override {return false;};
	bool DoesOutputPinSupportRead(const UOptimusNodePin* InPin) const override {return false;};
private:
	UE_API UOptimusNode_ComputeKernelFunctionGeneratorClass *GetGeneratorClass() const;
};

#undef UE_API
