// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_ConstantValue.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusValue.h"
#include "OptimusNodePin.h"
#include "OptimusNodeGraph.h"

#include "OptimusHelpers.h"
#include "OptimusValueContainerStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_ConstantValue)

void UOptimusNode_ConstantValueGeneratorClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	// Force assembly of the reference token stream so that we can be properly handled by the
	// garbage collector.
	AssembleReferenceTokenStream(/*bForce=*/true);
}


UClass* UOptimusNode_ConstantValueGeneratorClass::GetClassForType(UPackage* InPackage, FOptimusDataTypeRef InDataType)
{
	const FString ClassName = TEXT("OptimusNode_ConstantValue_") + InDataType.TypeName.ToString();

	// Check if the package already owns this class.
	UOptimusNode_ConstantValueGeneratorClass *TypeClass = FindObject<UOptimusNode_ConstantValueGeneratorClass>(InPackage, *ClassName);
	if (!TypeClass)
	{
		UClass *ParentClass = UOptimusNode_ConstantValue::StaticClass();
		// Construct a value node class for this data type
		TypeClass = NewObject<UOptimusNode_ConstantValueGeneratorClass>(InPackage, *ClassName, RF_Standalone|RF_Public);
		TypeClass->SetSuperStruct(ParentClass);
		TypeClass->PropertyLink = ParentClass->PropertyLink;

		// Nodes of this type should not be listed in the node palette.
		TypeClass->ClassFlags |= CLASS_Hidden;

		// Stash the data type so that the node can return it later.
		TypeClass->DataType = InDataType;

		// Create the property chain that represents this value.
		FProperty* InputValueProp = InDataType->CreateProperty(TypeClass, "Value");
		InputValueProp->PropertyFlags |= CPF_Edit;
#if WITH_EDITOR
		InputValueProp->SetMetaData(UOptimusNode::PropertyMeta::Input, TEXT("1"));
		InputValueProp->SetMetaData(UOptimusNode::PropertyMeta::Category, TEXT("Value"));
#endif

		// Out value doesn't need storage or saving.
		FProperty* OutputValueProp = InDataType->CreateProperty(TypeClass, "Out");
		// OutputValueProp->SetFlags(RF_Transient); // todo: SetFlags(RF_Transient) is a deprecated no-op, consider using SetPropertyFlags(CPF_Transient)
#if WITH_EDITOR
		OutputValueProp->SetMetaData(UOptimusNode::PropertyMeta::Output, TEXT("1"));
#endif

		// AddCppProperty chains backwards.
		TypeClass->AddCppProperty(OutputValueProp);
		TypeClass->AddCppProperty(InputValueProp);

		// Finalize the class
		TypeClass->Bind();
		TypeClass->StaticLink(true);
		TypeClass->AddToRoot();

		// Make sure the CDO exists.
		(void)TypeClass->GetDefaultObject();
	}
	return TypeClass;
}


void UOptimusNode_ConstantValue::PostLoadNodeSpecificData()
{
	Super::PostLoadNodeSpecificData();

	if (!GetClass()->GetOuter()->IsA<UPackage>())
	{
		// This class should be parented to the package instead of the asset object
		// because the engine no longer supports asset object as uclass outer
		Optimus::RenameObject(GetClass(), nullptr, GetPackage());
	}
}

FOptimusValueContainerStruct UOptimusNode_ConstantValue::GetValue() const
{
	FOptimusValueContainerStruct ValueContainer;
	
	const UOptimusNodePin *ValuePin = FindPinFromPath({TEXT("Value")});
	if (ensure(ValuePin))
	{
		const FProperty *ValueProperty = ValuePin->GetPropertyFromPin();
		FOptimusDataTypeRef DataType = GetValueDataType();
		if (ensure(ValueProperty) && ensure(DataType.IsValid()))
		{
			TArrayView<const uint8> ValueData(ValueProperty->ContainerPtrToValuePtr<uint8>(this), ValueProperty->GetSize());

			ValueContainer.SetType(DataType);
			ValueContainer.SetValue(DataType, ValueData);
		}
	}

	return ValueContainer;
}


#if WITH_EDITOR

void UOptimusNode_ConstantValue::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UOptimusNodeGraph* Graph = GetOwningGraph();
	if (UOptimusNodePin* ValuePin = FindPinFromPath({ TEXT("Value") }))
	{
		Graph->Notify(EOptimusGraphNotifyType::PinValueChanged, ValuePin);
	}
	Graph->GlobalNotify(EOptimusGlobalNotifyType::ConstantValueChanged, this);
}

#endif // WITH_EDITOR


FOptimusValueIdentifier UOptimusNode_ConstantValue::GetValueIdentifier() const
{
	return {EOptimusValueType::Constant, Optimus::GetSanitizedNameForHlsl(*GetNodePath())};
}

FOptimusDataTypeRef UOptimusNode_ConstantValue::GetValueDataType() const
{
	const UOptimusNode_ConstantValueGeneratorClass* Class = GetGeneratorClass();
	if (ensure(Class))
	{
		return Class->DataType;
	}
	return {};
}

FTopLevelAssetPath UOptimusNode_ConstantValue::GetAssetPathForClassDefiner() const
{
	return StaticClass()->GetClassPathName();
}


FString UOptimusNode_ConstantValue::GetClassCreationString() const
{
	if (GetGeneratorClass())
	{
		return FString::Printf(TEXT("DataType=%s"), *GetGeneratorClass()->DataType->TypeName.ToString()); 
	}
	return {};
}


UClass* UOptimusNode_ConstantValue::GetClassFromCreationString(
	UPackage* InPackage,
	const TCHAR* InCreationString
	) const
{
	FName DataTypeName;
	if (!FParse::Value(InCreationString, TEXT("DataType="), DataTypeName))
	{
		return nullptr;
	}

	FOptimusDataTypeHandle FoundDataType = FOptimusDataTypeRegistry::Get().FindType(DataTypeName);
	if (!FoundDataType.IsValid())
	{
		return nullptr;
	}

	return UOptimusNode_ConstantValueGeneratorClass::GetClassForType(InPackage, FoundDataType);
}


void UOptimusNode_ConstantValue::ConstructNode()
{
	SetDisplayName(FText::Format(FText::FromString(TEXT("{0} Constant")), GetValueDataType()->DisplayName));

	UOptimusNode::ConstructNode();
}


UOptimusNode_ConstantValueGeneratorClass* UOptimusNode_ConstantValue::GetGeneratorClass() const
{
	return Cast<UOptimusNode_ConstantValueGeneratorClass>(GetClass());	
}
