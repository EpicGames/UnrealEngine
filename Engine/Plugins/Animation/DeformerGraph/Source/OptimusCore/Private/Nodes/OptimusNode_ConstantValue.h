// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusGeneratedClassDefiner.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "IOptimusValueProvider.h"
#include "OptimusNode.h"
#include "OptimusDataType.h"

#include "OptimusNode_ConstantValue.generated.h"

struct FOptimusDataTypeRef;

UCLASS()
class UOptimusNode_ConstantValueGeneratorClass :
	public UClass
{
	GENERATED_BODY()

	// DECLARE_WITHIN(UObject) is only kept for back-compat, please don't parent the class
	// to the asset object.
	// This class should be parented to the package, instead of the asset object
	// because the engine no longer asset object as UClass outer
	// however, since in the past we have parented the class to the asset object
	// this flag has to be kept such that we can load the old asset in the first place and
	// re-parent it back to the package in post load
	DECLARE_WITHIN(UObject)
	
public:
	// UClass overrides
	void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	
	static UClass *GetClassForType(
		UPackage* InPackage,
		FOptimusDataTypeRef InDataType
		);

	UPROPERTY()
	FOptimusDataTypeRef DataType;
};

UCLASS(Hidden)
class UOptimusNode_ConstantValue :
	public UOptimusNode,
	public IOptimusValueProvider,
	public IOptimusGeneratedClassDefiner,
	public IOptimusPinMutabilityDefiner
{
	GENERATED_BODY()

public:
	
	FName GetNodeCategory() const override
	{
		return CategoryName::Values; 
	}


	void PostLoadNodeSpecificData() override;

#if WITH_EDITOR
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// IOptimusValueProvider overrides 
	FOptimusValueIdentifier GetValueIdentifier() const override;
	FOptimusDataTypeRef GetValueDataType() const override;
	FOptimusValueContainerStruct GetValue() const override;

	// IOptimusGeneratedClassDefiner implementation
	FTopLevelAssetPath GetAssetPathForClassDefiner() const override;
	FString GetClassCreationString() const override;
	UClass* GetClassFromCreationString(
		UPackage* InPackage,
		const TCHAR* InCreationString
		) const override;

	// IOptimusPinMutabilityDefiner implementation
	EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const override { return EOptimusPinMutability::Immutable; };
	
	
protected:
	void ConstructNode() override;
	
	UOptimusNode_ConstantValueGeneratorClass* GetGeneratorClass() const;
};
