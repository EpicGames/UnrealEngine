// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_CustomComputeKernel.h"

#include "OptimusActionStack.h"
#include "OptimusComponentSource.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusNode_DataInterface.h"
#include "OptimusObjectVersion.h"
#include "Actions/OptimusNodeActions.h"
#include "ComponentSources/OptimusSkeletalMeshComponentSource.h"
#include "ComponentSources/OptimusSkinnedMeshComponentSource.h"
#include "DataInterfaces/OptimusDataInterfaceCustomComputeKernel.h"
#include "IOptimusDeprecatedExecutionDataInterface.h"
#include "IOptimusUnnamedNodePinProvider.h"
#include "OptimusNode_ResourceAccessorBase.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_CustomComputeKernel)

#define LOCTEXT_NAMESPACE "OptimusNode_CustomComputeKernel"

static const FName DefaultKernelName("MyKernel");
static const FName DefaultGroupName("Group"); 
static const FName InputBindingsName= GET_MEMBER_NAME_CHECKED(UOptimusNode_CustomComputeKernel, InputBindingArray);
static const FName OutputBindingsName = GET_MEMBER_NAME_CHECKED(UOptimusNode_CustomComputeKernel, OutputBindingArray);
static const FName ExtraInputBindingGroupsName = GET_MEMBER_NAME_CHECKED(UOptimusNode_CustomComputeKernel, SecondaryInputBindingGroups);

static const FName AdderPinActionKeyAddNewGroup("@NewGroup");

// Secondary group name cannot have spaces, so putting a space here should guard against name collision
static const FName PrimaryGroupPinName = TEXT("Primary Group");

static FString GetShaderValueTypeFriendlyName(const FOptimusDataTypeRef& InDataType)
{
	FName FriendlyName = NAME_None;
	if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InDataType->TypeObject))
	{
		FriendlyName = Optimus::GetTypeName(UserDefinedStruct, false);
	}
	return InDataType->ShaderValueType->ToString(FriendlyName);
}


static bool IsConnectableInputDataPin(const UOptimusNodePin *InPin)
{
	return InPin->GetDirection() == EOptimusNodePinDirection::Input && !InPin->IsGroupingPin();
}

static bool IsConnectableOutputPin(const UOptimusNodePin *InPin)
{
	return InPin->GetDirection() == EOptimusNodePinDirection::Output && !InPin->IsGroupingPin();
}

static bool IsGroupingInputPin(const UOptimusNodePin *InPin)
{
	return InPin->GetDirection() == EOptimusNodePinDirection::Input && InPin->IsGroupingPin();
}

static bool IsSecondaryGroupInputPin(const UOptimusNodePin *InPin)
{
	return InPin->GetName() != PrimaryGroupPinName && InPin->GetDirection() == EOptimusNodePinDirection::Input && InPin->IsGroupingPin();
}

static bool DoesBindingSupportAtomic(const FOptimusParameterBinding& InBinding)
{
	return InBinding.bSupportAtomicIfCompatibleDataType && FOptimusDataTypeRegistry::Get().DoesTypeSupportAtomic(InBinding.DataType.Resolve()) ;	
}

bool UOptimusNode_CustomComputeKernel::DoesSourceSupportUnifiedDispatch(const UOptimusNodePin& InOtherNodesPin)
{
	if (InOtherNodesPin.GetDirection() == EOptimusNodePinDirection::Output )
	{
		if (const UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(InOtherNodesPin.GetOwningNode()))
		{
			if (const UOptimusComputeDataInterface* DataInterface = DataInterfaceNode->GetDataInterface(GetTransientPackage()))
			{
				if (!DataInterface->CanSupportUnifiedDispatch())
				{
					return false;
				}
			}
		}	
	}

	return true;
}


UOptimusNode_CustomComputeKernel::UOptimusNode_CustomComputeKernel()
{
	EnableDynamicPins();
	UpdatePreamble();
	
	KernelName = DefaultKernelName;
}


FString UOptimusNode_CustomComputeKernel::GetKernelSourceText(bool bInIsUnifiedDispatch) const
{
	using UKernelDI = UOptimusCustomComputeKernelDataInterface;
	
	return Optimus::GetCookedKernelSource(
		GetPathName(),
		ShaderSource.ShaderText,
		GetKernelHlslName(),
		GroupSize,
		UKernelDI::StaticClass()->GetDefaultObject<UKernelDI>()->GetReadNumThreadsPerInvocationFunctionName(),
		UKernelDI::StaticClass()->GetDefaultObject<UKernelDI>()->GetReadThreadIndexOffsetFunctionName(),
		bInIsUnifiedDispatch
		);
}

FOptimusExecutionDomain UOptimusNode_CustomComputeKernel::GetExecutionDomain() const
{
	return ExecutionDomain;
}

const UOptimusNodePin* UOptimusNode_CustomComputeKernel::GetPrimaryGroupPin() const
{
	return GetPrimaryGroupPin_Internal();
}

UComputeDataInterface* UOptimusNode_CustomComputeKernel::MakeKernelDataInterface(UObject* InOuter) const
{
	UOptimusCustomComputeKernelDataInterface* KernelDataInterface = NewObject<UOptimusCustomComputeKernelDataInterface>(InOuter);

	return KernelDataInterface;
}

bool UOptimusNode_CustomComputeKernel::DoesOutputPinSupportAtomic(const UOptimusNodePin* InPin) const
{
	if (ensure(InPin->GetDirection() == EOptimusNodePinDirection::Output))
	{
		if (const FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate(
			[InPin](const FOptimusParameterBinding& InBinding)
				{
					return InBinding.Name == InPin->GetFName(); 
				}))
		{
			return DoesBindingSupportAtomic(*Binding);
		}
	}

	return false;
}

bool UOptimusNode_CustomComputeKernel::DoesOutputPinSupportRead(const UOptimusNodePin* InPin) const
{
	if (ensure(InPin->GetDirection() == EOptimusNodePinDirection::Output))
	{
		if (const FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate(
			[InPin](const FOptimusParameterBinding& InBinding)
				{
					return InBinding.Name == InPin->GetFName(); 
				}))
		{
			return Binding->bSupportRead;
		}
	}

	return false;
}

#if WITH_EDITOR

FString UOptimusNode_CustomComputeKernel::GetNameForShaderTextEditor() const
{
	return KernelName.ToString();
}

FString UOptimusNode_CustomComputeKernel::GetDeclarations() const
{
	return ShaderSource.Declarations;
}

FString UOptimusNode_CustomComputeKernel::GetShaderText() const
{
	return ShaderSource.ShaderText;
}

void UOptimusNode_CustomComputeKernel::SetShaderText(const FString& NewText) 
{
	PreEditChange(nullptr);

	Modify();
	ShaderSource.ShaderText = NewText;

	FPropertyChangedEvent PropertyChangedEvent(FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(UOptimusNode_CustomComputeKernel, ShaderSource)),
		EPropertyChangeType::ValueSet);
	PostEditChangeProperty(PropertyChangedEvent);
}

bool UOptimusNode_CustomComputeKernel::IsShaderTextReadOnly() const
{
	return GetOwningGraph()->IsReadOnly();
}

#endif // WITH_EDITOR

FString UOptimusNode_CustomComputeKernel::GetBindingDeclaration(
	FName BindingName
	) const
{
	auto ParameterBindingPredicate = [BindingName](const FOptimusParameterBinding& InBinding)
	{
		if (InBinding.Name == BindingName)
		{
			return true;	
		}
			
		return false;
	};

	if (const FOptimusParameterBinding* Binding = InputBindingArray.FindByPredicate(ParameterBindingPredicate))
	{
		return GetDeclarationForBinding(*Binding, true);
	}
	if (const FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate(ParameterBindingPredicate))
	{
		return GetDeclarationForBinding(*Binding, false);
	}
	for (const FOptimusSecondaryInputBindingsGroup& InputGroup: SecondaryInputBindingGroups)
	{
		if (const FOptimusParameterBinding* Binding = InputGroup.BindingArray.FindByPredicate(ParameterBindingPredicate))
		{
			return GetDeclarationForBinding(*Binding, true);
		}
	}

	return FString();
}

bool UOptimusNode_CustomComputeKernel::GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const
{
	auto ParameterBindingPredicate = [BindingName](const FOptimusParameterBinding& InBinding)
	{
		if (InBinding.Name == BindingName)
		{
			return true;	
		}
			
		return false;
	};

	if (const FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate(ParameterBindingPredicate))
	{
		if (FOptimusDataTypeRegistry::Get().DoesTypeSupportAtomic(Binding->DataType.Resolve()))
		{
			return true;
		}
	}

	return false;
}

bool UOptimusNode_CustomComputeKernel::GetBindingSupportReadCheckBoxVisibility(FName BindingName) const
{
	auto ParameterBindingPredicate = [BindingName](const FOptimusParameterBinding& InBinding)
	{
		if (InBinding.Name == BindingName)
		{
			return true;	
		}
			
		return false;
	};

	if (const FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate(ParameterBindingPredicate))
	{
		return true;
	}

	return false;
}

EOptimusDataTypeUsageFlags UOptimusNode_CustomComputeKernel::GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const
{
	if (InDataDomain.IsSingleton())
	{
		return EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput;
	}

	return EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute;
}


TArray<IOptimusNodeAdderPinProvider::FAdderPinAction> UOptimusNode_CustomComputeKernel::GetAvailableAdderPinActions(
	const UOptimusNodePin* InSourcePin,
	EOptimusNodePinDirection InNewPinDirection,
	FString* OutReason
	) const
{
	if (InSourcePin->GetDataType().IsValid())
	{
		if (!EnumHasAnyFlags(InSourcePin->GetDataType()->UsageFlags, GetTypeUsageFlags(InSourcePin->GetDataDomain())))
		{
			if (OutReason)
			{
				*OutReason = TEXT("Can't add pin with this type");
			}
			
			return {};
		}

	}
	
	TSet<UOptimusComponentSourceBinding*> SourceComponentBindings = InSourcePin->GetComponentSourceBindings({});
	
	TArray<UOptimusNodePin*> GroupPins;

	for (UOptimusNodePin* GroupPin: GetPins())
	{
		if (IsGroupingInputPin(GroupPin))
		{
			GroupPins.Add(GroupPin);
		}
	}

	if (InNewPinDirection == EOptimusNodePinDirection::Output)
	{
		if (InSourcePin->GetDataDomain().IsSingleton())
		{
			if (OutReason)
			{
				*OutReason = TEXT("Can't add parameter pin as output");
			}

			return {};
		}

		FAdderPinAction	Action;
		// Action is unique, no need to set name/key
		Action.DisplayName = NAME_None;
		Action.NewPinDirection = InNewPinDirection;
		Action.Key = NAME_None;
		Action.ToolTip = FText::GetEmpty();
		Action.bCanAutoLink = false;

		if (InSourcePin->GetDirection() != InNewPinDirection)
		{
			const UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin();
			TSet<UOptimusComponentSourceBinding*> PrimaryGroupComponentBindings = PrimaryGroupPin->GetComponentSourceBindingsRecursively({});
			TSet<UOptimusComponentSourceBinding*> MatchingBindings = PrimaryGroupComponentBindings.Intersect(SourceComponentBindings);
		
			if (MatchingBindings.Num() > 0 ||
				SourceComponentBindings.Num() == 0 || // Source-less pin can be added to kernel
				PrimaryGroupComponentBindings.Num() == 0) // Source-less kernel can be a target for any pin 
			{
				Action.bCanAutoLink = true;
			}
		}

		return { Action };
	}
	else if (InNewPinDirection == EOptimusNodePinDirection::Input)
	{
		if (InSourcePin->GetDirection() == EOptimusNodePinDirection::Input)
		{
			// For convenience lets allow input pins to be created from other input pins 
			// of course linking is not allowed
			TArray<FAdderPinAction> Actions;
			
			for (UOptimusNodePin* GroupPin: GroupPins)
			{
				FAdderPinAction Action;
				Action.NewPinDirection = InNewPinDirection;
				Action.DisplayName = GroupPin->GetFName();
				Action.Key = GroupPin->GetFName();
				
				Action.bCanAutoLink = false;
				Action.ToolTip = LOCTEXT("AdderPinToolTipDirectionMismatch", "Add pin only, can't link input to input");
				
				Actions.Add(Action);
			}

			return Actions;
		}
		else if (InSourcePin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			TArray<FAdderPinAction> Actions;
			
			const bool bSourceSupportUnifiedDispatch = DoesSourceSupportUnifiedDispatch(*InSourcePin);
			
			for (int32 GroupIndex = 0; GroupIndex < GroupPins.Num(); GroupIndex++)
			{
				UOptimusNodePin* GroupPin = GroupPins[GroupIndex];
				FAdderPinAction Action;
				Action.NewPinDirection = InNewPinDirection;
				Action.DisplayName = GroupPin->GetFName();
				Action.Key = GroupPin->GetFName();
					
				TSet<UOptimusComponentSourceBinding*> ComponentBindingsForGroup = GroupPin->GetComponentSourceBindingsRecursively({});
				TSet<UOptimusComponentSourceBinding*> MatchingBindings = ComponentBindingsForGroup.Intersect(SourceComponentBindings);
	
				if (MatchingBindings.Num() > 0 ||
					SourceComponentBindings.Num() == 0 || // Source-less pin can be added to any group
					ComponentBindingsForGroup.Num() == 0) // Source-less group can be a target for any pin 
				{
					Action.bCanAutoLink = true;
				}
				else
				{
					Action.bCanAutoLink = false;
					Action.ToolTip = LOCTEXT("AdderPinToolTipComponentMismatch", "Add pin only, can't auto-link due to group's component binding not matching that of the source pin");
				}

				// Extra restrictions
				// Secondary group does not accept data interface that does not support non unified dispatch
				if (GroupIndex >= 1 && !bSourceSupportUnifiedDispatch)
				{
					Action.bCanAutoLink = false;
					Action.ToolTip = LOCTEXT("AdderPinToolTipCannotLinkNonUnified", "Add pin only, can't link due to data interface not supporting unified dispatch, consider insert kernel inbetween");
				}
				
				Actions.Add(Action);
			}

			bool bHasAutoLinkAction = false;
			for (const FAdderPinAction& Action : Actions)
			{
				if (Action.bCanAutoLink)
				{
					bHasAutoLinkAction = true;
					break;
				}
			}
			
			if (!bHasAutoLinkAction && bSourceSupportUnifiedDispatch)
			{
				FAdderPinAction AddNewGroupAction;
				AddNewGroupAction.DisplayName = TEXT("Add a New Group");
				AddNewGroupAction.NewPinDirection = InNewPinDirection;
				AddNewGroupAction.Key = AdderPinActionKeyAddNewGroup;
				AddNewGroupAction.bCanAutoLink = true;
				
				Actions.Add(AddNewGroupAction);
			}

			for (FAdderPinAction& Action : Actions)
			{
				if (Action.bCanAutoLink)
				{
					Action.ToolTip = LOCTEXT("AdderPinToolTipAutoLink", "Group accepts pin connection");
					break;
				}
			}
			
			return Actions;	
		}
	}

	return {};
}

TArray<UOptimusNodePin*> UOptimusNode_CustomComputeKernel::TryAddPinFromPin(
	const FAdderPinAction& InSelectedAction,
	UOptimusNodePin* InSourcePin,
	FName InNameToUse
	)
{
	TArray<UOptimusNodePin*> AddedPins;
	
	TArray<FOptimusParameterBinding>* BindingArray = nullptr;
	UOptimusNodePin* ParentPin = nullptr;
	UOptimusNodePin* BeforePin = nullptr;

	if (InSelectedAction.NewPinDirection == EOptimusNodePinDirection::Output)
	{
		BindingArray = &OutputBindingArray.InnerArray;
	}
	else if (InSelectedAction.NewPinDirection == EOptimusNodePinDirection::Input)
	{
		if (InSelectedAction.Key == AdderPinActionKeyAddNewGroup)
		{
			//Creating a new group
			
			TSet<UOptimusComponentSourceBinding*> ComponentBindings = InSourcePin->GetComponentSourceBindings({});
			
			check(ComponentBindings.Num() != 0);

			UOptimusComponentSourceBinding* ComponentBinding = *ComponentBindings.begin();

			FName GroupName = GetSanitizedNewPinName(nullptr, ComponentBinding->GetFName());

			SecondaryInputBindingGroups.AddDefaulted_GetRef().GroupName = GroupName;
			BindingArray = &SecondaryInputBindingGroups.Last().BindingArray.InnerArray;
			
			AddedPins.Add(AddGroupingPinDirect(GroupName, EOptimusNodePinDirection::Input));
			ParentPin = AddedPins.Last();	
		}
		else
		{
			// Adding to existing group
			
			TArray<UOptimusNodePin*> GroupPins;
			for (UOptimusNodePin* GroupPin: GetPins())
			{
				if (IsGroupingInputPin(GroupPin))
				{
					GroupPins.Add(GroupPin);
				}
			}	
			
			for (int32 GroupIndex = 0; GroupIndex < GroupPins.Num(); GroupIndex++)
			{
				UOptimusNodePin* GroupPin = GroupPins[GroupIndex];
				
				if (GroupPin->GetFName() == InSelectedAction.Key)
				{
					ParentPin = GroupPin;

					if (GroupIndex == 0)
					{
						BindingArray = &InputBindingArray.InnerArray;
					}
					else
					{
						// Secondary group
						FOptimusSecondaryInputBindingsGroup* Group = SecondaryInputBindingGroups.FindByPredicate(
						[ParentPin](const FOptimusSecondaryInputBindingsGroup& InGroup)
						{
							return InGroup.GroupName == ParentPin->GetFName();
						});

						if (!ensure(Group))
						{
							return {};
						}
				
						BindingArray = &Group->BindingArray.InnerArray;
					}
					break;
				}
			}
		}
	}	

	if (!ensure(BindingArray))
	{
		return {};
	}

	if (ParentPin)
	{
		ParentPin->SetIsExpanded(true);
	}
	
	FOptimusParameterBinding Binding;
	Binding.Name = GetSanitizedNewPinName(ParentPin, InNameToUse);
	Binding.DataType = {InSourcePin->GetDataType()};
	Binding.DataDomain = InSourcePin->GetDataDomain();
	
	BindingArray->Add(Binding);
	UpdatePreamble();
	
	AddedPins.Add(AddPinDirect(Binding.Name, InSelectedAction.NewPinDirection, Binding.DataDomain, Binding.DataType, BeforePin, ParentPin));

	return AddedPins;
}


bool UOptimusNode_CustomComputeKernel::RemoveAddedPins(
	TConstArrayView<UOptimusNodePin*> InAddedPinsToRemove
)
{
	if (InAddedPinsToRemove.Num() == 0)
	{
		return true;
	}

	// Currently only two pins can be added in 1 go at the most, group pin + some data pin
	if (!ensure(InAddedPinsToRemove.Num() <= 2))
	{
		return false;
	}
	
	if (InAddedPinsToRemove.Num() == 1)
	{
		UOptimusNodePin* PinToRemove = InAddedPinsToRemove[0];
		
		TArray<FOptimusParameterBinding>* BindingArray = 
			PinToRemove->GetDirection() == EOptimusNodePinDirection::Input ?
				&InputBindingArray.InnerArray : &OutputBindingArray.InnerArray;
	
		if (UOptimusNodePin* GroupPin = PinToRemove->GetParentPin())
		{
			if (GroupPin->GetFName() != PrimaryGroupPinName)
			{
				FOptimusSecondaryInputBindingsGroup* Group = SecondaryInputBindingGroups.FindByPredicate(
				   [GroupPin](const FOptimusSecondaryInputBindingsGroup& InGroup)
				   {
					   return InGroup.GroupName == GroupPin->GetFName();
				   });
				
				if (!ensure(Group))
				{
					return false;
				}

				BindingArray = &Group->BindingArray.InnerArray;
			}
		}
	

		BindingArray->RemoveAll(
			[PinToRemove](const FOptimusParameterBinding& Binding)
			{
				return Binding.Name == PinToRemove->GetFName(); 
			});
		UpdatePreamble();
		return RemovePinDirect(PinToRemove);	
	}

	check(InAddedPinsToRemove.Num() == 2);

	UOptimusNodePin* GroupPin = InAddedPinsToRemove[0];
	check(GroupPin->GetDirection() == EOptimusNodePinDirection::Input);
	check(GroupPin->GetSubPins().Num() == 1);
	check(GroupPin->GetSubPins()[0] == InAddedPinsToRemove[1]);
	
	SecondaryInputBindingGroups.RemoveAll(
		[GroupPin](const FOptimusSecondaryInputBindingsGroup& InGroup)
		{
			return InGroup.GroupName == GroupPin->GetFName();
		});

	UpdatePreamble();

	return RemovePinDirect(GroupPin);
}


FName UOptimusNode_CustomComputeKernel::GetSanitizedNewPinName(
	UOptimusNodePin* InParentPin,
	FName InPinName
	)
{
	FName NewName = Optimus::GetSanitizedNameForHlsl(InPinName);
	
	// "NumThreads" is reserved for the kernel data interface
	if (NewName == GetNumThreadsReservedName())
	{
		NewName.SetNumber(1);
	}

	UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
	
	if (InParentPin && InParentPin != PrimaryGroupPin)
	{
		NewName = GetAvailablePinNameStable(InParentPin, NewName);
	}
	else
	{
		// Primary Input and Output need to name check against node scope as well as primary group pin scope
		NewName = GetAvailablePinNameStable(this, NewName);
		NewName = GetAvailablePinNameStable(PrimaryGroupPin, NewName);
	}


	return NewName;
}

TArray<FName> UOptimusNode_CustomComputeKernel::GetExecutionDomains() const
{
	// Find all component sources for the primary pins. If we end up with any other number
	// than one, then something's gone wrong and we can't determine the execution domains.
	TSet<UOptimusComponentSourceBinding*> PrimaryBindings = GetPrimaryGroupPin()->GetComponentSourceBindingsRecursively({});

	if (PrimaryBindings.Num() == 1)
	{
		return PrimaryBindings.Array()[0]->GetComponentSource()->GetExecutionDomains();
	}
	else
	{
		return UOptimusComponentSource::GetAllExecutionDomains().Array();
	}
}

void UOptimusNode_CustomComputeKernel::OnDataTypeChanged(FName InTypeName)
{
	Super::OnDataTypeChanged(InTypeName);
	
	UpdatePreamble();
}

void UOptimusNode_CustomComputeKernel::PostLoadNodeSpecificData()
{
	Super::PostLoadNodeSpecificData();

	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::SwitchToParameterBindingArrayStruct)
	{
		Modify();
		InputBindingArray.InnerArray = InputBindings_DEPRECATED;
		OutputBindingArray.InnerArray = OutputBindings_DEPRECATED;
	}
	
	if (!Parameters_DEPRECATED.IsEmpty())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS		
		TArray<FOptimusParameterBinding> ParameterInputBindings;

		for (const FOptimus_ShaderBinding& OldBinding: Parameters_DEPRECATED)
		{
			FOptimusParameterBinding NewBinding;
			NewBinding.Name = OldBinding.Name;
			NewBinding.DataType = OldBinding.DataType;
			NewBinding.DataDomain = FOptimusDataDomain();
			
			ParameterInputBindings.Add(NewBinding);
		}
		
		InputBindingArray.InnerArray.Insert(ParameterInputBindings, 0);
		
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	
	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::KernelDataInterface)
	{
		PostLoadExtractExecutionDomain();
		PostLoadAddMissingPrimaryGroupPin();
	}

	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::KernelParameterBindingToggleAtomic)
	{
		PostLoadExtractAtomicModeFromConnectedResource();
	}

	SetDisplayName(FText::FromName(KernelName));	
}

FString UOptimusNode_CustomComputeKernel::GetKernelHlslName() const
{
	// We need to append the node name here because of a weird issue on Mac where
	// if the last word of the name of a shader parameter matches the kernel name, the value
	// for the parameter is always zero for unknown reasons. (e.g. AnimAttribute named "XXXBend" & Kernel also named "Bend")
	// Adding this bit of "random" string after the user provided kernel name should
	// greatly reduce the chance of things like that happening
	return KernelName.ToString() + TEXT("_") + GetName();
}


void UOptimusNode_CustomComputeKernel::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FOptimusObjectVersion::GUID);
}


#if WITH_EDITOR
void UOptimusNode_CustomComputeKernel::PostEditChangeProperty(
	FPropertyChangedEvent& PropertyChangedEvent
	)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
	{
		if (CastField<FArrayProperty>(PropertyChangedEvent.Property) &&
			PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.GetPropertyName().ToString()) == INDEX_NONE)
		{
			PropertyArrayPasted(PropertyChangedEvent);
		}
		else
		{
			PropertyValueChanged(PropertyChangedEvent);
		}
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayAdd)
	{
		PropertyArrayItemAdded(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayRemove)
	{
		PropertyArrayItemRemoved(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayClear)
	{
		PropertyArrayCleared(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayMove)
	{
		PropertyArrayItemMoved(PropertyChangedEvent);
	}
}

void UOptimusNode_CustomComputeKernel::PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	auto RemoveAllPinsByPredicate = [this](
	TArrayView<UOptimusNodePin* const> InPins,
	TFunction<bool(const UOptimusNodePin*)> InPredicate)
	{
		// Make a copy of the pins, since we're removing from the array represented by the view.
		for (UOptimusNodePin* Pin: TArray<UOptimusNodePin*>(InPins))
		{
			if (InPredicate(Pin))
			{
				RemovePin(Pin);
			}
		}
	};
	
	FOptimusActionScope(*GetActionStack(), TEXT("Paste Bindings"));
	
	if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
	{
		// Two cases here
		// 1. Pasting into the group array - Recreate the entire group array, including the sub pins
		// 2. Pasting into the binding array of one of the groups
		//		- Unfortunately we cannot know which group is the subject, so recreate the entire group array as well

		RemoveAllPinsByPredicate(GetPins(), IsSecondaryGroupInputPin);

		for (const FOptimusSecondaryInputBindingsGroup& InputGroup: SecondaryInputBindingGroups)
		{
			UOptimusNodePin* GroupPin = AddGroupingPin(InputGroup.GroupName, EOptimusNodePinDirection::Input);

			for (const FOptimusParameterBinding& Binding: InputGroup.BindingArray)
			{
				AddPin(Binding.Name, EOptimusNodePinDirection::Input, Binding.DataDomain, Binding.DataType, nullptr, GroupPin);
			}

			if (InputGroup.BindingArray.Num() > 0)
			{
				// Make sure the group pin is expanded so that the change is visible in the graph.
				GroupPin->SetIsExpanded(true);
			}
		}

		UpdatePreamble();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
	{
		UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
		RemoveAllPinsByPredicate(PrimaryGroupPin->GetSubPins() , IsConnectableInputDataPin);

		for (const FOptimusParameterBinding& Binding: InputBindingArray)
		{
			AddPin(Binding.Name, EOptimusNodePinDirection::Input, Binding.DataDomain, Binding.DataType, nullptr, PrimaryGroupPin);
		}

		if (InputBindingArray.Num() > 0)
		{
			PrimaryGroupPin->SetIsExpanded(true);
		}
		
		UpdatePreamble();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
	{
		RemoveAllPinsByPredicate(GetPins(), IsConnectableOutputPin);

		for (const FOptimusParameterBinding& Binding: OutputBindingArray)
		{
			AddPin(Binding.Name, EOptimusNodePinDirection::Output, Binding.DataDomain, Binding.DataType);
		}

		UpdatePreamble();
	}
}

void UOptimusNode_CustomComputeKernel::PropertyValueChanged(
	const FPropertyChangedEvent& InPropertyChangedEvent
	)
{
	auto UpdatePinsFromBindings = [this](
		UOptimusNodePin* InParentPin,
		TArrayView<UOptimusNodePin* const> InPins,
		TFunction<bool(const UOptimusNodePin*)> InPinFilter,
		FOptimusParameterBindingArray &InBindings,
		TFunction<void(UOptimusNodePin*, FOptimusParameterBinding&, UOptimusNodePin*)> InApplyFunc
		)
	{
		int32 Index = 0;
		for (UOptimusNodePin* Pin: InPins)
		{
			if (InPinFilter(Pin))
			{
				FOptimusParameterBinding& Binding = InBindings[Index];

				InApplyFunc(InParentPin, Binding, Pin);

				Index++;
			}
		}
	};

	auto UpdateAllBindings = [&](
		TFunction<void(UOptimusNodePin*, FOptimusParameterBinding&, UOptimusNodePin*)> InApplyFunc
		) -> bool
	{
		if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
		{
			UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();	
			UpdatePinsFromBindings(PrimaryGroupPin,  PrimaryGroupPin->GetSubPins(), IsConnectableInputDataPin, InputBindingArray, InApplyFunc);
			return true;
		}
		if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
		{
			UpdatePinsFromBindings(nullptr, GetPins(), IsConnectableOutputPin, OutputBindingArray, InApplyFunc);
			return true;
		}
		if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
		{
			int32 GroupIndex = 0;
			for (UOptimusNodePin* GroupPin: GetPins())
			{
				if (IsSecondaryGroupInputPin(GroupPin))
				{
					FOptimusSecondaryInputBindingsGroup& BindingsGroup = SecondaryInputBindingGroups[GroupIndex];
					UpdatePinsFromBindings(GroupPin, GroupPin->GetSubPins(), IsConnectableInputDataPin, BindingsGroup.BindingArray, InApplyFunc);

					GroupIndex++;
				}
			}
			return true;
		}

		return false;
	};
	
	auto UpdateName = [this](UOptimusNodePin* InParentPin, FOptimusParameterBinding& InBinding, UOptimusNodePin* InPin)
	{
		if (InPin->GetFName() != InBinding.Name)
		{
			InBinding.Name = GetSanitizedNewPinName(InParentPin, InBinding.Name);
			SetPinName(InPin, InBinding.Name);
		}
	};

	auto UpdatePinType = [this](UOptimusNodePin* , FOptimusParameterBinding& InBinding, UOptimusNodePin* InPin)
	{
		const FOptimusDataTypeHandle DataType = InBinding.DataType.Resolve();
		if (InPin->GetDataType() != DataType)
		{
			SetPinDataType(InPin, DataType);
		}
	};

	auto UpdatePinDataDomain = [this](UOptimusNodePin* , FOptimusParameterBinding& InBinding, UOptimusNodePin* InPin)
	{
		if (InPin->GetDataDomain() != InBinding.DataDomain)
		{
			SetPinDataDomain(InPin, InBinding.DataDomain);
		}
	};
	
	auto UpdatePin = [&](UOptimusNodePin* InParentPin, FOptimusParameterBinding& InBinding, UOptimusNodePin* InPin)
	{
		UpdateName(InParentPin, InBinding, InPin);
		UpdatePinType(InParentPin, InBinding, InPin);
		UpdatePinDataDomain(InParentPin, InBinding, InPin);
	};
	
	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(UOptimusNode_CustomComputeKernel, KernelName) && 
		InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusValidatedName, Name))
	{
		// Making sure the display name(NonTransactional) is updated as KernelName(Transactional) changes
		GetActionStack()->RunAction<FOptimusNodeAction_RenameNode>(this, FText::FromName(KernelName));
		
		UpdatePreamble();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(UOptimusNode_CustomComputeKernel, SecondaryInputBindingGroups) &&
			 InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusValidatedName, Name))
	{
		// The group name or secondary pin names changed.
		int32 Index = 0;
		bool bUpdatePreamble = false;
		for (UOptimusNodePin* GroupPin: GetPins())
		{
			if (IsSecondaryGroupInputPin(GroupPin))
			{
				FOptimusSecondaryInputBindingsGroup& SecondaryGroup = SecondaryInputBindingGroups[Index++]; 
				if (GroupPin->GetFName() != SecondaryGroup.GroupName)
				{
					SecondaryGroup.GroupName = GetSanitizedNewPinName(nullptr, SecondaryGroup.GroupName);
					SetPinName(GroupPin, SecondaryGroup.GroupName);
					bUpdatePreamble = true;
				}
			}
		}

		if (bUpdatePreamble || UpdateAllBindings(UpdateName))
		{
			UpdatePreamble();
		}
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusValidatedName, Name))
	{
		if (UpdateAllBindings(UpdateName))
		{
			UpdatePreamble();
			return;
		}
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusDataTypeRef, TypeName))
	{
		if (UpdateAllBindings(UpdatePinType))
		{
			UpdatePreamble();
			return;
		}
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBinding, DataDomain))
	{
		if (UpdateAllBindings(UpdatePinDataDomain))
		{
			UpdatePreamble();
			return;
		}
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBinding, bSupportAtomicIfCompatibleDataType))
	{
		UpdatePreamble();
		return;
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBinding, bSupportRead))
	{
		UpdatePreamble();
		return;
	}
	else if ( InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray))
	{
		// Pasting into a specific binding
		if (UpdateAllBindings(UpdatePin))
		{
			UpdatePreamble();
			return;
		}	
	}
}


void UOptimusNode_CustomComputeKernel::PropertyArrayItemAdded(
	const FPropertyChangedEvent& InPropertyChangedEvent
	)
{
	auto GetArrayIndex = [Event=InPropertyChangedEvent](
		const FString& InArrayName,
		const auto& InArray
		) -> int32
	{
		int32 ArrayIndex = Event.GetArrayIndex(InArrayName);
		if (ArrayIndex == INDEX_NONE)
		{
			ArrayIndex = InArray.Num() - 1;
		}
		return ArrayIndex;
	};
	auto AddPinForBinding = [this](
		FOptimusParameterBinding& InBinding,
		FName InName,
		EOptimusNodePinDirection InDirection,
		UOptimusNodePin* InBeforePin = nullptr,
		UOptimusNodePin* InGroupPin = nullptr
	) -> void
	{
		InBinding.Name = GetSanitizedNewPinName(InGroupPin, InName);
		InBinding.DataType = FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass());
		if (InDirection == EOptimusNodePinDirection::Input)
		{
			// Default to parameters for input bindings.
			InBinding.DataDomain = FOptimusDataDomain();
		}
		else
		{
			// Pick a suitable fallback for output pins.
			if (ExecutionDomain.IsDefined())
			{
				InBinding.DataDomain = FOptimusDataDomain(ExecutionDomain);
			}
			else
			{
				FName DomainName = NAME_None;
				if (TArray<FName> ExecutionDomains = GetExecutionDomains(); !ExecutionDomains.IsEmpty())
				{
					DomainName = ExecutionDomains[0];
				}
				else
				{
					// FIXME: There should be a generic mechanism to get the most suitable default. 
					DomainName = UOptimusSkeletalMeshComponentSource::Domains::Vertex;
				}
				
				InBinding.DataDomain = FOptimusDataDomain(TArray<FName>({DomainName}));
			}
		}

		AddPin(InBinding.Name, InDirection, InBinding.DataDomain, InBinding.DataType, InBeforePin, InGroupPin);

		UpdatePreamble();
	};


	if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
	{
		if (InPropertyChangedEvent.GetPropertyName() == ExtraInputBindingGroupsName)
		{
			const int32 GroupIndex = InPropertyChangedEvent.GetArrayIndex(ExtraInputBindingGroupsName.ToString());
			if (!ensure(GroupIndex != INDEX_NONE))
			{
				return;
			}

			// If adding to the list of groups before the last, then we need to specify the before-pin otherwise
			// we just add the pin to the end.
			UOptimusNodePin* BeforePin = nullptr;
			if (GroupIndex < (SecondaryInputBindingGroups.Num() - 1))
			{
				// The top level hierarchy is always PrimaryGroupPin -> OutputPins -> SecondaryGroupPins
				BeforePin = GetPins()[1 + OutputBindingArray.Num() + GroupIndex];
			}
			
			const FName GroupName = GetSanitizedNewPinName(nullptr, DefaultGroupName);

			AddGroupingPin(GroupName, EOptimusNodePinDirection::Input, BeforePin);

			SecondaryInputBindingGroups[GroupIndex].GroupName = GroupName;

			UpdatePreamble();
		}
		else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FOptimusParameterBindingArray, InnerArray))
		{
			// Unfortunately, we don't get told which group index we're under. So find the group where the binding
			// count is one greater than the pin count.
			if (!ensure(!SecondaryInputBindingGroups.IsEmpty()))
			{
				return;
			}
			
			int32 GroupIndex = 0;
			UOptimusNodePin* GroupPin = nullptr;
			for (UOptimusNodePin *Pin: GetPins())
			{
				if (IsSecondaryGroupInputPin(Pin))
				{
					if (SecondaryInputBindingGroups[GroupIndex].BindingArray.Num() == (Pin->GetSubPins().Num() + 1))
					{
						GroupPin = Pin;
						break;
					}
					GroupIndex++;
				}
			}

			if (!ensure(GroupPin))
			{
				return;
			}
			
			const int32 BindingIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));

			FOptimusSecondaryInputBindingsGroup& BindingGroup = SecondaryInputBindingGroups[GroupIndex];
			
			UOptimusNodePin* BeforePin = nullptr;
			if (BindingIndex < (BindingGroup.BindingArray.Num() - 1))
			{
				BeforePin = GroupPin->GetSubPins()[BindingIndex];
			}

			AddPinForBinding(BindingGroup.BindingArray[BindingIndex], "Input", EOptimusNodePinDirection::Input, BeforePin, GroupPin);
			
			// Make sure the group pin is expanded so that the change is visible in the graph.
			GroupPin->SetIsExpanded(true);
		}
	}
	if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
	{
		const int32 BindingIndex = GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray), InputBindingArray);

		UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
		UOptimusNodePin* BeforePin = PrimaryGroupPin->GetSubPins().IsValidIndex(BindingIndex) ? PrimaryGroupPin->GetSubPins()[BindingIndex] : nullptr;

		AddPinForBinding(InputBindingArray[BindingIndex], "Input", EOptimusNodePinDirection::Input, BeforePin, PrimaryGroupPin);

		if (InputBindingArray.Num() > 0)
		{
			PrimaryGroupPin->SetIsExpanded(true);
		}
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
	{
		const int32 BindingIndex = GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray), OutputBindingArray);

		UOptimusNodePin* BeforePin = nullptr;

		int32 PinCount = BindingIndex;
		for (UOptimusNodePin *Pin: GetPins())
		{
			// Inserting after the last output pin
			if (IsSecondaryGroupInputPin(Pin))
			{
				BeforePin = Pin;
				break;
			}

			// Inserting before the last output pin
			if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
			{
				if (PinCount-- == 0)
				{
					BeforePin = Pin;
					break;
				}
			}
		}

		AddPinForBinding(OutputBindingArray[BindingIndex], "Output", EOptimusNodePinDirection::Output, BeforePin);
	}
}


void UOptimusNode_CustomComputeKernel::PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	auto RemoveTopLevelPinByIndex = [this, Event=InPropertyChangedEvent](EOptimusNodePinDirection InPinDirection)
	{
		int32 PinIndex = Event.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));
		if (!ensure(PinIndex != INDEX_NONE))
		{
			return;
		}
		
		for (UOptimusNodePin *Pin: GetPins())
		{
			if (Pin->GetDirection() == InPinDirection && !Pin->IsGroupingPin())
			{
				if (PinIndex-- == 0)
				{
					RemovePin(Pin);
					UpdatePreamble();
					return;
				}
			}
		}
	};
	
	if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
	{
		if (InPropertyChangedEvent.GetPropertyName() == ExtraInputBindingGroupsName)
		{
			int32 GroupIndex = InPropertyChangedEvent.GetArrayIndex(ExtraInputBindingGroupsName.ToString());
			if (!ensure(GroupIndex != INDEX_NONE))
			{
				return;
			}

			for (UOptimusNodePin *Pin: GetPins())
			{
				if (IsSecondaryGroupInputPin(Pin))
				{
					if (GroupIndex-- == 0)
					{
						RemovePin(Pin);
						UpdatePreamble();
						return;
					}
				}
			}
		}
		else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FOptimusParameterBindingArray, InnerArray))
		{
			// Unfortunately, we don't get told which group index we're under. So find the group where the binding
			// count is one less than the pin count.
			if (!ensure(!SecondaryInputBindingGroups.IsEmpty()))
			{
				return;
			}
			
			int32 GroupIndex = 0;
			UOptimusNodePin* GroupPin = nullptr;
			for (UOptimusNodePin *Pin: GetPins())
			{
				if (IsSecondaryGroupInputPin(Pin))
				{
					if (SecondaryInputBindingGroups[GroupIndex].BindingArray.Num() == (Pin->GetSubPins().Num() - 1))
					{
						GroupPin = Pin;
						break;
					}
					GroupIndex++;
				}
			}

			if (!ensure(GroupPin))
			{
				return;
			}

			if (SecondaryInputBindingGroups[GroupIndex].BindingArray.IsEmpty())
			{
				// If the group goes empty, collapse the pin for consistent look.
				GroupPin->SetIsExpanded(false);
			}
			
			const int32 BindingIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));
			if (ensure(GroupPin->GetSubPins().IsValidIndex(BindingIndex)))
			{
				RemovePin(GroupPin->GetSubPins()[BindingIndex]);
				UpdatePreamble();
			}
		}
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
	{
		UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
		
		int32 BindingIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));
		
		if (ensure(PrimaryGroupPin->GetSubPins().IsValidIndex(BindingIndex)))
		{
			RemovePin(PrimaryGroupPin->GetSubPins()[BindingIndex]);
			UpdatePreamble();
		}

		if (InputBindingArray.IsEmpty())
		{
			PrimaryGroupPin->SetIsExpanded(false);
		}
		
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
	{
		int32 PinIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));
		if (!ensure(PinIndex != INDEX_NONE))
		{
			return;
		}
		
		for (UOptimusNodePin *Pin: GetPins())
		{
			if (IsConnectableOutputPin(Pin))
			{
				if (PinIndex-- == 0)
				{
					RemovePin(Pin);
					UpdatePreamble();
					break;
				}
			}
		}	
	}
}

void UOptimusNode_CustomComputeKernel::PropertyArrayCleared(
	const FPropertyChangedEvent& InPropertyChangedEvent
	)
{
	auto RemoveAllPinsByPredicate = [this](
		TArrayView<UOptimusNodePin* const> InPins,
		TFunction<bool(const UOptimusNodePin*)> InPredicate)
	{
		// Make a copy of the pins, since we're removing from the array represented by the view.
		for (UOptimusNodePin* Pin: TArray<UOptimusNodePin*>(InPins))
		{
			if (InPredicate(Pin))
			{
				RemovePin(Pin);
			}
		}
		UpdatePreamble();
	};
	
	if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
	{
		if (InPropertyChangedEvent.GetPropertyName() == ExtraInputBindingGroupsName)
		{
			RemoveAllPinsByPredicate(GetPins(), IsSecondaryGroupInputPin);
			UpdatePreamble();
		}
		else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FOptimusParameterBindingArray, InnerArray))
		{
			// Unfortunately, we don't get told which group index we're under. So find the group where the binding
			// count is one less than the pin count.
			if (!ensure(!SecondaryInputBindingGroups.IsEmpty()))
			{
				return;
			}
			
			int32 GroupIndex = 0;
			UOptimusNodePin* GroupPin = nullptr;
			for (UOptimusNodePin *Pin: GetPins())
			{
				if (IsSecondaryGroupInputPin(Pin))
				{
					// If the binding array is empty but the pins are still there, then that's our group.
					if (SecondaryInputBindingGroups[GroupIndex].BindingArray.IsEmpty() && !Pin->GetSubPins().IsEmpty())
					{
						GroupPin = Pin;
						break;
					}
					GroupIndex++;
				}
			}

			if (!ensure(GroupPin))
			{
				return;
			}

			// Since  the group is empty, collapse the pin for consistent look.
			GroupPin->SetIsExpanded(false);
			
			RemoveAllPinsByPredicate(GroupPin->GetSubPins(), IsConnectableInputDataPin);
			UpdatePreamble();
		}
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
	{
		UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
		RemoveAllPinsByPredicate(PrimaryGroupPin->GetSubPins(), IsConnectableInputDataPin);

		PrimaryGroupPin->SetIsExpanded(false);
		
		UpdatePreamble();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
	{
		RemoveAllPinsByPredicate(GetPins(), IsConnectableOutputPin);
		UpdatePreamble();
	}
}

void UOptimusNode_CustomComputeKernel::PropertyArrayItemMoved(
	const FPropertyChangedEvent& InPropertyChangedEvent
	)
{
	auto FindMovedBindingAndMoveMatchingPin = [this](
		TArrayView<UOptimusNodePin* const> InPins,
		const TArray<FName>& BindingNameArray,
		TFunction<bool(const UOptimusNodePin *)> InPinPredicate
		)
	{
		TArray<FName> PinNames;
	
		for (const UOptimusNodePin* Pin: InPins)
		{
			if (InPinPredicate(Pin))
			{
				PinNames.Add(Pin->GetFName());
			}
		}


		
		FName PinName = NAME_None;
		FName NextPinName = NAME_None;

		if (Optimus::FindMovedItemInNameArray(PinNames, BindingNameArray, PinName, NextPinName))
		{
			UOptimusNodePin* MovedPin = *InPins.FindByPredicate([PinName](const UOptimusNodePin* InPin)
			{
				return InPin->GetFName() == PinName;
			});

			UOptimusNodePin* NextPin = nullptr;
			if (!NextPinName.IsNone())
			{
				NextPin = *InPins.FindByPredicate([NextPinName](const UOptimusNodePin* InPin)
				{
					return InPin->GetFName() == NextPinName;
				});	
			}

			MovePin(MovedPin, NextPin);
		}
	};

	auto MakeBindingNameArray = [](const FOptimusParameterBindingArray& InBindings) -> TArray<FName>
	{
		TArray<FName> Result;
		for (const FOptimusParameterBinding& Binding: InBindings)
		{
			Result.Add(Binding.Name);
		}
		return Result;
	};
	
	if (InPropertyChangedEvent.GetMemberPropertyName() == ExtraInputBindingGroupsName)
	{
		if (InPropertyChangedEvent.GetPropertyName() == ExtraInputBindingGroupsName)
		{
			TArray<FName> GroupNames;
			for (const FOptimusSecondaryInputBindingsGroup& BindingsGroup: SecondaryInputBindingGroups)
			{
				GroupNames.Add(BindingsGroup.GroupName);
			}
			
			FindMovedBindingAndMoveMatchingPin(GetPins(), GroupNames, IsSecondaryGroupInputPin);
			UpdatePreamble();
			
		}
		else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FOptimusParameterBindingArray, InnerArray))
		{
			// Unfortunately, we don't get told which group index we're under. So find the group where the binding
			// count is one less than the pin count.
			if (!ensure(!SecondaryInputBindingGroups.IsEmpty()))
			{
				return;
			}

			auto PinsAndBindingsDiffer = [](const FOptimusParameterBindingArray& InBindings, TArrayView<UOptimusNodePin* const> InPins)
			{
				if (ensure(InBindings.Num() == InPins.Num()))
				{
					for (int32 Index = 0; Index < InBindings.Num(); Index++)
					{
						if (InBindings[Index].Name != InPins[Index]->GetFName())
						{
							return true;
						}
					}
				}
				return false;
			};
			
			int32 GroupIndex = 0;
			UOptimusNodePin* GroupPin = nullptr;
			for (UOptimusNodePin *Pin: GetPins())
			{
				if (IsSecondaryGroupInputPin(Pin))
				{
					// If the binding array is empty but the pins are still there, then that's our group.
					if (PinsAndBindingsDiffer(SecondaryInputBindingGroups[GroupIndex].BindingArray, Pin->GetSubPins()))
					{
						GroupPin = Pin;
						break;
					}
					GroupIndex++;
				}
			}

			if (!ensure(GroupPin))
			{
				return;
			}

			FindMovedBindingAndMoveMatchingPin(
				GroupPin->GetSubPins(), MakeBindingNameArray(SecondaryInputBindingGroups[GroupIndex].BindingArray),
				IsConnectableInputDataPin);
			UpdatePreamble();
		}
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsName)
	{
		UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
		FindMovedBindingAndMoveMatchingPin(PrimaryGroupPin->GetSubPins(), MakeBindingNameArray(InputBindingArray), IsConnectableInputDataPin);
		UpdatePreamble();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsName)
	{
		FindMovedBindingAndMoveMatchingPin(GetPins(), MakeBindingNameArray(OutputBindingArray), IsConnectableOutputPin);
		UpdatePreamble();
	}
}

#endif


void UOptimusNode_CustomComputeKernel::ConstructNode()
{
	SetDisplayName(FText::FromName(KernelName));
	
	// After a duplicate, the kernel node has no pins, so we need to reconstruct them from
	// the bindings. We can assume that all naming clashes have already been dealt with.

	UOptimusNodePin* PrimaryGroupPin = AddGroupingPinDirect(PrimaryGroupPinName, EOptimusNodePinDirection::Input);
	
	for (const FOptimusParameterBinding& Binding: InputBindingArray)
	{
		AddPinDirect(Binding.Name, EOptimusNodePinDirection::Input, Binding.DataDomain, Binding.DataType, nullptr, PrimaryGroupPin);
	}
	for (const FOptimusParameterBinding& Binding: OutputBindingArray)
	{
		AddPinDirect(Binding.Name, EOptimusNodePinDirection::Output, Binding.DataDomain, Binding.DataType);
	}

	// FIXME: Group pins.
	for (const FOptimusSecondaryInputBindingsGroup& InputGroup: SecondaryInputBindingGroups)
	{
		UOptimusNodePin* GroupPin = AddGroupingPinDirect(InputGroup.GroupName, EOptimusNodePinDirection::Input);
		
		for (const FOptimusParameterBinding& Binding: InputGroup.BindingArray)
		{
			AddPinDirect(Binding.Name, EOptimusNodePinDirection::Input, Binding.DataDomain, Binding.DataType, nullptr, GroupPin);
		}
	}
}


bool UOptimusNode_CustomComputeKernel::ValidateConnection(
	const UOptimusNodePin& InThisNodesPin,
	const UOptimusNodePin& InOtherNodesPin,
	FString* OutReason
	) const
{
	if (InThisNodesPin.GetDirection() == EOptimusNodePinDirection::Input)
	{
		// Secondary group connections need additional checks
		if (InThisNodesPin.GetRootPin()->GetFName() != PrimaryGroupPinName)
		{
			if (!DoesSourceSupportUnifiedDispatch(InOtherNodesPin))
			{
				if (OutReason)
				{
					*OutReason = TEXT("Can't link due to data interface not supporting unified dispatch, consider insert kernel inbetween");
				}
				return false;
			}
		}
	}
	
	return true;
}

TOptional<FText> UOptimusNode_CustomComputeKernel::ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
{
	if (TOptional<FText> Result = Super::ValidateForCompile(InContext); Result.IsSet())
	{
		return Result;
	}
	return {};
}


void UOptimusNode_CustomComputeKernel::UpdatePreamble()
{
	TSet<FString> StructsSeen;
	TArray<FString> Structs;

	auto CollectStructs = [&StructsSeen, &Structs](const auto& BindingArray)
	{
		for (const FOptimusParameterBinding &Binding: BindingArray)
		{
			if (Binding.DataType.IsValid() && Binding.DataType->ShaderValueType.IsValid())
			{
				const FShaderValueType& ValueType = *Binding.DataType->ShaderValueType;
				if (ValueType.Type == EShaderFundamentalType::Struct)
				{
					TArray<FShaderValueTypeHandle> StructTypes = Binding.DataType->ShaderValueType->GetMemberStructTypes();
					StructTypes.Add(Binding.DataType->ShaderValueType);

					TMap<FName, FName> FriendlyNameMap;
					for (const FShaderValueTypeHandle& TypeHandle : StructTypes )
					{
						FOptimusDataTypeHandle DataTypeHandle = FOptimusDataTypeRegistry::Get().FindType(TypeHandle->Name);
						if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(DataTypeHandle->TypeObject))
						{
							FriendlyNameMap.Add(TypeHandle->Name) = Optimus::GetTypeName(Struct, false);
						}
					}

					for (const FShaderValueTypeHandle& TypeHandle : StructTypes )
					{
						const FString StructName = TypeHandle->ToString();
						if (!StructsSeen.Contains(StructName))
						{
							Structs.Add(TypeHandle->GetTypeDeclaration(FriendlyNameMap, true) + TEXT(";\n\n"));
							StructsSeen.Add(StructName);
						}	
					}
				}
			}
		}
	};

	CollectStructs(InputBindingArray);
	CollectStructs(OutputBindingArray);
	for (const FOptimusSecondaryInputBindingsGroup& InputGroup: SecondaryInputBindingGroups)
	{
		CollectStructs(InputGroup.BindingArray);
	}
	
	TArray<FString> Declarations;

	// FIXME: Lump input/output functions together into single context.
	auto ContextsPredicate = [](const FOptimusParameterBinding& A, const FOptimusParameterBinding &B)
	{
		for (int32 Index = 0; Index < FMath::Min(A.DataDomain.DimensionNames.Num(), B.DataDomain.DimensionNames.Num()); Index++)
		{
			if (A.DataDomain.DimensionNames[Index] != B.DataDomain.DimensionNames[Index])
			{
				return FNameLexicalLess()(A.DataDomain.DimensionNames[Index], B.DataDomain.DimensionNames[Index]);
			}
		}
		return false;
	};
	
	TSet<TArray<FName>> SeenDataDomains;

	auto AddCountFunctionIfNeeded = [&Declarations, &SeenDataDomains](const TArray<FName>& InContextNames)
	{
		if (!InContextNames.IsEmpty() && !SeenDataDomains.Contains(InContextNames))
		{
			FString CountNameInfix;

			for (FName ContextName: InContextNames)
			{
				CountNameInfix.Append(ContextName.ToString());
			}
			Declarations.Add(FString::Printf(TEXT("uint Num%s();"), *CountNameInfix));
			SeenDataDomains.Add(InContextNames);
		}
	};

	auto AddInputBindings = [&](const TArray<FOptimusParameterBinding>& InBindings)
	{
		for (const FOptimusParameterBinding& Binding: InBindings)
		{
			AddCountFunctionIfNeeded(Binding.DataDomain.DimensionNames);
		
			if (Binding.DataType.IsValid() && Binding.DataType->ShaderValueType.IsValid())
			{
				Declarations.Add(GetDeclarationForBinding(Binding, true));
			}
			else
			{
				Declarations.Add(FString::Printf(TEXT("// Error: Binding \"%s\" is not supported"), *Binding.Name.ToString()) );
			}
		}
	};

	TArray<FOptimusParameterBinding> Bindings = InputBindingArray.InnerArray;
	Bindings.StableSort(ContextsPredicate);
	AddInputBindings(Bindings);

	Bindings = OutputBindingArray.InnerArray;
	Bindings.StableSort(ContextsPredicate);
	for (const FOptimusParameterBinding& Binding: Bindings)
	{
		AddCountFunctionIfNeeded(Binding.DataDomain.DimensionNames);
		
		TArray<FString> Indexes;
		for (FString IndexName: GetIndexNamesFromDataDomainLevels(Binding.DataDomain.DimensionNames))
		{
			Indexes.Add(FString::Printf(TEXT("uint %s"), *IndexName));
		}
		
		Declarations.Add(GetDeclarationForBinding(Binding, false));
	}

	for (const FOptimusSecondaryInputBindingsGroup& InputGroup: SecondaryInputBindingGroups)
	{
		Bindings = InputGroup.BindingArray.InnerArray;
		if (!Bindings.IsEmpty())
		{
			Bindings.Sort(ContextsPredicate);
			Declarations.Add(FString::Printf(TEXT("namespace %s {"), *InputGroup.GroupName.ToString()));
			AddInputBindings(Bindings);
			Declarations.Add("}");
		}
	}
	
	ShaderSource.Declarations.Reset();
	if (!Structs.IsEmpty())
	{
		ShaderSource.Declarations += TEXT("// Type declarations\n");
		ShaderSource.Declarations += FString::Join(Structs, TEXT("\n")) + TEXT("\n");
	}
	if (!Declarations.IsEmpty())
	{
		ShaderSource.Declarations += TEXT("// Parameters and resource read/write functions\n");
		ShaderSource.Declarations += FString::Join(Declarations, TEXT("\n"));
	}
	ShaderSource.Declarations += "\n// Resource Indexing\n";
	ShaderSource.Declarations += "uint Index;	// From SV_DispatchThreadID.x\n";
	ShaderSource.Declarations += "uint3 ReadNumThreads();\n";
}

UOptimusNodePin* UOptimusNode_CustomComputeKernel::GetPrimaryGroupPin_Internal() const
{
	// Always the first pin, as per PostLoadAddMissingPrimaryGroupPin and ConstructNode
	return GetPins()[0];
}


void UOptimusNode_CustomComputeKernel::PostLoadExtractExecutionDomain()
{
	ExecutionDomain = {};
	// Check if there's an execution node connected and grab the domain from it.
	FOptimusDataTypeHandle IntVector3Type = FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector3>::Get());

	for (UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->GetDirection() == EOptimusNodePinDirection::Input && Pin->GetDataType() == IntVector3Type)
		{
			TArray<FOptimusRoutedNodePin> ConnectedPins = Pin->GetConnectedPinsWithRouting();
			if (ConnectedPins.Num() == 1)
			{
				if (const UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(ConnectedPins[0].NodePin->GetOwningNode()))
				{
					if (const IOptimusDeprecatedExecutionDataInterface* ExecDataInterface = Cast<IOptimusDeprecatedExecutionDataInterface>(DataInterfaceNode->GetDataInterface(GetTransientPackage())))
					{
						FName Domain = ExecDataInterface->GetSelectedExecutionDomainName();
						ExecutionDomain = {Domain};
					}
				}
			}
		}
	}
}

void UOptimusNode_CustomComputeKernel::PostLoadAddMissingPrimaryGroupPin()
{
	// Add a group pin for the primary group since old nodes did not have one
	TArray<UOptimusNodePin*> PrimaryGroupInputPins;
	UOptimusNodePin* BeforePin = nullptr;
		
	for (UOptimusNodePin* Pin: GetPins())
	{
		if (IsConnectableInputDataPin(Pin))
		{
			PrimaryGroupInputPins.Add(Pin);
		}
		else if (BeforePin == nullptr)
		{
			BeforePin = Pin;
		}
	}

	UOptimusNodePin* PrimaryGroupPin = AddGroupingPinDirect(PrimaryGroupPinName, EOptimusNodePinDirection::Input, BeforePin);
		
	for (UOptimusNodePin* InputPin : PrimaryGroupInputPins)
	{
		MovePinToGroupPinDirect(InputPin, PrimaryGroupPin);
	}

	// Making sure the primary group pin is the first pin
	if (!ensure(GetPins()[0] == PrimaryGroupPin))
	{
		MovePinDirect(PrimaryGroupPin, GetPins()[0]);
	}
	
	PrimaryGroupPin->SetIsExpanded(PrimaryGroupPin->GetSubPins().Num() > 0);

}

void UOptimusNode_CustomComputeKernel::PostLoadExtractAtomicModeFromConnectedResource()
{
	FOptimusDataTypeHandle IntType = FOptimusDataTypeRegistry::Get().FindType(*FIntProperty::StaticClass());

	TArray<TPair<UOptimusNodePin*, EOptimusBufferWriteType>> PinsToProcess;
	
	for (UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->GetDirection() == EOptimusNodePinDirection::Output && Pin->GetDataType() == IntType)
		{
			TArray<FOptimusRoutedNodePin> ConnectedPins = Pin->GetConnectedPinsWithRouting();

			for (const FOptimusRoutedNodePin& ConnectedPin : ConnectedPins)
			{
				if (const UOptimusNode_ResourceAccessorBase* ResourceNode = Cast<UOptimusNode_ResourceAccessorBase>(ConnectedPin.NodePin->GetOwningNode()))
				{
					if(ResourceNode->GetDeprecatedBufferWriteType() != EOptimusBufferWriteType::Write)
					{
						if (FOptimusParameterBinding* Binding = OutputBindingArray.FindByPredicate([Pin](const FOptimusParameterBinding& InBinding)
							{
								return InBinding.Name == Pin->GetFName(); 
							}))
						{
							Binding->bSupportAtomicIfCompatibleDataType = true;

							// Auto fixup is only possible for valid connections,
							// where only a single resource node is connected to the atomic pin
							if (ConnectedPins.Num() == 1)
							{
								PinsToProcess.Add({Pin, ResourceNode->GetDeprecatedBufferWriteType()});
							}
						}
					}
				}
			}
		}
	}

	if (!PinsToProcess.IsEmpty())
	{
		TArray<FString> Statements;
		ShaderSource.ShaderText.ParseIntoArray(Statements, TEXT(";"));

		bool bModified = false;
		for (FString& Statement : Statements)
		{
			for (const TPair<UOptimusNodePin*, EOptimusBufferWriteType> PinInfo : PinsToProcess)
			{
				// See UOptimusNode_ComputeKernelBase::ProcessOutputPinForComputeKernel
				FString OldName = FString::Printf(TEXT("Write%s"), *(PinInfo.Key->GetName()));;
				
				FString NewName = GetAtomicWriteFunctionName(PinInfo.Value, PinInfo.Key->GetName());

				int32 NumReplaced = Statement.ReplaceInline(*OldName, *NewName, ESearchCase::CaseSensitive );
				if (NumReplaced > 0)
				{
					// One function call per statement at most, no need to look further
					bModified = true;
					break;
				}
			}
		}

		if (bModified)
		{
			ShaderSource.ShaderText = FString::Join(Statements, TEXT(";"));
			UpdatePreamble();	
		}
	}
}


bool UOptimusNode_CustomComputeKernel::PostLoadRemoveDeprecatedNumThreadsPin()
{
	// NumThreads is now a reserved name that pins cannot use
	const FName NumThreadsPinName = GetNumThreadsReservedName();
	
	UOptimusNodePin* PrimaryGroupPin = GetPrimaryGroupPin_Internal();
	
	if (!ensure(InputBindingArray.InnerArray.Num() == PrimaryGroupPin->GetSubPins().Num()))
	{
		return false;
	}

	bool bNodeChanged = false;
	for (int32 BindingIndex = 0; BindingIndex < PrimaryGroupPin->GetSubPins().Num(); BindingIndex++)
	{
		UOptimusNodePin* Pin = PrimaryGroupPin->GetSubPins()[BindingIndex];
		
		if (Pin->GetFName() != NumThreadsPinName)
		{
			continue;
		}

		// Pins and Binding array should match 1:1
		if (!ensure(InputBindingArray.InnerArray[BindingIndex].Name == NumThreadsPinName))
		{
			continue;
		}

		RemovePin(Pin);
		
		InputBindingArray.InnerArray.RemoveAt(BindingIndex);
		bNodeChanged = true;
		
		break;
	}

	UpdatePreamble();
	
	return bNodeChanged;
}

FString UOptimusNode_CustomComputeKernel::GetDeclarationForBinding(const FOptimusParameterBinding& Binding, bool bIsInput)
{
	TArray<FString> Indexes;
	for (FString IndexName: GetIndexNamesFromDataDomain(Binding.DataDomain))
	{
		Indexes.Add(FString::Printf(TEXT("uint %s"), *IndexName));
	}

	if (bIsInput)
	{
		return FString::Printf(TEXT("%s Read%s(%s);"), 
			*GetShaderValueTypeFriendlyName(Binding.DataType), *Binding.Name.ToString(), *FString::Join(Indexes, TEXT(", ")));
	}
	else
	{
		TArray<FString> Declarations;
		Declarations.Add(FString::Printf(TEXT("void Write%s(%s, %s Value);"),
			*Binding.Name.ToString(), *FString::Join(Indexes, TEXT(", ")), *GetShaderValueTypeFriendlyName(Binding.DataType)));

		if (DoesBindingSupportAtomic(Binding))
		{
			for (uint8 WriteType = uint8(EOptimusBufferWriteType::Write) + 1; WriteType < uint8(EOptimusBufferWriteType::Count); WriteType++)
			{
				FString FunctionName = GetAtomicWriteFunctionName((EOptimusBufferWriteType)WriteType, Binding.Name.ToString());
				
				FString TypeName = GetShaderValueTypeFriendlyName(Binding.DataType);
				Declarations.Add(FString::Printf(TEXT("%s %s(%s, %s Value);"),
					*TypeName, *FunctionName, *FString::Join(Indexes, TEXT(", ")), *TypeName));	
			}
		}

		if (Binding.bSupportRead)
		{
			Declarations.Add(FString::Printf(TEXT("%s Read%s(%s);"), 
				*GetShaderValueTypeFriendlyName(Binding.DataType), *Binding.Name.ToString(), *FString::Join(Indexes, TEXT(", "))));
		}

		return FString::Join(Declarations, TEXT("\n"));
	}
}

#undef LOCTEXT_NAMESPACE