// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusExecutionDomainProvider.h"
#include "IOptimusNodeAdderPinProvider.h"
#include "IOptimusParameterBindingProvider.h"
#include "IOptimusShaderTextProvider.h"
#include "OptimusBindingTypes.h"
#include "OptimusDataDomain.h"
#include "OptimusNode_ComputeKernelBase.h"
#include "OptimusShaderText.h"
#include "OptimusValidatedName.h"

#include "OptimusNode_CustomComputeKernel.generated.h"


enum class EOptimusNodePinDirection : uint8;
struct FOptimusExecutionDomain;

USTRUCT()
struct FOptimusSecondaryInputBindingsGroup
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category=Group)
	FOptimusValidatedName GroupName;
	
	UPROPERTY(EditAnywhere, Category=Group, DisplayName = "Bindings", meta=(AllowParameters))
	FOptimusParameterBindingArray BindingArray;
};


UCLASS()
class UOptimusNode_CustomComputeKernel :
	public UOptimusNode_ComputeKernelBase,
	public IOptimusShaderTextProvider,
	public IOptimusParameterBindingProvider,
	public IOptimusNodeAdderPinProvider,
	public IOptimusExecutionDomainProvider
{
	GENERATED_BODY()

public:
	UOptimusNode_CustomComputeKernel();

	// UOptimusNode overrides
	FName GetNodeCategory() const override 
	{
		return CategoryName::Deformers;
	}

	void OnDataTypeChanged(FName InTypeName) override;
	void PostLoadNodeSpecificData() override;

	// UOptimusNode_ComputeKernelBase overrides
	FString GetKernelHlslName() const override;
	FIntVector GetGroupSize() const override { return GroupSize; }
	FString GetKernelSourceText(bool bInIsUnifiedDispatch) const override;
	TArray<TObjectPtr<UComputeSource>> GetAdditionalSources() const override { return AdditionalSources; }

	// IOptimusComputeKernelProvider
	FOptimusExecutionDomain GetExecutionDomain() const override;
	const UOptimusNodePin* GetPrimaryGroupPin() const override;
	UComputeDataInterface* MakeKernelDataInterface(UObject* InOuter) const override;
	bool DoesOutputPinSupportAtomic(const UOptimusNodePin* InPin) const override;
	bool DoesOutputPinSupportRead(const UOptimusNodePin* InPin) const override;

#if WITH_EDITOR
	// IOptimusShaderTextProvider overrides
	FString GetNameForShaderTextEditor() const override;
	FString GetDeclarations() const override;
	FString GetShaderText() const override;
	void SetShaderText(const FString& NewText) override;
	bool IsShaderTextReadOnly() const override;
	// IOptimusShaderTextProvider overrides
#endif
	
	// IOptimusParameterBindingProvider
	FString GetBindingDeclaration(FName BindingName) const override;
	bool GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const override;
	bool GetBindingSupportReadCheckBoxVisibility(FName BindingName) const override;
	EOptimusDataTypeUsageFlags GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const override;
	
	// IOptimusNodeAdderPinProvider
	TArray<FAdderPinAction> GetAvailableAdderPinActions(
		const UOptimusNodePin* InSourcePin,
		EOptimusNodePinDirection InNewPinDirection,
		FString* OutReason = nullptr
		) const override;

	TArray<UOptimusNodePin*> TryAddPinFromPin(
		const FAdderPinAction& InSelectedAction, 
		UOptimusNodePin* InSourcePin, 
		FName InNameToUse
		) override;
	
	bool RemoveAddedPins(
		TConstArrayView<UOptimusNodePin*> InAddedPinsToRemove
	) override;
	


	// IOptimusExecutionDomainProvider
	TArray<FName> GetExecutionDomains() const override;
	
	
	// FIXME: Use drop-down with a preset list + allow custom entry.
	UPROPERTY(EditAnywhere, Category=Settings)
	FName Category = CategoryName::Deformers;
	
	/** Name of kernel. This is also used as the entry point function name in generated code. */
	UPROPERTY(EditAnywhere, Category=Settings)
	FOptimusValidatedName KernelName;

	/** The execution domain that this kernel operates on. The size of the domain is governed by
	 *  the component binding that flows into the primary input group of this kernel.
	 */
	UPROPERTY(EditAnywhere, Category=Settings)
	FOptimusExecutionDomain ExecutionDomain;
	
	/** 
	 * Number of threads in a thread group. 
	 * Thread groups have 3 dimensions. 
	 * It's better to have the total threads (X*Y*Z) be a value divisible by 32. 
	 */
	UPROPERTY(EditAnywhere, Category=Settings, meta=(Min=1))
	FIntVector GroupSize = FIntVector(64, 1, 1);

	/** Parameter bindings. Parameters are uniform values. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta=(DeprecatedProperty))
	TArray<FOptimus_ShaderBinding> Parameters_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Input bindings. Each one is a function that should be connected to an implementation in a data interface. */
	UPROPERTY(meta=(DeprecatedProperty))
	TArray<FOptimusParameterBinding> InputBindings_DEPRECATED;

	/** Output bindings. Each one is a function that should be connected to an implementation in a data interface. */
	UPROPERTY(meta=(DeprecatedProperty))
	TArray<FOptimusParameterBinding> OutputBindings_DEPRECATED;	

	/** Input bindings. Each one is a function that should be connected to an implementation in a data interface. */
	UPROPERTY(EditAnywhere, Category = "Primary Bindings", DisplayName = "Input Bindings", meta=(AllowParameters))
	FOptimusParameterBindingArray InputBindingArray;

	/** Output bindings. Each one is a function that should be connected to an implementation in a data interface. */
	UPROPERTY(EditAnywhere, Category = "Primary Bindings", DisplayName = "Output Bindings")
	FOptimusParameterBindingArray OutputBindingArray;

	/** Secondary bindings.*/
	UPROPERTY(EditAnywhere, Category = "Secondary Input Bindings Groups", DisplayName = "Secondary Input Bindings")
	TArray<FOptimusSecondaryInputBindingsGroup> SecondaryInputBindingGroups;
	
	/** Additional source includes. */
	UPROPERTY(EditAnywhere, Category = Source)
	TArray<TObjectPtr<UComputeSource>> AdditionalSources;

	/** 
	 * The kernel source code. 
	 * If the code contains more than just the kernel entry function, then place the kernel entry function inside a KERNEL {} block.
	 */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DisplayName = "Kernel Source"))
	FOptimusShaderText ShaderSource;

#if WITH_EDITOR
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void Serialize(FArchive& Ar) override;
	
	
protected:
	void ConstructNode() override;
	bool ValidateConnection(const UOptimusNodePin& InThisNodesPin, const UOptimusNodePin& InOtherNodesPin, FString* OutReason) const override;
	TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const override;
	
private:

#if WITH_EDITOR
	void PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent);
	void PropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	void PropertyArrayItemAdded(const FPropertyChangedEvent& InPropertyChangedEvent);
	void PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent);
	void PropertyArrayCleared(const FPropertyChangedEvent& InPropertyChangedEvent);
	void PropertyArrayItemMoved(const FPropertyChangedEvent& InPropertyChangedEvent);
#endif
	
	void UpdatePreamble();

	UOptimusNodePin* GetPrimaryGroupPin_Internal() const;
	void PostLoadExtractExecutionDomain();
	void PostLoadAddMissingPrimaryGroupPin();
	void PostLoadExtractAtomicModeFromConnectedResource();
	
	// Called during UOptimusDeformer::PostLoad()
	bool PostLoadRemoveDeprecatedNumThreadsPin();
	
	FName GetSanitizedNewPinName(
		UOptimusNodePin* InParentPin,
		FName InPinName
		);
	
	static FString GetDeclarationForBinding(const FOptimusParameterBinding& Binding, bool bIsInput);

	static bool DoesSourceSupportUnifiedDispatch(const UOptimusNodePin& InOtherNodesPin);
	friend class UOptimusDeformer; // For PostLoad fixup
	friend class UOptimusCustomComputeKernelDataInterface;
};
