// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_DataInterface.h"

#include "OptimusCoreModule.h"
#include "OptimusNodePin.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusNode_ComponentSource.h"
#include "OptimusObjectVersion.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_DataInterface)


#define LOCTEXT_NAMESPACE "OptimusNode_DataInterface"


UOptimusNode_DataInterface::UOptimusNode_DataInterface()
{
}


bool UOptimusNode_DataInterface::ValidateConnection(
	const UOptimusNodePin& InThisNodesPin,
	const UOptimusNodePin& InOtherNodesPin,
	FString* OutReason
	) const
{
	// FIXME: Once we have connection evaluation, use that.
	if (!GetPins().IsEmpty() && &InThisNodesPin == GetPins()[0])
	{
		const UOptimusNode_ComponentSource* SourceNode = Cast<UOptimusNode_ComponentSource>(InOtherNodesPin.GetOwningNode());
		if (SourceNode)
		{
			const UOptimusComponentSource* ComponentSource = SourceNode->GetComponentBinding()->GetComponentSource();
			if (!IsComponentSourceCompatible(ComponentSource))
			{
				if (OutReason)
				{
					*OutReason = FString::Printf(TEXT("This data interface requires a %s which is not a child class of %s from the Component Source."),
						*DataInterfaceData->GetRequiredComponentClass()->GetName(),
						*ComponentSource->GetComponentClass()->GetName());
				}
				return false;
			}	
		}

		// In other cases, the component source may come from the upstream of the connected node (eg. Sub Graph Terminal),
		// and thus no way to provide error check until compile time
	}

	return true;
}

TOptional<FText> UOptimusNode_DataInterface::ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
{
	if (!DataInterfaceClass)
	{
		return LOCTEXT("NoAssociatedClass", "Node has none or invalid data interface class associated with it. Delete and re-create the node.");
	}
	// Ensure that we have something connected to the component binding input pin.
	UOptimusComponentSourceBinding* PrimaryBinding = GetComponentBinding(InContext);
	if (PrimaryBinding == nullptr)
	{
		return FText::Format(LOCTEXT("NoBindingConnected", "No component binding connected to the {0} pin"), FText::FromName(GetComponentPin()->GetUniqueName()));
	}

	// Are all the other connected _input_ pins using the same binding?
	const UOptimusNodeGraph* Graph = GetOwningGraph();
	for (const UOptimusNodePin* Pin: GetPins())
	{
		if (Pin->GetDirection() == EOptimusNodePinDirection::Input && Pin != GetComponentPin())
		{
			TSet<UOptimusComponentSourceBinding*> Bindings = Graph->GetComponentSourceBindingsForPin(Pin, InContext);
			if (Bindings.Num() > 1)
			{
				return FText::Format(LOCTEXT("MultipleBindingsOnPin", "Multiple bindings found for pin {0}"), FText::FromName(Pin->GetUniqueName()));
			}

			if (Bindings.Num() == 1 && !Bindings.Contains(PrimaryBinding))
			{
				return FText::Format(LOCTEXT("IncompatibleBinding", "Bindings for pin {0} are not the same as for the {1} pin"), 
							FText::FromName(Pin->GetUniqueName()), FText::FromName(GetComponentPin()->GetUniqueName()));
			}
		}
	}

	if (DataInterfaceData)
	{
		const TOptional<FText> ErrorIfDetected = DataInterfaceData->ValidateForCompile();
		if (ErrorIfDetected.IsSet())
		{
			return ErrorIfDetected;
		}
	}
	
	return {};
}

void UOptimusNode_DataInterface::PostLoadNodeSpecificData()
{
	Super::PostLoadNodeSpecificData();
	
	// Previously DataInterfaceData wasn't always created.
	if (DataInterfaceClass && !DataInterfaceData)
	{
		DataInterfaceData = NewObject<UOptimusComputeDataInterface>(this, DataInterfaceClass);
		DataInterfaceData->SetFlags(RF_Transactional);
	}

	// Add in the component pin.
	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::ComponentProviderSupport)
	{
		CreateComponentPin();
	}

	// Add in the property pins if there are any.
	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::PropertyPinSupport)
	{
		CreatePropertyPinsFromDataInterface(DataInterfaceData, false);
	}
}

void UOptimusNode_DataInterface::OnDataTypeChanged(FName InTypeName)
{
	Super::OnDataTypeChanged(InTypeName);

	DataInterfaceData->OnDataTypeChanged(InTypeName);
}

FText UOptimusNode_DataInterface::GetTooltipText() const
{
#if WITH_EDITORONLY_DATA
	return DataInterfaceData->GetClass()->GetToolTipText();
#else
	return {};
#endif
}

void UOptimusNode_DataInterface::ExportState(FArchive& Ar) const
{
	Super::ExportState(Ar);

	DataInterfaceData->ExportState(Ar);
	
}

void UOptimusNode_DataInterface::ImportState(FArchive& Ar)
{
	Super::ImportState(Ar);

	DataInterfaceData = NewObject<UOptimusComputeDataInterface>(this, DataInterfaceClass);
	DataInterfaceData->SetFlags(RF_Transactional);
	
	DataInterfaceData->ImportState(Ar);
}

bool UOptimusNode_DataInterface::IsComponentSourceCompatible(const UOptimusComponentSource* InComponentSource) const
{
	return InComponentSource && InComponentSource->GetComponentClass()->IsChildOf(DataInterfaceData->GetRequiredComponentClass());
}

void UOptimusNode_DataInterface::RecreatePinsFromPinDefinitions()
{
	// Recreate all the pins
	// Save the links and readd them later when new pins are created
	TMap<FName, TArray<UOptimusNodePin*>> ConnectedPinsMap;

	TArray<UOptimusNodePin*> PinsToRemove;
	
	for (UOptimusNodePin* Pin : GetPins())
	{
		if (Pin != GetComponentPin())
		{
			ConnectedPinsMap.Add(Pin->GetFName()) = Pin->GetConnectedPins();
			PinsToRemove.Add(Pin);
		}
	}	

	for (UOptimusNodePin* Pin : PinsToRemove)
	{
		RemovePin(Pin);
	}

	CreatePropertyPinsFromDataInterface(DataInterfaceData, true);
	CreateShaderPinsFromDataInterface(DataInterfaceData, true);

	for (UOptimusNodePin* Pin : GetPins())
	{
		if (TArray<UOptimusNodePin*>* ConnectedPins = ConnectedPinsMap.Find(Pin->GetFName()))
		{
			for (UOptimusNodePin* ConnectedPin : *ConnectedPins)
			{
				GetOwningGraph()->AddLink(Pin, ConnectedPin);
			}
		}	
	}	
}

void UOptimusNode_DataInterface::RenamePinFromPinDefinition(FName InOld, FName InNew)
{
	UOptimusNodePin* Pin = FindPin(InOld.ToString());
	SetPinName(Pin, InNew);
}

void UOptimusNode_DataInterface::UpdateDisplayNameFromDataInterface()
{
	if (ensure(DataInterfaceData))
	{
		SetDisplayName(FText::FromString(DataInterfaceData->GetDisplayName()));
	}
}

void UOptimusNode_DataInterface::InitializeTransientData()
{
	if (ensure(DataInterfaceData))
	{
		if (DataInterfaceData->CanPinDefinitionChange())
		{
			EnableDynamicPins();
		}
		
		DataInterfaceData->RegisterPropertyChangeDelegatesForOwningNode(this);
	}
}

void UOptimusNode_DataInterface::InitializePinsDisplayName()
{
	if (ensure(DataInterfaceData))
	{
		for (const FOptimusCDIPinDefinition& Def : DataInterfaceData->GetPinDefinitions())
		{
			if (ensure(!Def.PinName.IsNone()))
			{
				if (UOptimusNodePin* Pin = GetPinByName(Def.PinName))
				{
					Pin->SetDisplayName(Def.DisplayName);
				}
			}
		}
	}
}

void UOptimusNode_DataInterface::Serialize(FArchive& Ar)
{
 	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FOptimusObjectVersion::GUID);
}


UOptimusComputeDataInterface* UOptimusNode_DataInterface::GetDataInterface(UObject* InOuter) const
{
	// Asset is probably broken, or refers to a class that no longer exists. 
	if (!DataInterfaceClass)
	{
		return nullptr;
	}
	
	// Legacy data may not have a DataInterfaceData object.
	if (DataInterfaceData == nullptr || !DataInterfaceData->IsA(DataInterfaceClass))
	{
		return NewObject<UOptimusComputeDataInterface>(InOuter, DataInterfaceClass);
	}

	FObjectDuplicationParameters DupParams = InitStaticDuplicateObjectParams(DataInterfaceData, InOuter);
	return Cast<UOptimusComputeDataInterface>(StaticDuplicateObjectEx(DupParams));
}


int32 UOptimusNode_DataInterface::GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const
{
	if (!InPin || InPin->GetParentPin() != nullptr)
	{
		return INDEX_NONE;
	}

	// FIXME: This information should be baked into the pin definition so we don't have to
	// look it up repeatedly.
	const TArray<FOptimusCDIPinDefinition> PinDefinitions = DataInterfaceData->GetPinDefinitions();

	int32 PinIndex = INDEX_NONE;
	for (int32 Index = 0 ; Index < PinDefinitions.Num(); ++Index)
	{
		if (InPin->GetUniqueName() == PinDefinitions[Index].PinName)
		{
			PinIndex = Index;
			break;
		}
	}
	if (!ensure(PinIndex != INDEX_NONE))
	{
		return INDEX_NONE;
	}

	const FString FunctionName = PinDefinitions[PinIndex].DataFunctionName;
	
	TArray<FShaderFunctionDefinition> FunctionDefinitions;
	if (InPin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		DataInterfaceData->GetSupportedOutputs(FunctionDefinitions);
	}
	else
	{
		DataInterfaceData->GetSupportedInputs(FunctionDefinitions);
	}
	
	return FunctionDefinitions.IndexOfByPredicate(
		[FunctionName](const FShaderFunctionDefinition& InDef)
		{
			return InDef.Name == FunctionName;
		});
}


void UOptimusNode_DataInterface::SetDataInterfaceClass(
	TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass
	)
{
	DataInterfaceClass = InDataInterfaceClass;
	DataInterfaceData = NewObject<UOptimusComputeDataInterface>(this, DataInterfaceClass);
	// Undo support
	DataInterfaceData->SetFlags(RF_Transactional);
	DataInterfaceData->Initialize();
}

UOptimusComponentSourceBinding* UOptimusNode_DataInterface::GetComponentBinding(const FOptimusPinTraversalContext& InContext) const
{
	const UOptimusNodeGraph* Graph = GetOwningGraph();
	TSet<UOptimusComponentSourceBinding*> Bindings = Graph->GetComponentSourceBindingsForPin(GetComponentPin(), InContext);
	
	if (!Bindings.IsEmpty() && ensure(Bindings.Num() == 1))
	{
		return Bindings.Array()[0];
	}

	// Default to the primary binding, but only if we're at the top-most level of the graph.
	if (Optimus::IsExecutionGraphType(Graph->GetGraphType()))
	{
		const UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(Graph->GetCollectionOwner());
		if (ensure(Deformer))
		{
			return Deformer->GetPrimaryComponentBinding();
		}
	}
	else
	{
		const UOptimusNodeSubGraph* SubGraph = Cast<UOptimusNodeSubGraph>(Graph);
		if (ensure(SubGraph))
		{
			return SubGraph->GetDefaultComponentBinding(InContext);
		}
	}

	
	return nullptr;
}

EOptimusPinMutability UOptimusNode_DataInterface::GetOutputPinMutability(const UOptimusNodePin* InPin) const
{
	const TArray<FOptimusCDIPinDefinition> PinDefinitions = DataInterfaceData->GetPinDefinitions();

	int32 PinDefinitionIndex = INDEX_NONE;
	for (int32 Index = 0 ; Index < PinDefinitions.Num(); ++Index)
	{
		if (PinDefinitions[Index].PinName == InPin->GetUniqueName())
		{
			PinDefinitionIndex = Index;
			break;
		}
	}
	if (!ensure(PinDefinitionIndex != INDEX_NONE))
	{
		return EOptimusPinMutability::Mutable;
	}

	return PinDefinitions[PinDefinitionIndex].bMutable ? EOptimusPinMutability::Mutable : EOptimusPinMutability::Immutable;
}

TArray<UOptimusNodePin*> UOptimusNode_DataInterface::GetPropertyPins() const
{
	const TArray<FOptimusCDIPropertyPinDefinition> PropertyPinDefinitions = DataInterfaceData->GetPropertyPinDefinitions();

	TArray<UOptimusNodePin*> Result;
	for (const FOptimusCDIPropertyPinDefinition& Definition : PropertyPinDefinitions)
	{
		UOptimusNodePin* Pin = FindPinFromPath({Definition.PinName});
		// No need to ensure a pin is found here since older version of the data interface does not attempt to create
		// property pins that were added later. 
		if (Pin && Pin->GetDirection() == EOptimusNodePinDirection::Input)
		{
			Result.Add(Pin);
		}
	}

	return Result;
}


void UOptimusNode_DataInterface::ConstructNode()
{
	// Create the component pin.
	if (ensure(DataInterfaceClass))
	{
		if (!DataInterfaceData)
		{
			DataInterfaceData = NewObject<UOptimusComputeDataInterface>(this, DataInterfaceClass);
			DataInterfaceData->SetFlags(RF_Transactional);
		}
		SetDisplayName(FText::FromString(DataInterfaceData->GetDisplayName()));
		CreateComponentPin();
		CreatePropertyPinsFromDataInterface(DataInterfaceData, false);
		CreateShaderPinsFromDataInterface(DataInterfaceData, false);
	}
}

FText UOptimusNode_DataInterface::GetDisplayName() const
{
	FText SerializedDisplayName = Super::GetDisplayName();

	if (DataInterfaceData && !DataInterfaceData->IsVisible())
	{
		FText OutdatedSuffix = LOCTEXT("OutdatedSuffix", "(Outdated)");

		return FText::Join(FText::GetEmpty(), SerializedDisplayName, OutdatedSuffix);
	}

	return SerializedDisplayName;
}


void UOptimusNode_DataInterface::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	check(DataInterfaceData);
	check(DataInterfaceData->GetOuter() == this);
}


void UOptimusNode_DataInterface::CreateShaderPinsFromDataInterface(const UOptimusComputeDataInterface* InDataInterface, bool bSupportUndo)
{
	// A data interface provides read and write functions. A data interface node exposes
	// the read functions as output pins to be fed into kernel nodes (or into other interface
	// nodes' write functions). Conversely all write functions are exposed as input pins,
	// since the data is being written to.
	const TArray<FOptimusCDIPinDefinition> PinDefinitions = InDataInterface->GetPinDefinitions();

	TArray<FShaderFunctionDefinition> ReadFunctions;
	InDataInterface->GetSupportedInputs(ReadFunctions);
	
	TMap<FString, const FShaderFunctionDefinition *> ReadFunctionMap;
	for (const FShaderFunctionDefinition& Def: ReadFunctions)
	{
		ReadFunctionMap.Add(Def.Name, &Def);
	}

	TArray<FShaderFunctionDefinition> WriteFunctions;
	InDataInterface->GetSupportedOutputs(WriteFunctions);
	
	TMap<FString, const FShaderFunctionDefinition *> WriteFunctionMap;
	for (const FShaderFunctionDefinition& Def: WriteFunctions)
	{
		WriteFunctionMap.Add(Def.Name, &Def);
	}

	for (const FOptimusCDIPinDefinition& Def: PinDefinitions)
	{
		if (ensure(!Def.PinName.IsNone()))
		{
			CreatePinFromDefinition(Def, ReadFunctionMap, WriteFunctionMap, bSupportUndo);
		}
	}
}

void UOptimusNode_DataInterface::CreatePinFromDefinition(const FOptimusCDIPinDefinition& InDefinition, const TMap<FString, const FShaderFunctionDefinition*>& InReadFunctionMap, const TMap<FString, const FShaderFunctionDefinition*>& InWriteFunctionMap, bool bSupportUndo)
{
	const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();

	// If there's no count function, then we have a value pin. The data function should
	// have a return parameter but no input parameters. The value function only exists in 
	// the read function map and so can only be an output pin.
	if (InDefinition.DataDimensions.IsEmpty())
	{
		if (!InReadFunctionMap.Contains(InDefinition.DataFunctionName))
		{
			UE_LOGF(LogOptimusCore, Error, "Data function %ls given for pin %ls in %ls does not exist",
				*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
			return;
		}

		const FShaderFunctionDefinition* FuncDef = InReadFunctionMap[InDefinition.DataFunctionName];
		if (!FuncDef->bHasReturnType || FuncDef->ParamTypes.Num() != 1)
		{
			UE_LOGF(LogOptimusCore, Error, "Data function %ls given for pin %ls in %ls does not return a single value",
				*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
			return;
		}

		const FShaderValueTypeHandle ValueTypeHandle = FuncDef->ParamTypes[0].ValueType;
		const FOptimusDataTypeRef PinDataType = TypeRegistry.FindType(ValueTypeHandle);
		if (!PinDataType.IsValid())
		{
			UE_LOGF(LogOptimusCore, Error, "Data function %ls given for pin %ls in %ls uses unsupported type '%ls'",
				*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName(),
				*ValueTypeHandle->ToString());
			return;
		}
		
		if (bSupportUndo)
		{
			AddPin(InDefinition.PinName, EOptimusNodePinDirection::Output, {}, PinDataType);
		}
		else
		{
			AddPinDirect(InDefinition.PinName, EOptimusNodePinDirection::Output, {}, PinDataType);
		}
	}
	else if (!InDefinition.DataFunctionName.IsEmpty())
	{
		// The count function is always in the read function list.
		for (const FOptimusCDIPinDefinition::FDimensionInfo& ContextInfo: InDefinition.DataDimensions)
		{
			if (!InReadFunctionMap.Contains(ContextInfo.CountFunctionName))
			{
				UE_LOGF(LogOptimusCore, Error, "Count function %ls given for pin %ls in %ls does not exist",
					*ContextInfo.CountFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
				return;
			}
		}

		FShaderValueTypeHandle ValueTypeHandle;
		EOptimusNodePinDirection PinDirection;
		
		if (InReadFunctionMap.Contains(InDefinition.DataFunctionName))
		{
			PinDirection = EOptimusNodePinDirection::Output;
			const FShaderFunctionDefinition* FuncDef = InReadFunctionMap[InDefinition.DataFunctionName];

			// FIXME: Ensure it takes a scalar uint/int as input index.
			if (!FuncDef->bHasReturnType || FuncDef->ParamTypes.Num() != (1 + InDefinition.DataDimensions.Num()))
			{
				UE_LOGF(LogOptimusCore, Error, "Data read function %ls given for pin %ls in %ls is not properly declared.",
					*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
				return;
			}

			// The return type dictates the pin type.
			ValueTypeHandle = FuncDef->ParamTypes[0].ValueType;
		}
		else if (InWriteFunctionMap.Contains(InDefinition.DataFunctionName))
		{
			PinDirection = EOptimusNodePinDirection::Input;
			
			const FShaderFunctionDefinition* FuncDef = InWriteFunctionMap[InDefinition.DataFunctionName];

			// FIXME: Ensure it takes a scalar uint/int as input index.
			if (FuncDef->bHasReturnType || FuncDef->ParamTypes.Num() != (1 + InDefinition.DataDimensions.Num()))
			{
				UE_LOGF(LogOptimusCore, Error, "Data write function %ls given for pin %ls in %ls is not properly declared.",
					*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
				return;
			}

			// The second argument dictates the pin type.
			ValueTypeHandle = FuncDef->ParamTypes[1].ValueType;
		}
		else
		{
			UE_LOGF(LogOptimusCore, Error, "Data function %ls given for pin %ls in %ls does not exist",
				*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
			return;
		}

		const FOptimusDataTypeRef PinDataType = TypeRegistry.FindType(ValueTypeHandle);
		if (!PinDataType.IsValid())
		{
			UE_LOGF(LogOptimusCore, Error, "Data function %ls given for pin %ls in %ls uses unsupported type '%ls'",
				*InDefinition.DataFunctionName, *InDefinition.PinName.ToString(), *DataInterfaceClass->GetName(),
				*ValueTypeHandle->ToString());
			return;
		}

		TArray<FName> ContextNames;
		for (const FOptimusCDIPinDefinition::FDimensionInfo& ContextInfo: InDefinition.DataDimensions)
		{
			ContextNames.Add(ContextInfo.ContextName);
		}

		const FOptimusDataDomain DataDomain{ContextNames, InDefinition.DomainMultiplier};
		if (bSupportUndo)
		{
			AddPin(InDefinition.PinName, PinDirection, DataDomain, PinDataType);	
		}
		else
		{
			AddPinDirect(InDefinition.PinName, PinDirection, DataDomain, PinDataType);
		}
	}
	else
	{
		UE_LOGF(LogOptimusCore, Error, "No data function given for pin %ls in %ls",
			*InDefinition.PinName.ToString(), *DataInterfaceClass->GetName());
	}
}

void UOptimusNode_DataInterface::CreatePropertyPinsFromDataInterface(const UOptimusComputeDataInterface* InDataInterface, bool bSupportUndo)
{
	if (!InDataInterface)
	{
		return;
	}

	const TArray<FOptimusCDIPropertyPinDefinition> PropertyPinDefinitions = InDataInterface->GetPropertyPinDefinitions();

	// Property pins should go before any shader pins
	UOptimusNodePin* BeforePin =nullptr;
	if (GetPins().Num() > 1)
	{
		BeforePin = GetPins()[1];
	}
	
	for (const FOptimusCDIPropertyPinDefinition& Definition : PropertyPinDefinitions)
	{
		if (bSupportUndo)
		{
			AddPin(Definition.PinName, EOptimusNodePinDirection::Input, FOptimusDataDomain(), Definition.DataType, BeforePin);
		}
		else
		{
			AddPinDirect(Definition.PinName, EOptimusNodePinDirection::Input, FOptimusDataDomain(), Definition.DataType, BeforePin);
		}
	}
}


void UOptimusNode_DataInterface::CreateComponentPin()
{
	const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
	FOptimusDataTypeRef ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());

	const UOptimusComponentSource* ComponentSource = UOptimusComponentSource::GetSourceFromDataInterface(DataInterfaceData);
	if (ensure(ComponentSourceType.IsValid()) &&
		ensure(ComponentSource))
	{
		// For back-comp: If we're coming in here from PostLoad and pins already exist, make sure to inject this new pin
		// as the first pin in the list.
		UOptimusNodePin* BeforePin = GetPins().IsEmpty() ? nullptr : GetPins()[0];
		
		AddPinDirect(ComponentSource->GetBindingName(), EOptimusNodePinDirection::Input, {}, ComponentSourceType, BeforePin);
	}
}

UOptimusNodePin* UOptimusNode_DataInterface::GetComponentPin() const
{
	// Is always the first pin, per CreateComponentPin.
	return GetPins()[0];
}

#undef LOCTEXT_NAMESPACE
