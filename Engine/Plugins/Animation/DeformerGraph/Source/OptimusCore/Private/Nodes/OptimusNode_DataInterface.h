// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusComponentBindingProvider.h"
#include "IOptimusDataInterfaceProvider.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "IOptimusPropertyPinProvider.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusComponentSource.h"

#include "OptimusNode.h"
#include "Templates/SubclassOf.h"

#include "OptimusNode_DataInterface.generated.h"

/**
 * 
 */
UCLASS(Hidden)
class UOptimusNode_DataInterface :
	public UOptimusNode,
	public IOptimusDataInterfaceProvider,
	public IOptimusComponentBindingProvider,
	public IOptimusPinMutabilityDefiner,
	public IOptimusPropertyPinProvider
{
	GENERATED_BODY()

public:
	UOptimusNode_DataInterface();

	virtual void SetDataInterfaceClass(TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass);
	bool IsComponentSourceCompatible(const UOptimusComponentSource* InComponentSource) const;
	
	// -- UOptimusNode overrides
	FName GetNodeCategory() const override 
	{
		return CategoryName::DataInterfaces;
	}
	void InitializeTransientData() override;
	void RecreatePinsFromPinDefinitions() override;
	void RenamePinFromPinDefinition(FName InOld, FName InNew) override;
	void UpdateDisplayNameFromDataInterface() override;
	
	// -- UObject overrides
	void Serialize(FArchive& Ar) override;

	// -- IOptimusDataInterfaceProvider implementations
	UOptimusComputeDataInterface *GetDataInterface(UObject *InOuter) const override;
	int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const override;
	
	// -- IOptimusDataInterfaceProvider & IOptimusComponentBindingProvider
	UOptimusComponentSourceBinding* GetComponentBinding(const FOptimusPinTraversalContext& InContext) const override;

	// -- IOptimusPinMutabilityDefiner
	EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const override;

	// -- IOptimusPropertyPinProvider
	TArray<UOptimusNodePin*> GetPropertyPins() const override;
protected:
	// -- UOptimusNode overrides
	void ConstructNode() override;
	FText GetDisplayName() const override;
	bool ValidateConnection(const UOptimusNodePin& InThisNodesPin, const UOptimusNodePin& InOtherNodesPin, FString* OutReason) const override;
	TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const override;
	void PostLoadNodeSpecificData() override;
	void OnDataTypeChanged(FName InTypeName) override;
	FText GetTooltipText() const override;

	void ExportState(FArchive& Ar) const override;
	void ImportState(FArchive& Ar) override;

	// -- UObject overrides
	void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	
private:
	void CreateShaderPinsFromDataInterface(const UOptimusComputeDataInterface *InDataInterface, bool bSupportUndo);
	void CreatePinFromDefinition(
		const FOptimusCDIPinDefinition &InDefinition,
		const TMap<FString, const FShaderFunctionDefinition *>& InReadFunctionMap,
		const TMap<FString, const FShaderFunctionDefinition *>& InWriteFunctionMap,
		bool bSupportUndo
		);

	void CreatePropertyPinsFromDataInterface(const UOptimusComputeDataInterface *InDataInterface, bool bSupportUndo);
	void CreateComponentPin();

	/** Allows the node to initialize all the pins display names from the definitions */
	virtual void InitializePinsDisplayName() override;

protected:
	friend class UOptimusDeformer;
	friend class UOptimusNodeGraph;

	/** Accessor for the deformer object to connect this node automatically on backcomp and to unlink it as well
	 *  if a component binding changes the source type.
	 */
	UOptimusNodePin* GetComponentPin() const;
	
	// The class of the data interface that this node represents. We call the CDO
	// to interrogate display names and pin definitions. This may change in the future once
	// data interfaces get tied closer to the objects they proxy.
	UPROPERTY()
	TObjectPtr<UClass> DataInterfaceClass;

	// Editable copy of the DataInterface for storing properties that will customize behavior on on the data interface.
	UPROPERTY(VisibleAnywhere, Instanced, Category=DataInterface)
	TObjectPtr<UOptimusComputeDataInterface> DataInterfaceData;
};
