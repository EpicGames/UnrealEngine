// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_FunctionReference.h"

#include "IOptimusCoreModule.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusNode_GraphTerminal.h"
#include "OptimusComponentSource.h"
#include "OptimusCoreModule.h"
#include "OptimusDeformer.h"
#include "OptimusObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_FunctionReference)


UOptimusFunctionNodeGraph* FOptimusFunctionGraphIdentifier::Resolve()
{
	if (Asset)
	{
		return Asset->FindFunctionByGuid(Guid);
	}

	return nullptr;
}

FName UOptimusNode_FunctionReference::GetNodeCategory() const
{
	if (ResolvedFunctionGraph.IsValid())
	{
		return ResolvedFunctionGraph->Category;
	}

	return NAME_None;
}

FText UOptimusNode_FunctionReference::GetDisplayName() const
{
	if (ResolvedFunctionGraph.IsValid())
	{
		return FText::FromString(ResolvedFunctionGraph->GetNodeName());
	}
	
	return FText::FromString("<graph missing>");
}


void UOptimusNode_FunctionReference::ConstructNode()
{
	if (ResolvedFunctionGraph.IsValid())
	{
		const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
		FOptimusDataTypeRef ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());
		DefaultComponentPin = AddPinDirect(UOptimusNodeSubGraph::GraphDefaultComponentPinName, EOptimusNodePinDirection::Input, {}, ComponentSourceType);
		
		// After a duplicate, the kernel node has no pins, so we need to reconstruct them from
		// the bindings. We can assume that all naming clashes have already been dealt with.
		for (const FOptimusParameterBinding& Binding: ResolvedFunctionGraph->InputBindings)
		{
			AddPinDirect(Binding, EOptimusNodePinDirection::Input);
		}
		for (const FOptimusParameterBinding& Binding: ResolvedFunctionGraph->OutputBindings)
		{
			AddPinDirect(Binding, EOptimusNodePinDirection::Output);
		}
	}
}


FOptimusRoutedNodePin UOptimusNode_FunctionReference::GetPinCounterpart(
	UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InTraversalContext
) const
{
	if (!InNodePin || InNodePin->GetOwningNode() != this)
	{
		return {};
	}

	if (!ResolvedFunctionGraph.IsValid())
	{
		return {};
	}

	UOptimusNode_GraphTerminal* CounterpartNode = nullptr;
	if (InNodePin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		CounterpartNode = ResolvedFunctionGraph->GetTerminalNode(EOptimusTerminalType::Entry);
	}
	else if (InNodePin->GetDirection() == EOptimusNodePinDirection::Output)
	{
		CounterpartNode = ResolvedFunctionGraph->GetTerminalNode(EOptimusTerminalType::Return);
	}

	if (!ensure(CounterpartNode))
	{
		return {};
	}

	FOptimusRoutedNodePin Result{
		CounterpartNode->FindPinFromPath(InNodePin->GetPinNamePath()),
		InTraversalContext
	};
	Result.TraversalContext.ReferenceNesting.Push(this);

	return Result;
}

UOptimusNodeGraph* UOptimusNode_FunctionReference::GetNodeGraphToShow()
{
	return ResolvedFunctionGraph.Get();
}

UOptimusNodeSubGraph* UOptimusNode_FunctionReference::GetReferencedSubGraph() const
{
	return ResolvedFunctionGraph.Get();
}

UOptimusComponentSourceBinding* UOptimusNode_FunctionReference::GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const
{
	if (!ensure(DefaultComponentPin.IsValid()))
	{
		return nullptr;
	}
	
	const UOptimusNodeGraph* OwningGraph = GetOwningGraph();
	TSet<UOptimusComponentSourceBinding*> Bindings = OwningGraph->GetComponentSourceBindingsForPin(DefaultComponentPin.Get(), InTraversalContext);
	
	if (!Bindings.IsEmpty() && ensure(Bindings.Num() == 1))
	{
		return Bindings.Array()[0];
	}

	// Default to the primary binding, but only if we're at the top-most level of the graph.
	if (const UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(OwningGraph->GetCollectionOwner()))
	{
		return Deformer->GetPrimaryComponentBinding();
	}

	if (const UOptimusNodeSubGraph* OwningSubGraph = Cast<UOptimusNodeSubGraph>(OwningGraph))
	{
		return OwningSubGraph->GetDefaultComponentBinding(InTraversalContext);
	}

	return nullptr;		
}

UOptimusNodePin* UOptimusNode_FunctionReference::GetDefaultComponentBindingPin() const
{
	return DefaultComponentPin.Get();
}

void UOptimusNode_FunctionReference::SetReferencedFunctionGraph(const FOptimusFunctionGraphIdentifier& InGraphIdentifier)
{
	FunctionGraphIdentifier = InGraphIdentifier;
}

const FOptimusFunctionGraphIdentifier& UOptimusNode_FunctionReference::GetReferencedFunctionGraphIdentifier()
{
	return FunctionGraphIdentifier;
}

void UOptimusNode_FunctionReference::UpdateDisplayName()
{
	SetDisplayName(GetDisplayName());
}

void UOptimusNode_FunctionReference::PostLoadNodeSpecificData()
{
	Super::PostLoadNodeSpecificData();

	FOptimusObjectVersion::Type ObjectVersion = static_cast<FOptimusObjectVersion::Type>(GetLinkerCustomVersion(FOptimusObjectVersion::GUID));
	if (ObjectVersion < FOptimusObjectVersion::FunctionGraphUseGuid)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!FunctionGraph_DEPRECATED.IsNull())
		{
			// Unfortnately we need to do a sync load here in post load to convert soft reference to hard reference.
			// The load is not guaranteed to reach RF_LoadCompleted, hence the name "PartiallyLoadedGraph".
			if (UOptimusFunctionNodeGraph* PartiallyLoadedGraph = FunctionGraph_DEPRECATED.LoadSynchronous())
			{
				FunctionGraphIdentifier.Asset = PartiallyLoadedGraph->GetTypedOuter<UOptimusDeformer>();
				FunctionGraphIdentifier.Guid = UOptimusFunctionNodeGraph::GetGuidForGraphWithoutGuid(FunctionGraph_DEPRECATED);
				
				ResolvedFunctionGraph = PartiallyLoadedGraph;
				
				FunctionGraph_DEPRECATED.Reset();

				// Suggest a resave to avoid going down this path all the time
				UE_LOGF(LogOptimusCore, Warning ,"Deformer Graph %ls should be resaved to improve loading performance", *GetPackage()->GetName());
				Modify();
			}
			
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UOptimusNode_FunctionReference::InitializeTransientData()
{
	Super::InitializeTransientData();
	if (!ResolvedFunctionGraph.IsValid())
	{
		ResolvedFunctionGraph = FunctionGraphIdentifier.Resolve();
	}
	
	UpdateDisplayName();
}