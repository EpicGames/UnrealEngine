// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IOptimusNodeGraphProvider.h"
#include "IOptimusNodePinRouter.h"
#include "IOptimusNodeSubGraphReferencer.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusFunctionNodeGraphHeader.h"
#include "OptimusNode.h"

#include "OptimusNode_FunctionReference.generated.h"



UCLASS(Hidden)
class UOptimusNode_FunctionReference :
	public UOptimusNode,
	public IOptimusNodePinRouter,
	public IOptimusNodeGraphProvider,
	public IOptimusNodeSubGraphReferencer
{
	GENERATED_BODY()

public:
	// UOptimusNode overrides
	FName GetNodeCategory() const override;
	FText GetDisplayName() const override;
	void ConstructNode() override;

	// IOptimusNodePinRouter implementation
	FOptimusRoutedNodePin GetPinCounterpart(
		UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InTraversalContext
	) const override;

	// IOptimusNodeGraphProvider
	UOptimusNodeGraph* GetNodeGraphToShow() override;
	
	// IOptimusNodeSubGraphReferencer
	UOptimusNodeSubGraph* GetReferencedSubGraph() const override;
	UOptimusComponentSourceBinding* GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const override;
	UOptimusNodePin* GetDefaultComponentBindingPin() const override;

	void SetReferencedFunctionGraph(const FOptimusFunctionGraphIdentifier& InGraphIdentifier);
	const FOptimusFunctionGraphIdentifier& GetReferencedFunctionGraphIdentifier();
	void UpdateDisplayName();
	
protected:
	void PostLoadNodeSpecificData() override;
	void InitializeTransientData() override;
	
	UPROPERTY()
	FOptimusFunctionGraphIdentifier FunctionGraphIdentifier;

	UPROPERTY()
	TWeakObjectPtr<UOptimusNodePin> DefaultComponentPin;

	UPROPERTY(Transient, VisibleAnywhere, Category=FunctionReference, DisplayName = "Referenced Function Graph")
	TWeakObjectPtr<UOptimusFunctionNodeGraph> ResolvedFunctionGraph;
	
private:
	UE_DEPRECATED(5.6, "Use FunctionGraphIdentifier instead.")
	UPROPERTY(Meta = (DeprecatedProperty))
	TSoftObjectPtr<UOptimusFunctionNodeGraph> FunctionGraph_DEPRECATED;

	bool bDelayResolvingGraph = false;
};
