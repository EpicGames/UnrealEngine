// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_GetResource.h"

#include "OptimusResourceDescription.h"
#include "OptimusNodePin.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_GetResource)


int32 UOptimusNode_GetResource::GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const
{
	if (!InPin || InPin->GetParentPin() != nullptr)
	{
		return INDEX_NONE;
	}
	if (!ensure(GetPins().Contains(InPin)))
	{
		return INDEX_NONE;
	}

	return UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType::Default);
}

void UOptimusNode_GetResource::ConstructNode()
{
	if (const UOptimusResourceDescription *Res = GetResourceDescription())
	{
		// FIXME: Define context.
		AddPinDirect(
			GetResourcePinName(0),
			EOptimusNodePinDirection::Output,
			Res->DataDomain,
			Res->DataType);
	}
}
