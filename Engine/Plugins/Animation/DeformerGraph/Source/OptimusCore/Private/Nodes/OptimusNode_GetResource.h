// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Nodes/OptimusNode_ResourceAccessorBase.h"

#include "OptimusNode_GetResource.generated.h"


class UOptimusResourceDescription;


UCLASS(Hidden)
class UOptimusNode_GetResource
	: public UOptimusNode_ResourceAccessorBase
{
	GENERATED_BODY()

	// IOptimusDataInterfaceProvider implementations
	int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const override;

protected:
	void ConstructNode() override;
};
