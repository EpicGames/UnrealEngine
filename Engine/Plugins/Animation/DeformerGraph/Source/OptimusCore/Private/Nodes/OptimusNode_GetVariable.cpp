// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_GetVariable.h"

#include "OptimusCoreModule.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusNodePin.h"
#include "OptimusValueContainer.h"
#include "OptimusVariableDescription.h"
#include "Actions/OptimusAction.h"
#include "Actions/OptimusVariableActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_GetVariable)

#define LOCTEXT_NAMESPACE "OptimusGetVariable"


void UOptimusNode_GetVariable::PreDuplicateRequirementActions(
	const UOptimusNodeGraph* InTargetGraph,
	FOptimusCompoundAction* InCompoundAction
	)
{
	if (DuplicationInfo.VariableName.IsNone())
	{
		return;
	}

	// Check if the deformer has a variable that matches the name and type.
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InTargetGraph->GetCollectionRoot());
	
	for (const UOptimusVariableDescription* ExistingVariableDesc: Deformer->GetVariables())
	{
		if (ExistingVariableDesc->VariableName == DuplicationInfo.VariableName &&
			ExistingVariableDesc->DataType == DuplicationInfo.DataType)
		{
			// Note: We don't care about the default value in this context.
			return;
		}
	}

	// Note: This will fail if there are multiple overlapping names. A better system is needed.
	bool bFoundDuplicate = false;
	do
	{
		bFoundDuplicate = false;
		for (const UOptimusVariableDescription* ExistingVariableDesc: Deformer->GetVariables())
		{
			if (ExistingVariableDesc->VariableName == DuplicationInfo.VariableName)
			{
				DuplicationInfo.VariableName.SetNumber(DuplicationInfo.VariableName.GetNumber() + 1);
				bFoundDuplicate = true;
			}
		}
	}
	while(bFoundDuplicate);

	InCompoundAction->AddSubAction<FOptimusVariableAction_AddVariable>(DuplicationInfo.DataType, DuplicationInfo.VariableName);
}


void UOptimusNode_GetVariable::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (!GetOwningGraph())
	{
		return;
	}
		
	const UOptimusDeformer* NewDescOwner = Cast<UOptimusDeformer>(GetOwningGraph()->GetCollectionRoot());
	if (!NewDescOwner)
	{
		return;
	}

	if (VariableDesc.IsValid())
	{
		const UOptimusDeformer* OldDescOwner = VariableDesc->GetOwningDeformer();
		
		// No action needed if we are copying/pasting within the same deformer asset 
		if (OldDescOwner == NewDescOwner)
		{
			return;
		}
		
		// Refresh the variable description so that we don't hold a reference to a VariableDesc in another deformer asset
		VariableDesc = NewDescOwner->ResolveVariable(VariableDesc->GetFName());
	}
	else if (!DuplicationInfo.VariableName.IsNone())
	{
		// Resolve the variable description from the duplication information that likely came from a copy/paste operation
		// where the pointer didn't survive and the variable didn't exist and we had to set it up in PreDuplicateRequirementActions
		VariableDesc = NewDescOwner->ResolveVariable(DuplicationInfo.VariableName);

		if (VariableDesc.IsValid())
		{
			// Re-create the pins. Don't call PostCreateNode, since it won't notify the graph that the pin layout has
			// changed.
			check(GetPins().IsEmpty());
			ConstructNode();
		}

		// Empty the duplication info before we carry on.
		DuplicationInfo = FOptimusNode_GetVariable_DuplicationInfo();
	}

}


void UOptimusNode_GetVariable::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	if (const UOptimusVariableDescription* Var = VariableDesc.Get())
	{
		Out.Logf(TEXT("%sCustomProperties VariableDefinition Name=\"%s\" Type=%s"),
			FCString::Spc(Indent), *Var->VariableName.ToString(), *Var->DataType->TypeName.ToString());

		if (Var->DefaultValueStruct.IsInitialized())
		{
			FString ValueStr = Var->DefaultValueStruct.GetValueAsString();
			Out.Logf(TEXT(" DefaultValue=\"%s\""), *ValueStr.ReplaceCharWithEscapedChar());
		}
		Out.Logf(TEXT("\n"));
	}
}


void UOptimusNode_GetVariable::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	if (FParse::Command(&SourceText, TEXT("VariableDefinition")))
	{
		FName VariableName;
		if (!FParse::Value(SourceText, TEXT("Name="), VariableName))
		{
			return;
		}

		FName DataTypeName;
		if (!FParse::Value(SourceText, TEXT("Type="), DataTypeName))
		{
			return;
		}

		const FOptimusDataTypeRef DataType = FOptimusDataTypeRegistry::Get().FindType(DataTypeName);
		if (!DataType.IsValid())
		{
			return;
		}

		DuplicationInfo.VariableName = VariableName;
		DuplicationInfo.DataType = DataType;

		if (FParse::Value(SourceText, TEXT("DefaultValue="), DuplicationInfo.DefaultValue))
		{
			// Unescape control characters.
			DuplicationInfo.DefaultValue.ReplaceEscapedCharWithCharInline();
		}
	}
}


void UOptimusNode_GetVariable::SetVariableDescription(UOptimusVariableDescription* InVariableDesc)
{
	if (!ensure(InVariableDesc))
	{
		return;
	}

	if (!EnumHasAnyFlags(InVariableDesc->DataType->UsageFlags, EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::Property))
	{
		UE_LOGF(LogOptimusCore, Error, "Data type '%ls' is not usable in a variable",
		    *InVariableDesc->DataType->TypeName.ToString());
		return;
	}

	VariableDesc = InVariableDesc;
}


UOptimusVariableDescription* UOptimusNode_GetVariable::GetVariableDescription() const
{
	return VariableDesc.Get();
}


TOptional<FText> UOptimusNode_GetVariable::ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
{
	const UOptimusVariableDescription* VariableDescription = GetVariableDescription();
	if (!VariableDescription)
	{
		return LOCTEXT("NoDescriptor", "No variable descriptor set on this node");
	}

	return {};
}

FOptimusValueIdentifier UOptimusNode_GetVariable::GetValueIdentifier() const
{
	if (const UOptimusVariableDescription* Var = VariableDesc.Get())
	{
		return {EOptimusValueType::Variable, Var->VariableName};
	}

	return {};
}


FOptimusDataTypeRef UOptimusNode_GetVariable::GetValueDataType() const
{
	if (const UOptimusVariableDescription* Var = VariableDesc.Get())
	{
		return Var->DataType;
	}

	return {};
}

FOptimusValueContainerStruct UOptimusNode_GetVariable::GetValue() const
{
	if (const UOptimusVariableDescription* Var = VariableDesc.Get())
	{
		return Var->DefaultValueStruct;
	}

	return {};
}


void UOptimusNode_GetVariable::ConstructNode()
{
	if (const UOptimusVariableDescription *Var = VariableDesc.Get())
	{
		AddPinDirect(
			Var->VariableName, 
			EOptimusNodePinDirection::Output,
			{},
			Var->DataType);
	}
}


#undef LOCTEXT_NAMESPACE
