// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusNonCollapsibleNode.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "IOptimusValueProvider.h"
#include "OptimusNode.h"

#include "OptimusNode_GetVariable.generated.h"


class UOptimusVariableDescription;


// Information to hold data on the variable definition that persists over node duplication.
// FIXME: This could be generalized in a better fashion.
USTRUCT()
struct FOptimusNode_GetVariable_DuplicationInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName VariableName;

	UPROPERTY()
	FOptimusDataTypeRef DataType;

	UPROPERTY()
	FString DefaultValue;
};


UCLASS(Hidden)
class UOptimusNode_GetVariable : 
	public UOptimusNode,
	public IOptimusValueProvider,
	public IOptimusPinMutabilityDefiner,
	public IOptimusNonCollapsibleNode
{
	GENERATED_BODY()

public:
	void SetVariableDescription(UOptimusVariableDescription* InVariableDesc);

	UOptimusVariableDescription* GetVariableDescription() const;
	
	// UOptimusNode overrides
	FName GetNodeCategory() const override 
	{
		return CategoryName::Variables;
	}

	TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const override;
	
	// IOptimusValueProvider overrides 
	FOptimusValueIdentifier GetValueIdentifier() const override;
	FOptimusDataTypeRef GetValueDataType() const override;
	FOptimusValueContainerStruct GetValue() const override;

	//IOptimusPinMutabilityDefiner overrides 
	EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const override { return EOptimusPinMutability::Mutable; };
	
protected:
	void ConstructNode() override;

	void PreDuplicateRequirementActions(const UOptimusNodeGraph* InTargetGraph, FOptimusCompoundAction* InCompoundAction) override;

	// UObject overrides
	void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	
private:
	UPROPERTY()
	TWeakObjectPtr<UOptimusVariableDescription> VariableDesc;
	
	// Duplication data across graphs
	UPROPERTY(DuplicateTransient)
	FOptimusNode_GetVariable_DuplicationInfo DuplicationInfo;
};
