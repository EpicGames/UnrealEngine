// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_GraphTerminal.h"

#include "OptimusFunctionNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusBindingTypes.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "OptimusNode_ComputeKernelBase.h"
#include "OptimusNode_SubGraphReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_GraphTerminal)


#define LOCTEXT_NAMESPACE "OptimusNodeGraphTerminal"

FName UOptimusNode_GraphTerminal::EntryNodeName = TEXT("Entry");
FName UOptimusNode_GraphTerminal::ReturnNodeName= TEXT("Return");

UOptimusNode_GraphTerminal::UOptimusNode_GraphTerminal()
{
	EnableDynamicPins();
}

FText UOptimusNode_GraphTerminal::GetDisplayName() const
{
	switch(TerminalType)
	{
	case EOptimusTerminalType::Entry:
		return LOCTEXT("TerminalType_Entry", "Entry"); 
	case EOptimusTerminalType::Return:
		return LOCTEXT("TerminalType_Return", "Return");
	case EOptimusTerminalType::Unknown:
		checkNoEntry();
	}
	return FText();
}


void UOptimusNode_GraphTerminal::ConstructNode()
{
	UOptimusNodeSubGraph* Graph = OwningGraph.Get();
	if (!ensure(Graph))
	{
		return;
	}

	switch(TerminalType)
	{
	case EOptimusTerminalType::Entry:
		{
			const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
			FOptimusDataTypeRef ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());

			DefaultComponentPin = AddPinDirect(UOptimusNodeSubGraph::GraphDefaultComponentPinName, EOptimusNodePinDirection::Output, {}, ComponentSourceType);
			
			// The input bindings represent the _inputs_ to the graph, so when they are accessed
			// from within the graph, they get added as _outputs_ to the entry node.
			for (const FOptimusParameterBinding& Binding: Graph->InputBindings)
			{
				AddPinDirect(Binding, EOptimusNodePinDirection::Output);
			}
		}
		break;
	case EOptimusTerminalType::Return:
		// And vice-versa.
		{
			for (const FOptimusParameterBinding& Binding: Graph->OutputBindings)
			{
				AddPinDirect(Binding, EOptimusNodePinDirection::Input);
			}
		}
		break;
	case EOptimusTerminalType::Unknown:
		checkNoEntry();
	}
}



void UOptimusNode_GraphTerminal::BeginDestroy()
{
	Super::BeginDestroy();

	UnsubscribeFromOwningGraph();
}


FOptimusRoutedNodePin UOptimusNode_GraphTerminal::GetPinCounterpart(
	UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InTraversalContext
) const
{
	if (!InNodePin || InNodePin->GetOwningNode() != this)
	{
		return {};
	}
	
	if (InTraversalContext.ReferenceNesting.IsEmpty())
	{
		return {};
	}

	const UOptimusNode* CounterpartNode = Cast<const UOptimusNode>(InTraversalContext.ReferenceNesting.Last());
	if (!ensure(CounterpartNode))
	{
		return {};
	}

	FOptimusRoutedNodePin Result{
		CounterpartNode->FindPinFromPath(InNodePin->GetPinNamePath()),
		InTraversalContext
	};
	Result.TraversalContext.ReferenceNesting.Pop();
	
	return Result;
}

UObject* UOptimusNode_GraphTerminal::GetObjectToShowWhenSelected() const
{
	return OwningGraph.Get();
}

UOptimusComponentSourceBinding* UOptimusNode_GraphTerminal::GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const
{
	if (!ensure(DefaultComponentPin.IsValid()))
	{
		return nullptr;
	}
	
	const FOptimusRoutedNodePin PinCounterpart = GetPinCounterpart(DefaultComponentPin.Get(), InTraversalContext);

	if (!PinCounterpart.NodePin)
	{
		return nullptr;
	}
	
	const IOptimusNodeSubGraphReferencer* ReferenceNode = CastChecked<IOptimusNodeSubGraphReferencer>(PinCounterpart.NodePin->GetOwningNode());

	return ReferenceNode->GetDefaultComponentBinding(PinCounterpart.TraversalContext);
}

void UOptimusNode_GraphTerminal::InitializeTransientData()
{
	OwningGraph = Cast<UOptimusNodeSubGraph>(GetOwningGraph());
	SubscribeToOwningGraph();
}

void UOptimusNode_GraphTerminal::SubscribeToOwningGraph()
{
	if (ensure(OwningGraph.IsValid()) && ensure(!OwningGraph->GetOnBindingArrayPasted().IsBoundToObject(this)))
	{
		OwningGraph->GetOnBindingArrayPasted().AddUObject(this, &UOptimusNode_GraphTerminal::RecreateBindingPins);
		OwningGraph->GetOnBindingValueChanged().AddUObject(this, &UOptimusNode_GraphTerminal::SyncPinsToBindings);
		OwningGraph->GetOnBindingArrayItemAdded().AddUObject(this, &UOptimusNode_GraphTerminal::AddPinForNewBinding);
		OwningGraph->GetOnBindingArrayItemRemoved().AddUObject(this, &UOptimusNode_GraphTerminal::RemoveStalePins);
		OwningGraph->GetOnBindingArrayCleared().AddUObject(this, &UOptimusNode_GraphTerminal::RemoveStalePins);
		OwningGraph->GetOnBindingArrayItemMoved().AddUObject(this, &UOptimusNode_GraphTerminal::OnBindingMoved);
	}
	
}

void UOptimusNode_GraphTerminal::UnsubscribeFromOwningGraph() const
{
	// Technically we don't need to unsubscribe since the life time of the terminal is the same as that of the owning graph
	// but we still do it for good measure
	if (OwningGraph.IsValid())
	{
		OwningGraph->GetOnBindingArrayPasted().RemoveAll(this);
		OwningGraph->GetOnBindingValueChanged().RemoveAll(this);
		OwningGraph->GetOnBindingArrayItemAdded().RemoveAll(this);
		OwningGraph->GetOnBindingArrayItemRemoved().RemoveAll(this);
		OwningGraph->GetOnBindingArrayCleared().RemoveAll(this);
		OwningGraph->GetOnBindingArrayItemMoved().RemoveAll(this);
	}
}

TArray<UOptimusNodePin*> UOptimusNode_GraphTerminal::GetBindingPins()
{
	if (TerminalType == EOptimusTerminalType::Entry)
	{
		TArray<UOptimusNodePin*> BindingPins = GetPinsByDirection(EOptimusNodePinDirection::Output, false);

		BindingPins.RemoveAt(0);

		return BindingPins;
	}
	else if (TerminalType == EOptimusTerminalType::Return)
	{
		return GetPinsByDirection(EOptimusNodePinDirection::Input, false);
	}

	return {};
}

void UOptimusNode_GraphTerminal::AddPinForNewBinding(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	EOptimusNodePinDirection NewPinDirection = EOptimusNodePinDirection::Unknown;
	
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName && TerminalType == EOptimusTerminalType::Entry)
	{
		BindingArrayPtr = &OwningGraph->InputBindings;
		NewPinDirection = EOptimusNodePinDirection::Output;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName && TerminalType == EOptimusTerminalType::Return)
	{
		BindingArrayPtr = &OwningGraph->OutputBindings;
		NewPinDirection = EOptimusNodePinDirection::Input;
	}

	if (BindingArrayPtr)
	{
		check(BindingArrayPtr->Num() > 0);
		AddPin(BindingArrayPtr->Last(), NewPinDirection, nullptr);
	}
}

void UOptimusNode_GraphTerminal::RemoveStalePins(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	TArray<UOptimusNodePin*> PinsToCheck = GetBindingPins();
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName && TerminalType == EOptimusTerminalType::Entry)
	{
		BindingArrayPtr = &OwningGraph->InputBindings;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName && TerminalType == EOptimusTerminalType::Return)
	{
		BindingArrayPtr = &OwningGraph->OutputBindings;
	}

	if (BindingArrayPtr)
	{
		for (UOptimusNodePin* Pin : PinsToCheck)
		{
			int32 Index = BindingArrayPtr->IndexOfByPredicate([ PinName = Pin->GetFName()](const FOptimusParameterBinding& InBinding){
				 return InBinding.Name == PinName;
			});

			if (Index == INDEX_NONE)
			{
				RemovePin(Pin);
			}
		}	
	}
}

void UOptimusNode_GraphTerminal::OnBindingMoved(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	TArray<UOptimusNodePin*> PinsToCheck = GetBindingPins();
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName && TerminalType == EOptimusTerminalType::Entry)
	{
		BindingArrayPtr = &OwningGraph->InputBindings;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName && TerminalType == EOptimusTerminalType::Return)
	{
		BindingArrayPtr = &OwningGraph->OutputBindings;
	}

	if (BindingArrayPtr)
	{
		TArray<FName> PinNames;

		for (UOptimusNodePin* Pin : PinsToCheck)
		{
			PinNames.Add(Pin->GetFName());
		}

		TArray<FName> BindingNames;
		for (const FOptimusParameterBinding& Binding : *BindingArrayPtr)
		{
			BindingNames.Add(Binding.Name);
		}

		FName PinName;
		FName NextPinName;
		if (Optimus::FindMovedItemInNameArray(PinNames, BindingNames, PinName, NextPinName))
		{
			const int32 PinIndex = PinsToCheck.IndexOfByPredicate([PinName](const UOptimusNodePin* InPin)
			{
				return InPin->GetFName() == PinName;
			});
		
			const int32 NextPinIndex = NextPinName == NAME_None ?
				INDEX_NONE : PinsToCheck.IndexOfByPredicate([NextPinName](const UOptimusNodePin* InPin)
			{
				return InPin->GetFName() == NextPinName;
			});

			MovePin(PinsToCheck[PinIndex], NextPinIndex != INDEX_NONE ? PinsToCheck[NextPinIndex] : nullptr);
		}
	}
}

void UOptimusNode_GraphTerminal::RecreateBindingPins(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	EOptimusNodePinDirection NewPinDirection = EOptimusNodePinDirection::Unknown;
	TArray<UOptimusNodePin*> PinsToRemove = GetBindingPins();
	
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName && TerminalType == EOptimusTerminalType::Entry)
	{
		BindingArrayPtr = &OwningGraph->InputBindings;
		NewPinDirection = EOptimusNodePinDirection::Output;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName && TerminalType == EOptimusTerminalType::Return)
	{
		BindingArrayPtr = &OwningGraph->OutputBindings;
		NewPinDirection = EOptimusNodePinDirection::Input;
	}

	if (BindingArrayPtr)
	{
		for (UOptimusNodePin* Pin : PinsToRemove)
		{
			RemovePin(Pin);
		}

		for (const FOptimusParameterBinding& Binding : *BindingArrayPtr)
		{
			AddPin(Binding, NewPinDirection, nullptr);
		}
	}	
}

void UOptimusNode_GraphTerminal::SyncPinsToBindings(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;

	TArray<UOptimusNodePin*> BindingPins = GetBindingPins();
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName && TerminalType == EOptimusTerminalType::Entry)
	{
		BindingArrayPtr = &OwningGraph->InputBindings;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName && TerminalType == EOptimusTerminalType::Return)
	{
		BindingArrayPtr = &OwningGraph->OutputBindings;
	}

	if (BindingArrayPtr)
	{
		check(BindingArrayPtr->Num() == BindingPins.Num());
		for (int32 Index = 0 ; Index < BindingArrayPtr->Num(); Index++)
		{
			UOptimusNodePin* BindingPin = BindingPins[Index];
			const FOptimusParameterBinding& Binding = (*BindingArrayPtr)[Index];
			if (BindingPin->GetFName() != Binding.Name)
			{
				SetPinName(BindingPin, Binding.Name);
			}
			if (BindingPin->GetDataType() != Binding.DataType.Resolve())
			{
				SetPinDataType(BindingPin, Binding.DataType);
			}
			if (BindingPin->GetDataDomain() != Binding.DataDomain)
			{
				SetPinDataDomain(BindingPin, Binding.DataDomain);
			}
		}
	}	
}


#undef LOCTEXT_NAMESPACE
