// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IOptimusAlternativeSelectedObjectProvider.h"
#include "IOptimusNodePinRouter.h"
#include "IOptimusNonCollapsibleNode.h"
#include "IOptimusNonCopyableNode.h"
#include "OptimusNode.h"

#include "OptimusNode_GraphTerminal.generated.h"


class UOptimusNodeSubGraph;


UENUM()
enum class EOptimusTerminalType
{
	Unknown,
	Entry,
	Return
};


UCLASS(Hidden)
class UOptimusNode_GraphTerminal :
	public UOptimusNode,
	public IOptimusNodePinRouter,
	public IOptimusAlternativeSelectedObjectProvider,
	public IOptimusNonCollapsibleNode,
	public IOptimusNonCopyableNode
{
	GENERATED_BODY()
	
public:
	static FName EntryNodeName;
	static FName ReturnNodeName;
	UOptimusNode_GraphTerminal();
	
	// UOptimusNode overrides
	bool CanUserDeleteNode() const override { return false; }
	FName GetNodeCategory() const override { return NAME_None; }
	FText GetDisplayName() const override;
	void ConstructNode() override;

	// UObject overrides
	void BeginDestroy() override;
	
	// IOptimusNodePinRouter implementation
	FOptimusRoutedNodePin GetPinCounterpart(
		UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InTraversalContext
	) const override;
	
	// IOptimusAlternativeSelectedObjectProvider
	UObject* GetObjectToShowWhenSelected() const override;

	UOptimusComponentSourceBinding* GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const;

protected:
	friend class UOptimusNodeGraph;
	friend class UOptimusNodeSubGraph;

	void InitializeTransientData() override;
	void SubscribeToOwningGraph();

	void UnsubscribeFromOwningGraph() const;
	TArray<UOptimusNodePin*> GetBindingPins();
	
	void AddPinForNewBinding(FName InBindingArrayPropertyName);
	void RemoveStalePins(FName InBindingArrayPropertyName);
	void OnBindingMoved(FName InBindingArrayPropertyName);
	void RecreateBindingPins(FName InBindingArrayPropertyName);
	void SyncPinsToBindings(FName InBindingArrayPropertyName);	
	
	/** Indicates whether this is an entry or a return terminal node */
	UPROPERTY()
	EOptimusTerminalType TerminalType;


	UPROPERTY()
	TWeakObjectPtr<UOptimusNodePin> DefaultComponentPin;
	
	/** The graph that owns us. This contains all the necessary pin information to add on
	 * the terminal node. Initialized during InitializeTransientData()
	 */
	TWeakObjectPtr<UOptimusNodeSubGraph> OwningGraph;
};
