// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_LoopTerminal.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "OptimusNodeGraph.h"
#include "ComponentSources/OptimusSkeletalMeshComponentSource.h"
#include "DataInterfaces/OptimusDataInterfaceLoopTerminal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_LoopTerminal)

#define LOCTEXT_NAMESPACE "OptimusNodeLoopTerminal"


static FName GetHiddenPinName(FName InBindingName)
{
	return *(TEXT("(Hidden)") + InBindingName.ToString());
}

UOptimusNode_LoopTerminal::UOptimusNode_LoopTerminal()
{
	EnableDynamicPins();
}
#if WITH_EDITOR
void UOptimusNode_LoopTerminal::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
	{
		if (CastField<FArrayProperty>(PropertyChangedEvent.Property) &&
			PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.GetPropertyName().ToString()) == INDEX_NONE)
		{
			PropertyArrayPasted(PropertyChangedEvent);
		}
		else
		{
			PropertyValueChanged(PropertyChangedEvent);
		}
		
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayAdd)
	{
		PropertyArrayItemAdded(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayRemove)
	{
		PropertyArrayItemRemoved(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayClear)
	{
		PropertyArrayCleared(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayMove)
	{
		PropertyArrayItemMoved(PropertyChangedEvent);
	}

}
#endif


FText UOptimusNode_LoopTerminal::GetDisplayName() const
{
	switch(TerminalType)
	{
	case EOptimusTerminalType::Entry:
		return LOCTEXT("LoopTerminalType_Entry", "Loop Entry"); 
	case EOptimusTerminalType::Return:
		return LOCTEXT("LoopTerminalType_Return", "Loop Return");
	case EOptimusTerminalType::Unknown:
		checkNoEntry();
	}
	return FText();
}


void UOptimusNode_LoopTerminal::ConstructNode()
{
	check(GetPins().IsEmpty());
	
	if (TerminalType == EOptimusTerminalType::Entry)
	{
		FOptimusDataTypeHandle UintType = FOptimusDataTypeRegistry::Get().FindType(*FUInt32Property::StaticClass());
		IndexPin = AddPinDirect(TEXT("Index"),  EOptimusNodePinDirection::Output, FOptimusDataDomain(), UintType);
		CountPin = AddPinDirect(TEXT("Count"),  EOptimusNodePinDirection::Output, FOptimusDataDomain(), UintType);
	}

	PinPairInfos.Reset();
}

bool UOptimusNode_LoopTerminal::ValidateConnection(const UOptimusNodePin& InThisNodesPin, const UOptimusNodePin& InOtherNodesPin, FString* OutReason) const
{
	if (InThisNodesPin.GetDirection() == EOptimusNodePinDirection::Input)
	{
		if (TerminalType == EOptimusTerminalType::Entry)
		{
			if (InOtherNodesPin.GetOwningNode() == GetOtherTerminal())
			{
				if (OutReason)
				{
					*OutReason = TEXT("Cannot connect output of return terminal to input of entry terminal");
				}
				return false;
			}
		}
	}

	return true;
}

TArray<IOptimusNodeAdderPinProvider::FAdderPinAction> UOptimusNode_LoopTerminal::GetAvailableAdderPinActions(
	const UOptimusNodePin* InSourcePin, EOptimusNodePinDirection InNewPinDirection, FString* OutReason) const
{
	if (InSourcePin->GetDataDomain().IsSingleton())
	{
		if (OutReason)
		{
			*OutReason = TEXT("Can't add parameter pin ");
		}

		return {};
	}
	
	FAdderPinAction Action;
	Action.bCanAutoLink = InSourcePin->GetDirection() != InNewPinDirection;
	Action.NewPinDirection = InNewPinDirection;
	return {Action};
}

TArray<UOptimusNodePin*> UOptimusNode_LoopTerminal::TryAddPinFromPin(const FAdderPinAction& InSelectedAction, UOptimusNodePin* InSourcePin, FName InNameToUse)
{
	FOptimusParameterBinding Binding;
	Binding.Name = InNameToUse;
	Binding.DataType = {InSourcePin->GetDataType()};
	Binding.DataDomain = InSourcePin->GetDataDomain();	

	SanitizeBinding(Binding, NAME_None);

	check(GetLoopInfo());
	GetLoopInfo()->Bindings.InnerArray.Add(Binding);

	TArray<UOptimusNodePin*> AddedPins = GetTerminalByType(EOptimusTerminalType::Entry)->AddPinPairsDirect(Binding);	
	AddedPins.Append(GetTerminalByType(EOptimusTerminalType::Return)->AddPinPairsDirect(Binding));	

	// The last pin should be the one to auto link
	if (TerminalType == EOptimusTerminalType::Entry)
	{
		if (InSelectedAction.NewPinDirection == EOptimusNodePinDirection::Input)
		{
			AddedPins.Swap(0, 3);
		}
		else
		{
			AddedPins.Swap(1, 3);
		}
	}
	else
	{
		if (InSelectedAction.NewPinDirection == EOptimusNodePinDirection::Input)
		{
			AddedPins.Swap(2, 3);
		}
	}

	return AddedPins;
}

bool UOptimusNode_LoopTerminal::RemoveAddedPins(TConstArrayView<UOptimusNodePin*> InAddedPinsToRemove)
{
	if (!ensure(!InAddedPinsToRemove.IsEmpty()))
	{
		return false;
	}

	int32 PairIndex = GetPairIndex(InAddedPinsToRemove[0]);

	if (!ensure(PairIndex != INDEX_NONE))
	{
		return false;
	}

	GetTerminalByType(EOptimusTerminalType::Entry)->RemovePinPairDirect(PairIndex);
	GetTerminalByType(EOptimusTerminalType::Return)->RemovePinPairDirect(PairIndex);
	check(GetLoopInfo());
	GetLoopInfo()->Bindings.InnerArray.RemoveAt(PairIndex);;

	return true;
}

bool UOptimusNode_LoopTerminal::IsPinNameHidden(UOptimusNodePin* InPin) const
{
	if (!InPin->GetDataDomain().IsSingleton())
	{
		if (TerminalType == EOptimusTerminalType::Entry && InPin->GetDirection() == EOptimusNodePinDirection::Output)
		{
			return true;
		}
		
		if (TerminalType == EOptimusTerminalType::Return && InPin->GetDirection() == EOptimusNodePinDirection::Input)
		{
			return true;
		}
	}

	return false;
}

FName UOptimusNode_LoopTerminal::GetNameForAdderPin(UOptimusNodePin* InPin) const
{
	if (IsPinNameHidden(InPin))
	{
		return GetPinCounterpart(InPin, TerminalType)->GetFName();
	}

	return InPin->GetFName();
}

EOptimusPinMutability UOptimusNode_LoopTerminal::GetOutputPinMutability(const UOptimusNodePin* InPin) const
{
	if (InPin == IndexPin || InPin == CountPin)
	{
		return EOptimusPinMutability::Immutable;
	}

	return EOptimusPinMutability::Undefined;
}

void UOptimusNode_LoopTerminal::PairToCounterpartNode(const IOptimusNodePairProvider* NodePairProvider)
{
	if (FOptimusLoopTerminalInfo* LoopInfoPtr = GetLoopInfo(); ensure(LoopInfoPtr))
	{
		for (const FOptimusParameterBinding& Binding : LoopInfoPtr->Bindings)
		{
			AddPinPairsDirect(Binding);
		}
	}	
}

FString UOptimusNode_LoopTerminal::GetBindingDeclaration(FName BindingName) const
{
	return {};
}

bool UOptimusNode_LoopTerminal::GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const
{
	return false;
}

bool UOptimusNode_LoopTerminal::GetBindingSupportReadCheckBoxVisibility(FName BindingName) const
{
	return false;
}

EOptimusDataTypeUsageFlags UOptimusNode_LoopTerminal::GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const
{
	if (ensure(!InDataDomain.IsSingleton()))
	{
		return EOptimusDataTypeUsageFlags::Resource;
	}

	return EOptimusDataTypeUsageFlags::None;
}

UOptimusNodePin* UOptimusNode_LoopTerminal::GetPinCounterpart(const UOptimusNodePin* InNodePin, EOptimusTerminalType InTerminalType, TOptional<EOptimusNodePinDirection> InDirection) const
{
	const int32 PairIndex = GetPairIndex(InNodePin);

	const UOptimusNode_LoopTerminal* Terminal = GetTerminalByType(InTerminalType);
	TArray<UOptimusNodePin*> PairedPins = Terminal->GetPairedPins(Terminal->PinPairInfos[PairIndex]);

	EOptimusNodePinDirection CounterpartDirection =
		InNodePin->GetDirection() == EOptimusNodePinDirection::Input ?
		EOptimusNodePinDirection::Output : EOptimusNodePinDirection::Input;

	if (InDirection)
	{
		CounterpartDirection = *InDirection;
	}
	
	return CounterpartDirection == EOptimusNodePinDirection::Input ? PairedPins[0] : PairedPins[1];
}

UOptimusNode_LoopTerminal* UOptimusNode_LoopTerminal::GetOtherTerminal() const
{
	return Cast<UOptimusNode_LoopTerminal>(GetOwningGraph()->GetNodeCounterpart(this));
}

int32 UOptimusNode_LoopTerminal::GetLoopCount() const
{
	check(GetLoopInfo());
	return GetLoopInfo()->Count;
}

EOptimusTerminalType UOptimusNode_LoopTerminal::GetTerminalType() const
{
	return TerminalType;
}

int32 UOptimusNode_LoopTerminal::GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin)
{
	TArray<FOptimusCDIPinDefinition> PinDefinitions = UOptimusLoopTerminalDataInterface::StaticClass()->GetDefaultObject<UOptimusLoopTerminalDataInterface>()->GetPinDefinitions();
	int32 PinIndex = INDEX_NONE;
	for (int32 Index = 0 ; Index < PinDefinitions.Num(); ++Index)
	{
		if (InPin->GetUniqueName() == PinDefinitions[Index].PinName)
		{
			PinIndex = Index;
			break;
		}
	}
	
	if (!ensure(PinIndex != INDEX_NONE))
	{
		return INDEX_NONE;
	}

	return PinIndex;
}

TArray<UOptimusNodePin*> UOptimusNode_LoopTerminal::AddPinPairs(const FOptimusParameterBinding& InBinding)
{
	TArray<UOptimusNodePin*> AddedPins;

	UOptimusNodePin* OutputBeforePin = nullptr;
	if (IndexPin)
	{
		OutputBeforePin = IndexPin;
	}

	FName InputName = InBinding.Name;
	FName OutputName = GetHiddenPinName(InBinding.Name);

	if (TerminalType == EOptimusTerminalType::Return)
	{
		Swap(InputName, OutputName);
	}
	
	AddedPins.Add(AddPin(InputName, EOptimusNodePinDirection::Input, InBinding.DataDomain, InBinding.DataType, nullptr));
	AddedPins.Add(AddPin(OutputName, EOptimusNodePinDirection::Output, InBinding.DataDomain, InBinding.DataType, OutputBeforePin, nullptr ));

	PinPairInfos.Add({AddedPins[0]->GetPinNamePath(), AddedPins[1]->GetPinNamePath()});

	return AddedPins;
}

TArray<UOptimusNodePin*> UOptimusNode_LoopTerminal::AddPinPairsDirect(const FOptimusParameterBinding& InBinding)
{
	TArray<UOptimusNodePin*> AddedPins;

	UOptimusNodePin* OutputBeforePin = nullptr;
	if (IndexPin)
	{
		OutputBeforePin = IndexPin;
	}

	FName InputName = InBinding.Name;
	FName OutputName = GetHiddenPinName(InBinding.Name);

	if (TerminalType == EOptimusTerminalType::Return)
	{
		Swap(InputName, OutputName);
	}
	
	AddedPins.Add(AddPinDirect(InputName, EOptimusNodePinDirection::Input, InBinding.DataDomain, InBinding.DataType, nullptr));
	AddedPins.Add(AddPinDirect(OutputName, EOptimusNodePinDirection::Output, InBinding.DataDomain, InBinding.DataType, OutputBeforePin, nullptr ));

	PinPairInfos.Add({AddedPins[0]->GetPinNamePath(), AddedPins[1]->GetPinNamePath()});

	return AddedPins;
}


TArray<UOptimusNodePin*> UOptimusNode_LoopTerminal::GetPairedPins(const FOptimusPinPairInfo& InPair) const
{
	return {FindPinFromPath(InPair.InputPinPath), FindPinFromPath(InPair.OutputPinPath)};
}

int32 UOptimusNode_LoopTerminal::GetPairIndex(const UOptimusNodePin* Pin)
{
	auto PinPairFilter = [Pin](const FOptimusPinPairInfo& InPinPair)
	{
		if (InPinPair.InputPinPath == Pin->GetPinNamePath() || InPinPair.OutputPinPath == Pin->GetPinNamePath())
		{
			return true;
		}
		return false;
	};
	
	const int32 PairIndex = CastChecked<UOptimusNode_LoopTerminal>(Pin->GetOwningNode())->PinPairInfos.IndexOfByPredicate(PinPairFilter);

	return PairIndex;
}

void UOptimusNode_LoopTerminal::RemovePinPair(int32 InPairIndex)
{
	if (ensure(PinPairInfos.IsValidIndex(InPairIndex)))
	{
		FOptimusPinPairInfo PinPair = PinPairInfos[InPairIndex];

		for (UOptimusNodePin* Pin : GetPairedPins(PinPair))
		{
			RemovePin(Pin);
		}

		PinPairInfos.Remove(PinPair);
	}	
}

void UOptimusNode_LoopTerminal::RemovePinPairDirect(int32 InPairIndex)
{
	if (ensure(PinPairInfos.IsValidIndex(InPairIndex)))
	{
		FOptimusPinPairInfo PinPair = PinPairInfos[InPairIndex];

		for (UOptimusNodePin* Pin : GetPairedPins(PinPair))
		{
			RemovePinDirect(Pin);
		}

		PinPairInfos.Remove(PinPair);
	}	
}

void UOptimusNode_LoopTerminal::ClearPinPairs()
{
	for (const FOptimusPinPairInfo& Pair : PinPairInfos)
	{
		for (UOptimusNodePin* Pin : GetPairedPins(Pair))
		{
			RemovePin(Pin);
		}
	}

	PinPairInfos.Reset();	
}

void UOptimusNode_LoopTerminal::MovePinPair()
{

	FOptimusParameterBindingArray& Bindings = GetLoopInfo()->Bindings;

	TArray<FName> BindingNames;
	for (FOptimusParameterBinding& Binding : Bindings)
	{
		BindingNames.Add(Binding.Name);
	}

	check(PinPairInfos.Num() == Bindings.Num());
	TArray<TArray<UOptimusNodePin*>> PinPairs;
	
	for (const FOptimusPinPairInfo& PinPairInfo : PinPairInfos)
	{
		PinPairs.Add(GetPairedPins(PinPairInfo));
	}

	int32 NamedPinIndex = 0;
	if (TerminalType == EOptimusTerminalType::Return)
	{
		NamedPinIndex = 1;
	}

	TArray<FName> PinNames;
	
	for (int32 Index = 0; Index < PinPairs.Num(); Index++)
	{
		const UOptimusNodePin* Pin = PinPairs[Index][NamedPinIndex];
		PinNames.Add({Pin->GetFName()});
	}


	FName PinName = NAME_None;
	FName NextPinName = NAME_None;

	if (Optimus::FindMovedItemInNameArray(PinNames, BindingNames, PinName, NextPinName))
	{
		const int32 PinIndex = PinPairs.IndexOfByPredicate([PinName, NamedPinIndex](const TArray<UOptimusNodePin*>& InPinPair)
		{
			return InPinPair[NamedPinIndex]->GetFName() == PinName;
		});

		int32 NextPinIndex = INDEX_NONE;
		if (!NextPinName.IsNone())
		{
			NextPinIndex = PinPairs.IndexOfByPredicate([NextPinName, NamedPinIndex](const TArray<UOptimusNodePin*>& InPinPair)
			{
				return InPinPair[NamedPinIndex]->GetFName() == NextPinName;
			});
		}

		MovePin(PinPairs[PinIndex][0], NextPinIndex != INDEX_NONE ? PinPairs[NextPinIndex][0]: nullptr);
		MovePin(PinPairs[PinIndex][1], NextPinIndex != INDEX_NONE ? PinPairs[NextPinIndex][1]: ToRawPtr(IndexPin));

		FOptimusPinPairInfo Pair = PinPairInfos[PinIndex];

		FOptimusPinPairInfo PairBefore = NextPinIndex != INDEX_NONE ? PinPairInfos[NextPinIndex] : FOptimusPinPairInfo();

		PinPairInfos.RemoveAt(PinIndex);
				
		if (NextPinIndex != INDEX_NONE)
		{
			const int32 BeforeIndex = PinPairInfos.IndexOfByKey(PairBefore);
			PinPairInfos.Insert(Pair, BeforeIndex);
		}
		else
		{
			PinPairInfos.Add(Pair);
		}
	}
}

void UOptimusNode_LoopTerminal::UpdatePinPairs()
{
	// PostEditChangeProperty does not give array index for changes to inner properties of a struct in an array, hence the traversal here
	TArray<TArray<UOptimusNodePin*>> PinPairs;

	check(GetLoopInfo());
	const FOptimusParameterBindingArray& Bindings = GetLoopInfo()->Bindings;

	check(Bindings.Num() == PinPairInfos.Num())
	
	for (const FOptimusPinPairInfo& PinPairInfo : PinPairInfos)
	{
		PinPairs.Add(GetPairedPins(PinPairInfo));
	}

	int32 NamedPinIndex = 0;
	int32 UnnamedPinIndex = 1;
	if (TerminalType == EOptimusTerminalType::Return)
	{
		Swap(NamedPinIndex, UnnamedPinIndex);
	}
	
	
	for (int32 Index = 0 ; Index < Bindings.Num(); Index++)
	{
		if (Bindings[Index].Name != PinPairs[Index][NamedPinIndex]->GetFName())
		{
			SetPinName(PinPairs[Index][NamedPinIndex], Bindings[Index].Name);
			SetPinName(PinPairs[Index][UnnamedPinIndex], GetHiddenPinName(Bindings[Index].Name));
			PinPairInfos[Index] = {PinPairs[Index][0]->GetPinNamePath(), PinPairs[Index][1]->GetPinNamePath()};
		}
		
		if (Bindings[Index].DataType.Resolve() != PinPairs[Index][0]->GetDataType())
		{
			SetPinDataType(PinPairs[Index][0], Bindings[Index].DataType);
			SetPinDataType(PinPairs[Index][1], Bindings[Index].DataType);
		}

		if (Bindings[Index].DataDomain != PinPairs[Index][0]->GetDataDomain())
		{
			SetPinDataDomain(PinPairs[Index][0], Bindings[Index].DataDomain);
			SetPinDataDomain(PinPairs[Index][1], Bindings[Index].DataDomain);
		}
	}
}

FOptimusLoopTerminalInfo* UOptimusNode_LoopTerminal::GetLoopInfo()
{
	if (TerminalType == EOptimusTerminalType::Entry)
	{
		return &LoopInfo;
	}
	
	if (UOptimusNode_LoopTerminal* Entry = GetOtherTerminal())
	{
		return &Entry->LoopInfo;
	}

	return nullptr;
}

const FOptimusLoopTerminalInfo* UOptimusNode_LoopTerminal::GetLoopInfo() const
{
	if (TerminalType == EOptimusTerminalType::Entry)
	{
		return &LoopInfo;
	}
	
	if (UOptimusNode_LoopTerminal* Entry = GetOtherTerminal())
	{
		return &Entry->LoopInfo;
	}

	return nullptr;
}

void UOptimusNode_LoopTerminal::SanitizeBinding(FOptimusParameterBinding& InOutBinding, FName InOldName)
{
	InOutBinding.Name = GetSanitizedBindingName(InOutBinding.Name, InOldName);
	
	if (!InOutBinding.DataType.IsValid())
	{
		InOutBinding.DataType = FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass());;
	}

	if (InOutBinding.DataDomain.IsSingleton())
	{
		InOutBinding.DataDomain = FOptimusDataDomain(TArray<FName>({UOptimusSkeletalMeshComponentSource::Domains::Vertex}));
	}
}

UOptimusNode_LoopTerminal* UOptimusNode_LoopTerminal::GetTerminalByType(EOptimusTerminalType InType)
{
	if (InType == EOptimusTerminalType::Unknown)
	{
		return nullptr;
	}
	
	if (TerminalType == InType)
	{
		return this;
	}

	return GetOtherTerminal();
}

const UOptimusNode_LoopTerminal* UOptimusNode_LoopTerminal::GetTerminalByType(EOptimusTerminalType InType) const
{
	if (InType == EOptimusTerminalType::Unknown)
	{
		return nullptr;
	}
	
	if (TerminalType == InType)
	{
		return this;
	}

	return GetOtherTerminal();	
}
#if WITH_EDITOR
void UOptimusNode_LoopTerminal::PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	GetOtherTerminal()->Modify();
	
	ClearPinPairs();
	GetOtherTerminal()->ClearPinPairs();

	for(FOptimusParameterBinding& Binding : LoopInfo.Bindings)
	{
		SanitizeBinding(Binding, NAME_None);
		AddPinPairs(Binding);
		GetOtherTerminal()->AddPinPairs(Binding);
	}	
}

void UOptimusNode_LoopTerminal::PropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.Property == FOptimusLoopTerminalInfo::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FOptimusLoopTerminalInfo, Count)))
	{
		return;
	}

	check(PinPairInfos.Num() == LoopInfo.Bindings.Num());
	
	for (int32 Index = 0; Index < LoopInfo.Bindings.Num(); Index++)
	{
		const FName OldName = GetPairedPins(PinPairInfos[Index])[0]->GetFName();
		SanitizeBinding(LoopInfo.Bindings[Index], OldName);
	}
	
	UpdatePinPairs();
	GetOtherTerminal()->Modify();
	GetOtherTerminal()->UpdatePinPairs();
}

void UOptimusNode_LoopTerminal::PropertyArrayItemAdded(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	const int32 ArrayIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));

	// We should only support appending to the end currently, given FOptimusParameterBindingArray detail customizaiton
	if (ensure(ArrayIndex == LoopInfo.Bindings.Num() - 1))
	{
		FOptimusParameterBinding& Binding = LoopInfo.Bindings[ArrayIndex];

		SanitizeBinding(Binding, NAME_None);
		
		AddPinPairs(Binding);
		GetOtherTerminal()->Modify();
		GetOtherTerminal()->AddPinPairs(Binding);
	}
}

void UOptimusNode_LoopTerminal::PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	const int32 ArrayIndex = InPropertyChangedEvent.GetArrayIndex(GET_MEMBER_NAME_STRING_CHECKED(FOptimusParameterBindingArray, InnerArray));
	
	RemovePinPair(ArrayIndex);
	GetOtherTerminal()->Modify();
	GetOtherTerminal()->RemovePinPair(ArrayIndex);
}

void UOptimusNode_LoopTerminal::PropertyArrayCleared(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	ClearPinPairs();
	GetOtherTerminal()->Modify();
	GetOtherTerminal()->ClearPinPairs();
}

void UOptimusNode_LoopTerminal::PropertyArrayItemMoved(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	MovePinPair();
	GetOtherTerminal()->Modify();
	GetOtherTerminal()->MovePinPair();
}
#endif

FName UOptimusNode_LoopTerminal::GetSanitizedBindingName(FName InNewName, FName InOldName)
{
	FName Name = InNewName;
	
	if (Name == NAME_None)
	{
		Name = TEXT("EmptyName");
	}

	Name = Optimus::GetSanitizedNameForHlsl(Name);

	if (Name != InOldName)
	{
		Name = GetAvailablePinNameStable(this, Name);
	}

	return Name;
}


#undef LOCTEXT_NAMESPACE
