// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IOptimusNodeAdderPinProvider.h"
#include "IOptimusNodePairProvider.h"
#include "IOptimusParameterBindingProvider.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "IOptimusUnnamedNodePinProvider.h"
#include "OptimusBindingTypes.h"
#include "OptimusNode.h"
#include "OptimusNode_GraphTerminal.h"

#include "OptimusNode_LoopTerminal.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusNode_LoopTerminal;


USTRUCT()
struct FOptimusPinPairInfo
{
	GENERATED_BODY()
	// Using PinNamePath here such that it plays well with default UObject undo/redo
	
	UPROPERTY()
	TArray<FName> InputPinPath;
	
	UPROPERTY()
	TArray<FName> OutputPinPath;

	bool operator==(const FOptimusPinPairInfo& InOther) const
	{
		return InOther.InputPinPath == InputPinPath && InOther.OutputPinPath == OutputPinPath;
	}

};

USTRUCT()
struct FOptimusLoopTerminalInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Loop Terminal")
	int32 Count = 1;

	UPROPERTY(EditAnywhere, Category="Loop Terminal", meta=(FullyExpand="true", NoResetToDefault))
	FOptimusParameterBindingArray Bindings;	
};

UCLASS(MinimalAPI, Hidden)
class UOptimusNode_LoopTerminal :
	public UOptimusNode,
	public IOptimusNodeAdderPinProvider,
	public IOptimusUnnamedNodePinProvider,
	public IOptimusNodePairProvider,
	public IOptimusPinMutabilityDefiner,
	public IOptimusParameterBindingProvider
{
	GENERATED_BODY()
	
public:
	UE_API UOptimusNode_LoopTerminal();

#if WITH_EDITOR
	// UObject overrides
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	// UOptimusNode overrides
	FName GetNodeCategory() const override { return NAME_None; }
	UE_API FText GetDisplayName() const override;
	UE_API void ConstructNode() override;
	UE_API bool ValidateConnection(const UOptimusNodePin& InThisNodesPin, const UOptimusNodePin& InOtherNodesPin, FString* OutReason) const override;

	// IOptimusNodeAdderPinProvider
	UE_API TArray<FAdderPinAction> GetAvailableAdderPinActions(const UOptimusNodePin* InSourcePin, EOptimusNodePinDirection InNewPinDirection, FString* OutReason) const override;
	UE_API TArray<UOptimusNodePin*> TryAddPinFromPin(const FAdderPinAction& InSelectedAction, UOptimusNodePin* InSourcePin, FName InNameToUse) override;
	UE_API bool RemoveAddedPins(TConstArrayView<UOptimusNodePin*> InAddedPinsToRemove) override;

	// IOptimusUnnamedNodePinProvider
	UE_API bool IsPinNameHidden(UOptimusNodePin* InPin) const override;
	UE_API FName GetNameForAdderPin(UOptimusNodePin* InPin) const override;

	// IOptimusPinMutabilityDefiner
	UE_API EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const override;

	// IOptimusNodePairProvider
	UE_API void PairToCounterpartNode(const IOptimusNodePairProvider* NodePairProvider) override;

	// IOptimusParameterBindingProvider
	UE_API FString GetBindingDeclaration(FName BindingName) const;
	UE_API bool GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const;
	UE_API bool GetBindingSupportReadCheckBoxVisibility(FName BindingName) const;
	UE_API EOptimusDataTypeUsageFlags GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const;
	
	UE_API UOptimusNodePin* GetPinCounterpart(const UOptimusNodePin* InNodePin, EOptimusTerminalType InTerminalType, TOptional<EOptimusNodePinDirection> InDirection = {}) const;
	UE_API UOptimusNode_LoopTerminal* GetOtherTerminal() const;
	UE_API int32 GetLoopCount() const;
	UE_API EOptimusTerminalType GetTerminalType() const;

	static UE_API int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin);

protected:
	friend class UOptimusNodeGraph;
	
	/** Indicates whether this is an entry or a return terminal node */
	UPROPERTY(VisibleAnywhere, Category="Loop Terminal")
	EOptimusTerminalType TerminalType;

	
	UPROPERTY(EditAnywhere, Category="Loop Terminal", DisplayName="Settings", meta=(FullyExpand="true", EditCondition="TerminalType==EOptimusTerminalType::Entry", EditConditionHides))
	FOptimusLoopTerminalInfo LoopInfo;

	UPROPERTY()
	TObjectPtr<UOptimusNodePin> IndexPin;
	
	UPROPERTY()
	TObjectPtr<UOptimusNodePin> CountPin;

	UPROPERTY()
	TArray<FOptimusPinPairInfo> PinPairInfos;
	
private:
#if WITH_EDITOR
	UE_API void PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemAdded(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayCleared(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemMoved(const FPropertyChangedEvent& InPropertyChangedEvent);
#endif

	UE_API TArray<UOptimusNodePin*> AddPinPairs(const FOptimusParameterBinding& InBinding);
	UE_API TArray<UOptimusNodePin*> AddPinPairsDirect(const FOptimusParameterBinding& InBinding);
	UE_API TArray<UOptimusNodePin*> GetPairedPins(const FOptimusPinPairInfo& InPair) const;
	static UE_API int32 GetPairIndex(const UOptimusNodePin* Pin);
	UE_API void RemovePinPair(int32 InPairIndex);
	UE_API void RemovePinPairDirect(int32 InPairIndex);
	UE_API void ClearPinPairs();
	UE_API void MovePinPair();
	UE_API void UpdatePinPairs();
	UE_API FOptimusLoopTerminalInfo* GetLoopInfo();
	UE_API const FOptimusLoopTerminalInfo* GetLoopInfo() const;
	
	UE_API void SanitizeBinding(FOptimusParameterBinding& InOutBinding, FName InOldName);
	UE_API UOptimusNode_LoopTerminal* GetTerminalByType(EOptimusTerminalType InType);
	UE_API const UOptimusNode_LoopTerminal* GetTerminalByType(EOptimusTerminalType InType) const;
	
	UE_API FName GetSanitizedBindingName(
		FName InNewName,
		FName InOldName
	);
};

#undef UE_API
