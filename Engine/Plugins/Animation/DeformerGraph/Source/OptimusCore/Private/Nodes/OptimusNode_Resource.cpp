// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_Resource.h"

#include "OptimusNodePin.h"
#include "OptimusResourceDescription.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_Resource)


int32 UOptimusNode_Resource::GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const
{
	if (!InPin || InPin->GetParentPin() != nullptr)
	{
		return INDEX_NONE;
	}
	if (!ensure(GetPins().Contains(InPin)))
	{
		return INDEX_NONE;
	}

	return InPin->GetDirection() == EOptimusNodePinDirection::Output 
		? UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType::Default) 
		: UOptimusRawBufferDataInterface::GetWriteValueOutputIndex(EOptimusBufferWriteType::Write);
}


FName UOptimusNode_Resource::GetResourcePinName(int32 InPinIndex, FName InNameOverride) const
{
	if (InNameOverride.IsNone())
	{
		if (const UOptimusResourceDescription* Res = GetResourceDescription())
		{
			InNameOverride = Res->ResourceName;
		}
		
	}
	if (ensure(InPinIndex >= 0 && InPinIndex < 2))
	{
		FString Suffix = InPinIndex == 0 ? TEXT("Set") : TEXT("Get");
		return FName(Suffix + InNameOverride.ToString());
	}

	return {};
}


void UOptimusNode_Resource::ConstructNode()
{
	if (const UOptimusResourceDescription* Res  = GetResourceDescription())
	{
		AddPinDirect(
			GetResourcePinName(0),
			EOptimusNodePinDirection::Input,
			Res->DataDomain,
			Res->DataType);

		const FName GetterName("Get");

		AddPinDirect(
			GetResourcePinName(1),
			EOptimusNodePinDirection::Output,
			Res->DataDomain,
			Res->DataType);
	}
}
