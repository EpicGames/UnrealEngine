// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode_ResourceAccessorBase.h"

#include "OptimusNode_Resource.generated.h"


UCLASS(Hidden)
class UOptimusNode_Resource
	: public UOptimusNode_ResourceAccessorBase
{
	GENERATED_BODY()

	// UOptimusNode_Resource implementations
	FName GetResourcePinName(int32 InPinIndex, FName InNameOverride = NAME_None) const override;
	// IOptimusDataInterfaceProvider implementations
	int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const override;

protected:
	void ConstructNode() override;
};
