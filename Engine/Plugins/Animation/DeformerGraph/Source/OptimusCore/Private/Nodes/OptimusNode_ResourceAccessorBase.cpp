// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_ResourceAccessorBase.h"

#include "OptimusCoreModule.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusResourceDescription.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "Actions/OptimusAction.h"
#include "Actions/OptimusResourceActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_ResourceAccessorBase)


#define LOCTEXT_NAMESPACE "OptimusResourceAccessorBase"


void UOptimusNode_ResourceAccessorBase::PreDuplicateRequirementActions(
	const UOptimusNodeGraph* InTargetGraph,
	FOptimusCompoundAction* InCompoundAction
	)
{
	if (DuplicationInfo.ResourceName.IsNone())
	{
		return;
	}
	
	// Check if the deformer has a variable that matches the name and type.
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(InTargetGraph->GetCollectionRoot());

	for (const UOptimusResourceDescription* ExistingResourceDesc: Deformer->GetResources())
	{
		if (ExistingResourceDesc->ResourceName == DuplicationInfo.ResourceName &&
			ExistingResourceDesc->DataType == DuplicationInfo.DataType &&
			ExistingResourceDesc->DataDomain == DuplicationInfo.DataDomain)
		{
			// We don't care about the binding itself, the user can fix that up later.
			return;
		}
	}

	// Note: This will fail if there are multiple overlapping names. A better system is needed.
	bool bFoundDuplicate = false;
	do
	{
		bFoundDuplicate = false;
		for (const UOptimusResourceDescription* ExistingResourceDesc: Deformer->GetResources())
		{
			if (ExistingResourceDesc->ResourceName == DuplicationInfo.ResourceName)
			{
				DuplicationInfo.ResourceName.SetNumber(DuplicationInfo.ResourceName.GetNumber() + 1);
				bFoundDuplicate = true;
			}
		}
	}
	while(bFoundDuplicate);

	InCompoundAction->AddSubAction<FOptimusResourceAction_AddResource>(DuplicationInfo.DataType, DuplicationInfo.ResourceName, DuplicationInfo.DataDomain);
}


void UOptimusNode_ResourceAccessorBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	if (!GetOwningGraph())
	{
		return;
	}

	const UOptimusDeformer* NewDescOwner = Cast<UOptimusDeformer>(GetOwningGraph()->GetCollectionRoot());
	if (!NewDescOwner)
	{
		return;
	}
	
	if (ResourceDesc.IsValid())
	{
		// This path is taken when the entire asset is duplicated
		const UOptimusDeformer* OldDescOwner = ResourceDesc->GetOwningDeformer();
		
		// No action needed if we are copying/pasting within the same deformer asset 
		if (OldDescOwner == NewDescOwner)
		{
			return;
		}

		// Refresh the ResourceDesc so that we don't hold a reference to a ResourceDesc in another deformer asset
		ResourceDesc = NewDescOwner->ResolveResource(ResourceDesc->GetFName());
	}
	else if (!DuplicationInfo.ResourceName.IsNone())
	{
		ResourceDesc = NewDescOwner->ResolveResource(DuplicationInfo.ResourceName);

		if (ResourceDesc.IsValid())
		{
			// Re-create the pins. Don't call PostCreateNode, since it won't notify the graph that the pin layout has
			// changed.
			check(GetPins().IsEmpty());
			ConstructNode();
		}

		// Empty the duplication info before we carry on.
		DuplicationInfo = FOptimusNode_ResourceAccessorBase_DuplicationInfo();
	}
}


void UOptimusNode_ResourceAccessorBase::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	if (const UOptimusResourceDescription* Res = ResourceDesc.Get())
	{
		Out.Logf(TEXT("%sCustomProperties ResourceDefinition Name=\"%s\" Type=\"%s\" DataDomain=\"%s\"\n"),
			FCString::Spc(Indent), *Res->ResourceName.ToString(), *Res->DataType->TypeName.ToString(),
			*Res->DataDomain.ToString());
	}
	
}


void UOptimusNode_ResourceAccessorBase::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	if (FParse::Command(&SourceText, TEXT("ResourceDefinition")))
	{
		FName ResourceName;
		if (!FParse::Value(SourceText, TEXT("Name="), ResourceName))
		{
			return;
		}
		
		FName DataTypeName;
		if (!FParse::Value(SourceText, TEXT("Type="), DataTypeName))
		{
			return;
		}

		const FOptimusDataTypeRef DataType = FOptimusDataTypeRegistry::Get().FindType(DataTypeName);
		if (!DataType.IsValid())
		{
			return;
		}

		FString DataDomainStr;
		if (!FParse::Value(SourceText, TEXT("DataDomain="), DataDomainStr))
		{
			return;
		}

		DuplicationInfo.ResourceName = ResourceName;
		DuplicationInfo.DataType = DataType;
		DuplicationInfo.DataDomain = FOptimusDataDomain::FromString(DataDomainStr);
	}
}


void UOptimusNode_ResourceAccessorBase::SetResourceDescription(UOptimusResourceDescription* InResourceDesc)
{
	if (!ensure(InResourceDesc))
	{
		return;
	}

	if (!EnumHasAnyFlags(InResourceDesc->DataType->UsageFlags, EOptimusDataTypeUsageFlags::Resource))
	{
		UE_LOGF(LogOptimusCore, Error, "Data type '%ls' is not usable in a resource",
		    *InResourceDesc->DataType->TypeName.ToString());
		return;
	}
	ResourceDesc = InResourceDesc;
}


UOptimusResourceDescription* UOptimusNode_ResourceAccessorBase::GetResourceDescription() const
{
	return ResourceDesc.Get();
}


FName UOptimusNode_ResourceAccessorBase::GetResourcePinName(int32 InPinIndex, FName InNameOverride) const
{
	if (InNameOverride.IsNone())
	{
		UOptimusResourceDescription const* Description = GetResourceDescription();
		if (!ensure(Description))
		{
			return {};
		}
	
		return Description->ResourceName;
	}
	else
	{
		return InNameOverride;
	}
}


TOptional<FText> UOptimusNode_ResourceAccessorBase::ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
{
	const UOptimusResourceDescription* ResourceDescription = GetResourceDescription();
	if (!ResourceDescription)
	{
		return LOCTEXT("NoDescriptor", "No resource descriptor set on this node");
	}

	if (!ResourceDescription->ComponentBinding.IsValid())
	{
		return FText::Format(LOCTEXT("NoComponentBinding", "No component binding set for resource '{0}'"), FText::FromName(ResourceDescription->ResourceName));
	}

	return {};
}

UOptimusComputeDataInterface* UOptimusNode_ResourceAccessorBase::GetDataInterface(
	UObject* InOuter
	) const
{
	// This should have been validated in ValidateForCompile.
	UOptimusResourceDescription* Description = ResourceDesc.Get();
	if (!ensure(Description))
	{
		return nullptr;
	}
	
	if (!Description->DataInterface)
	{
		Description->DataInterface = NewObject<UOptimusPersistentBufferDataInterface>(InOuter);
	}

	Description->DataInterface->ResourceName = Description->ResourceName;
	Description->DataInterface->ValueType = Description->DataType->ShaderValueType;
	Description->DataInterface->DataDomain = Description->DataDomain;
	Description->DataInterface->ComponentSourceBinding = Description->ComponentBinding;
	
	return Description->DataInterface;
}

UOptimusComponentSourceBinding* UOptimusNode_ResourceAccessorBase::GetComponentBinding(const FOptimusPinTraversalContext& InContext) const
{
	if (const UOptimusResourceDescription* Description = ResourceDesc.Get())
	{
		return Description->ComponentBinding.Get();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
