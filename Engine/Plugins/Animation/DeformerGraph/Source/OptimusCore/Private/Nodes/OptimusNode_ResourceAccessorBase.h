// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "IOptimusComponentBindingProvider.h"
#include "IOptimusDataInterfaceProvider.h"
#include "IOptimusNonCollapsibleNode.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "OptimusNode.h"

#include "OptimusNode_ResourceAccessorBase.generated.h"


class UOptimusComputeDataInterface;
class UOptimusResourceDescription;


USTRUCT()
struct FOptimusNode_ResourceAccessorBase_DuplicationInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName ResourceName;

	UPROPERTY()
	FOptimusDataTypeRef DataType;

	UPROPERTY()
	FOptimusDataDomain DataDomain;
};


UCLASS(Abstract)
class UOptimusNode_ResourceAccessorBase : 
	public UOptimusNode,
	public IOptimusDataInterfaceProvider,
	public IOptimusComponentBindingProvider,
	public IOptimusPinMutabilityDefiner,
	public IOptimusNonCollapsibleNode
{
	GENERATED_BODY()

public:
	void SetResourceDescription(UOptimusResourceDescription* InResourceDesc);

	UOptimusResourceDescription* GetResourceDescription() const;

	/** Returns name to use when creating each pin on the node. */
	virtual FName GetResourcePinName(int32 InPinIndex, FName InNameOverride = NAME_None) const;

	// UOptimusNode overrides
	FName GetNodeCategory() const override 
	{
		return CategoryName::Resources;
	}

	TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const override;

	// IOptimusDataInterfaceProvider implementations
	UOptimusComputeDataInterface* GetDataInterface(UObject *InOuter) const override;
	int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const override { return INDEX_NONE; }
	// Also IOptimusComponentBindingProvider implementation
	UOptimusComponentSourceBinding* GetComponentBinding(const FOptimusPinTraversalContext& InContext) const override;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EOptimusBufferWriteType GetDeprecatedBufferWriteType() const { return WriteType_DEPRECATED; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//IOptimusPinMutabilityDefiner overrides 
	EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const override { return EOptimusPinMutability::Mutable; };
	
protected:
	void PreDuplicateRequirementActions(const UOptimusNodeGraph* InTargetGraph, FOptimusCompoundAction* InCompoundAction) override;
	
	// UObject overrides
	void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	
protected:
	UPROPERTY()
	TWeakObjectPtr<UOptimusResourceDescription> ResourceDesc;

	/** Logical operation when writing to the resource. */
	UPROPERTY(meta=(DeprecatedProperty))
	EOptimusBufferWriteType WriteType_DEPRECATED= EOptimusBufferWriteType::Write;

	UPROPERTY(DuplicateTransient)
	FOptimusNode_ResourceAccessorBase_DuplicationInfo DuplicationInfo;
};
