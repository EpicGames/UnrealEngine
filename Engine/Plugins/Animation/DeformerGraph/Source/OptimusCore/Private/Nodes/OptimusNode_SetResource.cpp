// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/OptimusNode_SetResource.h"

#include "OptimusNodePin.h"
#include "OptimusResourceDescription.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_SetResource)


int32 UOptimusNode_SetResource::GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const
{
	if (!InPin || InPin->GetParentPin() != nullptr)
	{
		return INDEX_NONE;
	}
	if (!ensure(GetPins().Contains(InPin)))
	{
		return INDEX_NONE;
	}

	return UOptimusRawBufferDataInterface::GetWriteValueOutputIndex(EOptimusBufferWriteType::Write);
}


void UOptimusNode_SetResource::ConstructNode()
{
	if (const UOptimusResourceDescription* Res = GetResourceDescription())
	{
		AddPinDirect(
		    GetResourcePinName(0),
		    EOptimusNodePinDirection::Input,
			Res->DataDomain,
		    Res->DataType);
	}
}
