// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode_ResourceAccessorBase.h"

#include "OptimusNode_SetResource.generated.h"


UCLASS(Hidden)
class UOptimusNode_SetResource
	: public UOptimusNode_ResourceAccessorBase
{
	GENERATED_BODY()

	// IOptimusDataInterfaceProvider implementations
	int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const override;

protected:
	void ConstructNode() override;
};
