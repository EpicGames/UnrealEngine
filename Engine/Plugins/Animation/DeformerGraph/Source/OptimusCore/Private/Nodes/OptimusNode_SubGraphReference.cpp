// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_SubGraphReference.h"

#include "OptimusNodePin.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusBindingTypes.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "OptimusNode_GraphTerminal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_SubGraphReference)


UOptimusNode_SubGraphReference::UOptimusNode_SubGraphReference()
{
	EnableDynamicPins();
}

FText UOptimusNode_SubGraphReference::GetDisplayName() const
{
	if (SubGraph.Get())
	{
		return FText::FromString(SubGraph.Get()->GetName());
	}

	return FText::FromString("<graph missing>");
}


void UOptimusNode_SubGraphReference::ConstructNode()
{
	if (ensure(SubGraph.Get()))
	{
		const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
		FOptimusDataTypeRef ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());
		DefaultComponentPin = AddPinDirect(UOptimusNodeSubGraph::GraphDefaultComponentPinName, EOptimusNodePinDirection::Input, {}, ComponentSourceType);
		
		// After a duplicate, the kernel node has no pins, so we need to reconstruct them from
		// the bindings. We can assume that all naming clashes have already been dealt with.
		for (const FOptimusParameterBinding& Binding: SubGraph->InputBindings)
		{
			AddPinDirect(Binding, EOptimusNodePinDirection::Input);
		}
		for (const FOptimusParameterBinding& Binding: SubGraph->OutputBindings)
		{
			AddPinDirect(Binding, EOptimusNodePinDirection::Output);
		}
	}
}

void UOptimusNode_SubGraphReference::InitializeTransientData()
{
	Super::InitializeTransientData();
	
	ResolveSubGraphPointerAndSubscribe();
}

void UOptimusNode_SubGraphReference::BeginDestroy()
{
	Super::BeginDestroy();

	UnsubscribeFromSubGraph();
}


FOptimusRoutedNodePin UOptimusNode_SubGraphReference::GetPinCounterpart(
	UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InTraversalContext
) const
{
	if (!InNodePin || InNodePin->GetOwningNode() != this)
	{
		return {};
	}

	if (!ensure(SubGraph.Get()))
	{
		return {};
	}

	UOptimusNode_GraphTerminal* CounterpartNode = nullptr;
	if (InNodePin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		CounterpartNode = SubGraph->GetTerminalNode(EOptimusTerminalType::Entry);
	}
	else if (InNodePin->GetDirection() == EOptimusNodePinDirection::Output)
	{
		CounterpartNode = SubGraph->GetTerminalNode(EOptimusTerminalType::Return);
	}

	if (!ensure(CounterpartNode))
	{
		return {};
	}

	FOptimusRoutedNodePin Result{
		CounterpartNode->FindPinFromPath(InNodePin->GetPinNamePath()),
		InTraversalContext
	};
	Result.TraversalContext.ReferenceNesting.Push(this);

	return Result;
}

UOptimusNodeGraph* UOptimusNode_SubGraphReference::GetNodeGraphToShow()
{
	return SubGraph.Get();
}

UOptimusNodeSubGraph* UOptimusNode_SubGraphReference::GetReferencedSubGraph() const
{
	return SubGraph.Get();
}

UOptimusComponentSourceBinding* UOptimusNode_SubGraphReference::GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const
{
	if (!ensure(DefaultComponentPin.IsValid()))
	{
		return nullptr;
	}
	
	const UOptimusNodeGraph* OwningGraph = GetOwningGraph();
	TSet<UOptimusComponentSourceBinding*> Bindings = OwningGraph->GetComponentSourceBindingsForPin(DefaultComponentPin.Get(), InTraversalContext);
	
	if (!Bindings.IsEmpty() && ensure(Bindings.Num() == 1))
	{
		return Bindings.Array()[0];
	}

	// Default to the primary binding, but only if we're at the top-most level of the graph.
	if (const UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(OwningGraph->GetCollectionOwner()))
	{
		return Deformer->GetPrimaryComponentBinding();
	}

	if (const UOptimusNodeSubGraph* OwningSubGraph = Cast<UOptimusNodeSubGraph>(OwningGraph))
	{
		return OwningSubGraph->GetDefaultComponentBinding(InTraversalContext);
	}

	return nullptr;	
}

UOptimusNodePin* UOptimusNode_SubGraphReference::GetDefaultComponentBindingPin() const
{
	return DefaultComponentPin.Get();
}

void UOptimusNode_SubGraphReference::InitializeSerializedSubGraphName(FName InInitialSubGraphName)
{
	SubGraphName = InInitialSubGraphName;
	// SubGraph object pointer is resolved during InitializeTransientData()
}

void UOptimusNode_SubGraphReference::RefreshSerializedSubGraphName()
{
	SubGraphName = SubGraph->GetFName();
}

FName UOptimusNode_SubGraphReference::GetSerializedSubGraphName() const
{
	return SubGraphName;
}

void UOptimusNode_SubGraphReference::ResolveSubGraphPointerAndSubscribe()
{
	SubGraph = Cast<UOptimusNodeSubGraph>(GetOwningGraph()->FindGraphByName(GetSerializedSubGraphName()));
	SubscribeToSubGraph();
}

void UOptimusNode_SubGraphReference::SubscribeToSubGraph()
{
	if (ensure(SubGraph.IsValid() && !SubGraph->GetOnBindingArrayPasted().IsBoundToObject(this)))
	{
		SubGraph->GetOnBindingArrayPasted().AddUObject(this, &UOptimusNode_SubGraphReference::RecreateBindingPins);
		SubGraph->GetOnBindingValueChanged().AddUObject(this, &UOptimusNode_SubGraphReference::SyncPinsToBindings);
		SubGraph->GetOnBindingArrayItemAdded().AddUObject(this, &UOptimusNode_SubGraphReference::AddPinForNewBinding);
		SubGraph->GetOnBindingArrayItemRemoved().AddUObject(this, &UOptimusNode_SubGraphReference::RemoveStalePins);
		SubGraph->GetOnBindingArrayCleared().AddUObject(this, &UOptimusNode_SubGraphReference::RemoveStalePins);
		SubGraph->GetOnBindingArrayItemMoved().AddUObject(this, &UOptimusNode_SubGraphReference::OnBindingMoved);	
	}
	
}

void UOptimusNode_SubGraphReference::UnsubscribeFromSubGraph() const
{
	if (SubGraph.IsValid())
	{
		SubGraph->GetOnBindingArrayPasted().RemoveAll(this);
		SubGraph->GetOnBindingValueChanged().RemoveAll(this);
		SubGraph->GetOnBindingArrayItemAdded().RemoveAll(this);
		SubGraph->GetOnBindingArrayItemRemoved().RemoveAll(this);
		SubGraph->GetOnBindingArrayCleared().RemoveAll(this);
		SubGraph->GetOnBindingArrayItemMoved().RemoveAll(this);
	}
}

void UOptimusNode_SubGraphReference::AddPinForNewBinding(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	EOptimusNodePinDirection NewPinDirection = EOptimusNodePinDirection::Unknown;
	UOptimusNodePin* BeforePin = nullptr;
	
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->InputBindings;
		NewPinDirection = EOptimusNodePinDirection::Input;
		TArray<UOptimusNodePin*> OutputPins = GetPinsByDirection(EOptimusNodePinDirection::Output, false);
		BeforePin = OutputPins.Num() > 0 ? OutputPins[0] : nullptr;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->OutputBindings;
		NewPinDirection = EOptimusNodePinDirection::Output;
		BeforePin = nullptr;
	}

	if (BindingArrayPtr)
	{
		check(BindingArrayPtr->Num() > 0);
		AddPin(BindingArrayPtr->Last(), NewPinDirection, BeforePin);
	}
	
}

void UOptimusNode_SubGraphReference::RemoveStalePins(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	TArray<UOptimusNodePin*> PinsToCheck;
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->InputBindings;
		PinsToCheck = GetBindingPinsByDirection(EOptimusNodePinDirection::Input);
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->OutputBindings;
		PinsToCheck = GetBindingPinsByDirection(EOptimusNodePinDirection::Output);
	}

	for (UOptimusNodePin* Pin : PinsToCheck)
	{
		int32 Index = BindingArrayPtr->IndexOfByPredicate([ PinName = Pin->GetFName()](const FOptimusParameterBinding& InBinding){
			 return InBinding.Name == PinName;
		});

		if (Index == INDEX_NONE)
		{
			RemovePin(Pin);
		}
	}
}

void UOptimusNode_SubGraphReference::OnBindingMoved(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	TArray<UOptimusNodePin*> PinsToCheck;
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->InputBindings;
		PinsToCheck = GetBindingPinsByDirection(EOptimusNodePinDirection::Input);
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->OutputBindings;
		PinsToCheck = GetBindingPinsByDirection(EOptimusNodePinDirection::Output);
	}

	if (BindingArrayPtr)
	{
		TArray<FName> PinNames;

		for (UOptimusNodePin* Pin : PinsToCheck)
		{
			PinNames.Add(Pin->GetFName());
		}

		TArray<FName> BindingNames;
		for (const FOptimusParameterBinding& Binding : *BindingArrayPtr)
		{
			BindingNames.Add(Binding.Name);
		}

		FName PinName;
		FName NextPinName;
		if (Optimus::FindMovedItemInNameArray(PinNames, BindingNames, PinName, NextPinName))
		{
			const int32 PinIndex = PinsToCheck.IndexOfByPredicate([PinName](const UOptimusNodePin* InPin)
			{
				return InPin->GetFName() == PinName;
			});
		
			const int32 NextPinIndex = NextPinName == NAME_None ?
				INDEX_NONE : PinsToCheck.IndexOfByPredicate([NextPinName](const UOptimusNodePin* InPin)
			{
				return InPin->GetFName() == NextPinName;
			});

			MovePin(PinsToCheck[PinIndex], NextPinIndex != INDEX_NONE ? PinsToCheck[NextPinIndex] : nullptr);
		}
	}
}

void UOptimusNode_SubGraphReference::RecreateBindingPins(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	EOptimusNodePinDirection NewPinDirection = EOptimusNodePinDirection::Unknown;
	TArray<UOptimusNodePin*> PinsToRemove;
	UOptimusNodePin* BeforePin = nullptr;
	
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->InputBindings;
		NewPinDirection = EOptimusNodePinDirection::Input;
		PinsToRemove = GetBindingPinsByDirection(EOptimusNodePinDirection::Input);
		
		TArray<UOptimusNodePin*> OutputPins = GetPinsByDirection(EOptimusNodePinDirection::Output, false);
		BeforePin = OutputPins.Num() > 0 ? OutputPins[0] : nullptr;
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->OutputBindings;
		NewPinDirection = EOptimusNodePinDirection::Output;
		PinsToRemove = GetBindingPinsByDirection(EOptimusNodePinDirection::Output);
		
		BeforePin = nullptr;
	}

	if (BindingArrayPtr)
	{
		for (UOptimusNodePin* Pin : PinsToRemove)
		{
			RemovePin(Pin);
		}

		for (const FOptimusParameterBinding& Binding : *BindingArrayPtr)
		{
			AddPin(Binding, NewPinDirection, BeforePin);
		}
	}	
}

void UOptimusNode_SubGraphReference::SyncPinsToBindings(FName InBindingArrayPropertyName)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;

	TArray<UOptimusNodePin*> BindingPins;
	if (InBindingArrayPropertyName == UOptimusNodeSubGraph::InputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->InputBindings;
		BindingPins = GetBindingPinsByDirection(EOptimusNodePinDirection::Input);
	}
	else if (InBindingArrayPropertyName == UOptimusNodeSubGraph::OutputBindingsPropertyName)
	{
		BindingArrayPtr = &SubGraph->OutputBindings;
		BindingPins = GetBindingPinsByDirection(EOptimusNodePinDirection::Output);
	}

	if (BindingArrayPtr)
	{
		check(BindingArrayPtr->Num() == BindingPins.Num());
		for (int32 Index = 0 ; Index < BindingArrayPtr->Num(); Index++)
		{
			UOptimusNodePin* BindingPin = BindingPins[Index];
			const FOptimusParameterBinding& Binding = (*BindingArrayPtr)[Index];
			if (BindingPin->GetFName() != Binding.Name)
			{
				SetPinName(BindingPin, Binding.Name);
			}
			if (BindingPin->GetDataType() != Binding.DataType.Resolve())
			{
				SetPinDataType(BindingPin, Binding.DataType);
			}
			if (BindingPin->GetDataDomain() != Binding.DataDomain)
			{
				SetPinDataDomain(BindingPin, Binding.DataDomain);
			}
		}
	}		
}

TArray<UOptimusNodePin*> UOptimusNode_SubGraphReference::GetBindingPinsByDirection(EOptimusNodePinDirection InDirection)
{
	TArray<UOptimusNodePin*> Results = GetPinsByDirection(InDirection, false);
	
	Results.Remove(DefaultComponentPin.Get());

	return Results;
}


