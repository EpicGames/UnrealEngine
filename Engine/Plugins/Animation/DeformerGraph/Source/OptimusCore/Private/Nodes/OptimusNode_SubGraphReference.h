// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusNodeGraphProvider.h"
#include "IOptimusNodePinRouter.h"
#include "IOptimusNodeSubGraphReferencer.h"
#include "OptimusNode.h"

#include "OptimusNode_SubGraphReference.generated.h"

#define UE_API OPTIMUSCORE_API


class UOptimusNodeSubGraph;


UCLASS(MinimalAPI, Hidden)
class UOptimusNode_SubGraphReference :
	public UOptimusNode,
	public IOptimusNodePinRouter,
	public IOptimusNodeGraphProvider,
	public IOptimusNodeSubGraphReferencer
{
	GENERATED_BODY()

public:
	UE_API UOptimusNode_SubGraphReference();

	// UOptimusNode overrides
	FName GetNodeCategory() const override { return NAME_None; }
	UE_API FText GetDisplayName() const override;
	UE_API void ConstructNode() override;

	// UObject overrides
	UE_API void BeginDestroy() override;

	// IOptimusNodePinRouter implementation
	UE_API FOptimusRoutedNodePin GetPinCounterpart(
		UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InTraversalContext
	) const override;

	// IOptimusNodeGraphProvider
	UE_API UOptimusNodeGraph* GetNodeGraphToShow() override;

	// IOptimusNodeSubGraphReferencer
	UE_API UOptimusNodeSubGraph* GetReferencedSubGraph() const override;
	UE_API UOptimusComponentSourceBinding* GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const override;
	UE_API UOptimusNodePin* GetDefaultComponentBindingPin() const override;

	// Only used during node creation, cannot be used to reference a different graph once node is constructed
	UE_API void InitializeSerializedSubGraphName(FName InInitialSubGraphName);
	UE_API void RefreshSerializedSubGraphName();
	UE_API FName GetSerializedSubGraphName() const;
	
protected:
	// UOptimusNode overrides
	UE_API void InitializeTransientData() override;
	
	UE_API void ResolveSubGraphPointerAndSubscribe();
	UE_API void SubscribeToSubGraph();
	UE_API void UnsubscribeFromSubGraph() const;

	UE_API void AddPinForNewBinding(FName InBindingArrayPropertyName);
	UE_API void RemoveStalePins(FName InBindingArrayPropertyName);
	UE_API void OnBindingMoved(FName InBindingArrayPropertyName);
	UE_API void RecreateBindingPins(FName InBindingArrayPropertyName);
	UE_API void SyncPinsToBindings(FName InBindingArrayPropertyName);

	UE_API TArray<UOptimusNodePin*> GetBindingPinsByDirection(EOptimusNodePinDirection InDirection);


	UPROPERTY()
	FName SubGraphName;
	
	UPROPERTY()
	TWeakObjectPtr<UOptimusNodePin> DefaultComponentPin;

private:
	
	
	/** The graph that owns us. This contains all the necessary pin information to add on
	 * the terminal node. Initialized when the node is loaded/created
	 */
	TWeakObjectPtr<UOptimusNodeSubGraph> SubGraph;	
};

#undef UE_API
