// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusActionStack.h"

#include "Actions/OptimusAction.h"
#include "IOptimusPathResolver.h"

#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusActionStack)


UOptimusActionStack::UOptimusActionStack()
{
	
}


bool UOptimusActionStack::RunAction(TSharedPtr<FOptimusAction> InAction)
{
	if (!ensureMsgf(!bIsRunningAction, TEXT("RunAction is not re-entrant")))
	{
		return false;
	}

	// A security blanket to ensure we don't end up recursively running actions from an action.
	TGuardValue<bool> RunningActionScope(bIsRunningAction, true);

	if (ActionScopes.Num() == 0)
	{
		IOptimusPathResolver* Root = GetGraphCollectionRoot();
		if (!Root)
		{
			return false;
		}

		const bool bTransacted = BeginScopeFunc && EndScopeFunc;

		if (!InAction->Do(Root))
		{
			return false;
		}

		// Prune the undo stack if there are entries beyond the current action. For non-transacted
		// setups, this will always clear the stack.
		Actions.SetNum(CurrentActionIndex);

		Actions.Add(InAction);

		if (bTransacted)
		{
			CurrentActionIndex++;

			// Create a transaction scope on this object for modifying the transaction action index.
			// This will cause it to be out of sync with the current action index when PostTransacted
			// is called and we can use it to replay the stack in the direction of the transaction index.
			int32 TransactionId = BeginScopeFunc(this, InAction->GetTitle());

			TransactedActionIndex++;

			EndScopeFunc(TransactionId);
		}
	}
	else
	{
		ActionScopes.Last()->AddSubAction(InAction);
	}

	return true;
}


bool UOptimusActionStack::RunAction(FOptimusAction* InAction)
{
	return RunAction(TSharedPtr<FOptimusAction>(InAction));
}


bool UOptimusActionStack::Redo()
{
	if (CurrentActionIndex == Actions.Num())
	{
		// FIXME: Report lack of entries.
		return false;
	}

	return Actions[CurrentActionIndex++]->Do(GetGraphCollectionRoot());
}


bool UOptimusActionStack::Undo()
{
	if (CurrentActionIndex == 0)
	{
		// FIXME: Report lack of entries.
		return false;
	}

	return Actions[--CurrentActionIndex]->Undo(GetGraphCollectionRoot());
}


#if WITH_EDITOR
void UOptimusActionStack::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	if (TransactionEvent.GetEventType() != ETransactionObjectEventType::UndoRedo)
	{
		return;
	}

	// If the transacted action index is smaller than the current action index, that means we
	// have to undo to rewind the current action index back to it.
	while (TransactedActionIndex < CurrentActionIndex)
	{
		if (!Undo())
		{
			return;
		}
	}

	// If the transacted action index is greater than the current action index, that means we
	// have to redo to fast-forward the current action index up to it.
	while (TransactedActionIndex > CurrentActionIndex)
	{
		if (!Redo())
		{
			return;
		}
	}
}
#endif // WITH_EDITOR


void UOptimusActionStack::OpenActionScope(const FString& InTitle)
{
	ActionScopes.Add(MakeShared<FOptimusCompoundAction>(InTitle));
}


bool UOptimusActionStack::CloseActionScope()
{
	if (ensure(ActionScopes.Num() > 0))
	{
		TSharedPtr<FOptimusCompoundAction> Action = ActionScopes.Pop();

		if (Action->HasSubActions())
		{
			return RunAction(Action);
		}
	}

	return false;
}


IOptimusPathResolver* UOptimusActionStack::GetGraphCollectionRoot() const
{
	return Cast<IOptimusPathResolver>(GetOuter());
}


void UOptimusActionStack::SetTransactionScopeFunctions(
	TFunction<int(UObject* TransactObject, const FString& Title)> InBeginScopeFunc,
	TFunction<void(int InTransactionId)> InEndScopeFunc
)
{
	BeginScopeFunc = InBeginScopeFunc;
	EndScopeFunc = InEndScopeFunc;
}


FOptimusActionScope::FOptimusActionScope(
	UOptimusActionStack& InActionStack, 
	const FString& InTitle 
	) :
	ActionStack(InActionStack)
{
	ActionStack.OpenActionScope(InTitle);
}


FOptimusActionScope::~FOptimusActionScope()
{
	ActionStack.CloseActionScope();
}

void FOptimusActionScope::SetTitle(const FString& InTitle)
{
	ActionStack.ActionScopes.Last()->SetTitle(InTitle);
}
