// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusComponentSource.h"

#include "Components/SkinnedMeshComponent.h"
#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusComponentSource)


bool UOptimusComponentSource::IsUsableAsPrimarySource() const
{
	return GetComponentClass()->IsChildOf<USkinnedMeshComponent>();
}

TArray<const UOptimusComponentSource*> UOptimusComponentSource::GetAllSources()
{
	TArray<const UOptimusComponentSource*> ComponentSources;
	
	for (TObjectIterator<UClass> It; It; ++It)
	{
		const UClass* Class = *It;
		
		if (!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden) &&
			Class->IsChildOf(StaticClass()) &&
			Class->GetPackage() != GetTransientPackage())
		{
			if (const UOptimusComponentSource* ComponentSource = Cast<const UOptimusComponentSource>(Class->GetDefaultObject()))
			{
				ComponentSources.Add(ComponentSource);
			}
		}
	}
	return ComponentSources;
}

TSet<FName> UOptimusComponentSource::GetAllExecutionDomains()
{
	TSet<FName> Domains;

	for (const UOptimusComponentSource* Source : GetAllSources())
	{
		Domains.Append(Source->GetExecutionDomains());
	}

	return Domains;
}

const UOptimusComponentSource* UOptimusComponentSource::GetSourceFromDataInterface(
	const UOptimusComputeDataInterface* InDataInterface
	)
{
	// If the compute data interface requires a specific component, then there must be a source for it.
	for (const UOptimusComponentSource* ComponentSource: GetAllSources())
	{
		if (ComponentSource->GetComponentClass() == InDataInterface->GetRequiredComponentClass())
		{
			return ComponentSource;
		}
	}
	
	return nullptr;
}


const FName UOptimusComponentSourceBinding::PrimaryBindingName("Primary");

FName UOptimusComponentSourceBinding::GetPrimaryBindingName()
{
	return PrimaryBindingName;
}

UOptimusDeformer* UOptimusComponentSourceBinding::GetOwningDeformer() const
{
	const UOptimusComponentSourceBindingContainer* Container = CastChecked<UOptimusComponentSourceBindingContainer>(GetOuter());
	return Container ? CastChecked<UOptimusDeformer>(Container->GetOuter()) : nullptr;
}

int32 UOptimusComponentSourceBinding::GetIndex() const
{
	const UOptimusComponentSourceBindingContainer* Container = CastChecked<UOptimusComponentSourceBindingContainer>(GetOuter());
	return Container->Bindings.IndexOfByKey(this);
}

const UOptimusComponentSource* UOptimusComponentSourceBinding::GetComponentSource() const
{
	if (ComponentType)
	{
		return ComponentType->GetDefaultObject<UOptimusComponentSource>();
	}
	
	return nullptr;
}

#if WITH_EDITOR
void UOptimusComponentSourceBinding::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusComponentSourceBinding, BindingName))
	{
		UOptimusDeformer *Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Rename the object itself and update the nodes. A lot of this is covered by
			// UOptimusDeformer::RenameResource but since we're inside of a transaction, which
			// has already taken a snapshot of this object, we have to do the remaining 
			// operations on this object under the transaction scope.
			BindingName = Optimus::GetUniqueNameForScope(GetOuter(), BindingName);
			Rename(*BindingName.ToString(), nullptr);
			
			constexpr bool bForceChange = true;
			Deformer->RenameComponentBinding(this, BindingName, bForceChange);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusComponentSourceBinding, ComponentType))
	{
		UOptimusDeformer *Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Set the component source again, so that we can remove any links that are now
			// type-incompatible.
			constexpr bool bForceChange = true;
			Deformer->SetComponentBindingSource(this, ComponentType->GetDefaultObject<UOptimusComponentSource>(), bForceChange);
		}
	}
}

void UOptimusComponentSourceBinding::PreEditUndo()
{
	Super::PreEditUndo();

	BindingNameForUndo = BindingName;
}

void UOptimusComponentSourceBinding::PostEditUndo()
{
	Super::PostEditUndo();

	if (BindingNameForUndo != BindingName)
	{
		const UOptimusDeformer *Deformer = GetOwningDeformer();
		Deformer->Notify(EOptimusGlobalNotifyType::ComponentBindingRenamed, this);
	}
}
#endif
