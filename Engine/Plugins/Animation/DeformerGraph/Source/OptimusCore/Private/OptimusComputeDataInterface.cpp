// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusComputeDataInterface.h"

#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

#include "Templates/SubclassOf.h"
#include "UObject/UObjectIterator.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusComputeDataInterface)


const FName UOptimusComputeDataInterface::CategoryName::DataInterfaces("Data Interfaces");
const FName UOptimusComputeDataInterface::CategoryName::ExecutionDataInterfaces("Execution Data Interfaces");
const FName UOptimusComputeDataInterface::CategoryName::OutputDataInterfaces("Output Data Interfaces");


void UOptimusComputeDataInterface::ExportState(FArchive& Ar)
{
	// We have to use the proxy archive because FMemoryWriter simply asserts when the object has object references.
	// However if the data interface spawns more objects, it needs to override this function with custom logic for undo/redo to work properly
	FObjectAndNameAsStringProxyArchive NodeProxyArchive(
			Ar, /* bInLoadIfFindFails=*/ false);

	SerializeScriptProperties(NodeProxyArchive);	
}

void UOptimusComputeDataInterface::ImportState(FArchive& Ar)
{
	FObjectAndNameAsStringProxyArchive NodeProxyArchive(
			Ar, /* bInLoadIfFindFails=*/false);
	
	SerializeScriptProperties(NodeProxyArchive);
}

TSet<TArray<FName>> UOptimusComputeDataInterface::GetUniqueNestedContexts() const
{
	TSet<TArray<FName>> UniqueContextNames;
	for (const FOptimusCDIPinDefinition& PinDef: GetPinDefinitions())
	{
		if (!PinDef.DataDimensions.IsEmpty())
		{
			TArray<FName> ContextNames;
			for (const FOptimusCDIPinDefinition::FDimensionInfo& ContextInfo: PinDef.DataDimensions)
			{
				ContextNames.Add(ContextInfo.ContextName);
			}
			UniqueContextNames.Add(ContextNames);
		}
	}

	return UniqueContextNames;
}


TArray<TSubclassOf<UOptimusComputeDataInterface>> UOptimusComputeDataInterface::GetAllComputeDataInterfaceClasses()
{
	TArray<TSubclassOf<UOptimusComputeDataInterface>> DataInterfaceClasses;

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden) &&
			Class->IsChildOf(StaticClass()))
		{
			UOptimusComputeDataInterface* DataInterface = Cast<UOptimusComputeDataInterface>(Class->GetDefaultObject());
			if (DataInterface && DataInterface->IsVisible())
			{
				DataInterfaceClasses.Add(TSubclassOf<UOptimusComputeDataInterface>(Class));
			}
		}
	}
	return DataInterfaceClasses;
}


TSet<TArray<FName>> UOptimusComputeDataInterface::GetUniqueDomainDimensions()
{
	TSet<TArray<FName>> UniqueNestedContextNames;
	for (const TSubclassOf<UOptimusComputeDataInterface>& DataInterfaceClass: GetAllComputeDataInterfaceClasses())
	{
		UOptimusComputeDataInterface* DataInterface = Cast<UOptimusComputeDataInterface>(DataInterfaceClass->GetDefaultObject());
		if (DataInterface)
		{
			UniqueNestedContextNames.Append(DataInterface->GetUniqueNestedContexts());
		}
	}
	return UniqueNestedContextNames;
}

void UOptimusComputeDataInterface::RegisterAllTypes()
{
	for (const TSubclassOf<UOptimusComputeDataInterface>& DataInterfaceClass : GetAllComputeDataInterfaceClasses())
	{
		UOptimusComputeDataInterface* DataInterface = Cast<UOptimusComputeDataInterface>(DataInterfaceClass->GetDefaultObject());
		if (DataInterface != nullptr)
		{
			DataInterface->RegisterTypes();
		}
	}
}
