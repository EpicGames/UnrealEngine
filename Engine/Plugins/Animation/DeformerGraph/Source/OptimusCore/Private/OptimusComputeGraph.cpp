// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusComputeGraph.h"

#include "IOptimusOutputBufferWriter.h"
#include "Components/MeshComponent.h"
#include "ComputeFramework/ComputeKernelCompileResult.h"
#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "IOptimusComputeKernelProvider.h"
#include "IOptimusShaderTextProvider.h"
#include "IOptimusValueProvider.h"
#include "OptimusDeformer.h"
#include "OptimusCoreModule.h"
#include "OptimusHelpers.h"
#include "OptimusNode.h"
#include "OptimusObjectVersion.h"
#include "Animation/MeshDeformerInstance.h"
#include "DataInterfaces/OptimusDataInterfaceGraph.h"
#include "DataInterfaces/OptimusDataInterfaceHalfEdge.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusComputeGraph)

#define LOCTEXT_NAMESPACE "OptimusComputeGraph"

void UOptimusComputeGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FOptimusObjectVersion::GUID);
}

void UOptimusComputeGraph::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::AddBindingsToGraph)
	{
		// Add default bindings for legacy data.
		Bindings.Add(UMeshComponent::StaticClass());
		DataInterfaceToBinding.AddZeroed(DataInterfaces.Num());
	}
}

static EOptimusDiagnosticLevel ProcessCompilationMessage(UOptimusDeformer* InOwner,	UOptimusNode const* InKernelNode, FString const& InKernelFriendlyName, FComputeKernelCompileMessage const& InMessage)
{
	FOptimusCompilerDiagnostic Diagnostic;

	if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Error)
	{
		Diagnostic.Level = EOptimusDiagnosticLevel::Error;
	}
	else if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Warning)
	{
		Diagnostic.Level = EOptimusDiagnosticLevel::Warning;
	}
	else if (InMessage.Type == FComputeKernelCompileMessage::EMessageType::Info)
	{
		Diagnostic.Level = EOptimusDiagnosticLevel::Info;
	}

	Diagnostic.Line = InMessage.Line;
	Diagnostic.ColumnStart = InMessage.ColumnStart;
	Diagnostic.ColumnEnd = InMessage.ColumnEnd;

	// If error path is a UObject, then stpre a reference to the related object.
	FString Path = InMessage.VirtualFilePath;
	if (Optimus::ConvertShaderFilePathToObjectPath(Path))
	{
		Diagnostic.Object = StaticFindObject(nullptr, nullptr, *Path, EFindObjectFlags::ExactClass);
		if (UObject const* ObjectPtr = Diagnostic.Object.Get())
		{
			// Special case path display if object is the current kernel.
			Path = (ObjectPtr == InKernelNode) ? InKernelFriendlyName : ObjectPtr->GetName();
		}
	}
	
	// If error path is a real file, then store its absolute path on disk.
	if (!InMessage.RealFilePath.IsEmpty())
	{
		Diagnostic.AbsoluteFilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*InMessage.RealFilePath);
	}

	FString Message;
	if (!Path.IsEmpty())
	{
		Message = Path;
		if (InMessage.Line != -1)
		{
			if (InMessage.ColumnStart == InMessage.ColumnEnd)
			{
				Message += FString::Printf(TEXT(" (%d,%d)"), InMessage.Line, InMessage.ColumnStart);
			}
			else
			{
				Message += FString::Printf(TEXT(" (%d,%d-%d)"), InMessage.Line, InMessage.ColumnStart, InMessage.ColumnEnd);
			}
		}
		Message += TEXT(": ");
	}
	Message += InMessage.Text;
	Diagnostic.Message = FText::FromString(Message);

	if (InOwner)
	{
		InOwner->GetCompileMessageDelegate().Broadcast(Diagnostic);
	}

	return Diagnostic.Level;
}

void UOptimusComputeGraph::OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults)
{
	// Find the Optimus objects from the raw kernel index.
	if (KernelToNode.IsValidIndex(InKernelIndex))
	{
		UOptimusDeformer* Owner = Cast<UOptimusDeformer>(GetOuter());
		
		// Make sure the node hasn't been GC'd.
		if (UOptimusNode* Node = const_cast<UOptimusNode*>(KernelToNode[InKernelIndex].LoadSynchronous()))
		{
			FString FriendlyName = Owner->GetName() / GetFName().GetPlainNameString() / Node->GetDisplayName().ToString();
			EOptimusDiagnosticLevel DiagnosticLevel = EOptimusDiagnosticLevel::None;

			for (FComputeKernelCompileMessage const& CompileOutputMessage : InCompileResults.Messages)
			{
				EOptimusDiagnosticLevel MessageDiagnosticLevel = ProcessCompilationMessage(Owner, Node, FriendlyName, CompileOutputMessage);
				if (MessageDiagnosticLevel > DiagnosticLevel)
				{
					DiagnosticLevel = MessageDiagnosticLevel;
				}
			}

			Node->SetDiagnosticLevel(DiagnosticLevel);
		}
	}
}

EMeshDeformerOutputBuffer UOptimusComputeGraph::GetOutputBuffers() const
{
	EMeshDeformerOutputBuffer Result = EMeshDeformerOutputBuffer::None;
	
	for (const FComputeGraphEdge& GraphEdge : GraphEdges)
	{
		if (!GraphEdge.bKernelInput)
		{
			const UComputeDataInterface* DataInterface = DataInterfaces[GraphEdge.DataInterfaceIndex];
			if (const IOptimusOutputBufferWriter* BufferOverrider = Cast<const IOptimusOutputBufferWriter>(DataInterface))
			{
				Result |= BufferOverrider->GetOutputBuffer(GraphEdge.DataInterfaceBindingIndex);
			}
		}
	}

	return Result;
}

UOptimusGraphDataInterface* UOptimusComputeGraph::GetGraphDataInterfaceForPostLoadFixUp()
{
	for (UComputeDataInterface* DataInterface : DataInterfaces)
	{
		if (UOptimusGraphDataInterface* GraphDataInterface = Cast<UOptimusGraphDataInterface>(DataInterface))
		{
			// There should be only one graph data interface
			return GraphDataInterface;
		}
	}
	return nullptr;
}

bool UOptimusComputeGraph::HasHalfEdgeDataInterface() const
{
	for (UComputeDataInterface* DataInterface : DataInterfaces)
	{
		if (DataInterface->IsA<UOptimusHalfEdgeDataInterface>())
		{
			return true;
		}
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
