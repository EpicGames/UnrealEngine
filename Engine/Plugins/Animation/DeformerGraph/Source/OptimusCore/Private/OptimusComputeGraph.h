// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeGraph.h"
#include "OptimusShaderText.h"

#include "OptimusComputeGraph.generated.h"

enum class EMeshDeformerOutputBuffer : uint8;
class UOptimusDeformer;
class UOptimusNode;
class UOptimusGraphDataInterface;

UCLASS()
class UOptimusComputeGraph :
	public UComputeGraph
{
	GENERATED_BODY()

public:
	// UObject overrides
	void Serialize(FArchive& Ar) override;
	void PostLoad() override;

	// UComputeGraph overrides
	void OnKernelCompilationComplete(int32 InKernelIndex, FComputeKernelCompileResults const& InCompileResults) override;

	EMeshDeformerOutputBuffer GetOutputBuffers() const;
protected:
	// Lookup into Graphs array from the UComputeGraph kernel index.
	UPROPERTY()
	TArray<TSoftObjectPtr<const UOptimusNode>> KernelToNode;

	UOptimusGraphDataInterface* GetGraphDataInterfaceForPostLoadFixUp();
	bool HasHalfEdgeDataInterface() const;
	
	friend class UOptimusDeformer;
};
