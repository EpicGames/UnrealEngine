// Copyright Epic Games, Inc. All Rights Reserved.
#include "OptimusConstant.h"

#include "OptimusNode.h"
#include "OptimusExpressionEvaluator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusConstant)


FOptimusConstantIdentifier::FOptimusConstantIdentifier(const UOptimusNode* InNode,  const FName& InGroupName , const FName& InConstantName)
{
	NodePath = *InNode->GetNodePath();
	GroupName = InGroupName;
	ConstantName = InConstantName;
}

void FOptimusKernelConstantContainer::AddToKernelContainer(const FOptimusConstant& InConstant)
{
	TArray<FOptimusConstant>& Constants =
		InConstant.Type == EOptimusConstantType::Input ?
			InputConstants :OutputConstants;

	Constants.Add(InConstant);	
	
	GroupNameToBindingIndex.FindOrAdd(InConstant.Identifier.GroupName) = InConstant.ComponentBindingIndex;
}

FOptimusKernelConstantContainer& FOptimusConstantContainer::AddContainerForKernel()
{
	return KernelContainers.AddDefaulted_GetRef();
}



void FOptimusConstantContainer::Reset()
{
	KernelContainers.Reset();
}


bool FOptimusConstantContainerInstance::Initialize(const FOptimusConstantContainer& InContainer,
                                                   const TMap<int32, TMap<FName, TArray<float>>>& InBindingIndexToConstantValues)
{
	for (int32 KernelIndex = 0; KernelIndex < InContainer.KernelContainers.Num(); KernelIndex++)
	{
		const FOptimusKernelConstantContainer& KernelContainer = InContainer.KernelContainers[KernelIndex];

		// Process Input Constants
		for (const FOptimusConstant& InputConstant : KernelContainer.InputConstants)
		{
			// Constant that directly grab value from another constant 
			if (InputConstant.Definition.ReferencedConstant.IsValid())
			{
				ConstantToValuePerInvocation.Add(InputConstant.Identifier) = GetConstantValuePerInvocation(InputConstant.Definition.ReferencedConstant);
				continue;
			}

			// Otherwise evaluate the current constant
			if (ensure(!InputConstant.Definition.Expression.IsEmpty()))
			{
				TMap<FName, float> ValuePerDependentConstant;
				TMap<FName, TArray<float>> ValuePerInvocationPerDependentConstant;
				
				if (!EvaluateAndSaveResult(
					InputConstant.Identifier,
					InputConstant.Definition.Expression,
					InBindingIndexToConstantValues[InputConstant.ComponentBindingIndex]
					))
				{
					return false;
				}
			}
		}

		// Process Output Constants
		
		// Output constants can access binding constants of all group bindings
		TMap<FName, TArray<float>> BindingConstantsForOutput;

		for (const TPair<FName, int32>& Group : KernelContainer.GroupNameToBindingIndex)
		{
			const FName& GroupName = Group.Key;
			const int32 GroupBinding = Group.Value;
			
			const TMap<FName, TArray<float>>& BindingConstantValues = InBindingIndexToConstantValues[GroupBinding];
			
			for (const TPair<FName, TArray<float>>& BindingConstantValue : BindingConstantValues)
			{
				const FName& ConstantName = BindingConstantValue.Key;
		
				if (GroupName == NAME_None)
				{
					BindingConstantsForOutput.Add(BindingConstantValue);
				}
				else
				{
					FName LocalConstantName = *(GroupName.ToString() + TEXT(".") + ConstantName.ToString());

					float Sum = 0;
					for ( const float& Value : BindingConstantValue.Value)
					{
						Sum += Value;
					}
					
					BindingConstantsForOutput.Add(LocalConstantName) = {Sum};
				}
			}
		}
		
		for (const FOptimusConstant& OutputConstant : KernelContainer.OutputConstants)
		{
			// Constants that directly grab value from another constant 
			if (OutputConstant.Definition.ReferencedConstant.IsValid())
			{
				ConstantToValuePerInvocation.Add(OutputConstant.Identifier) = GetConstantValuePerInvocation(OutputConstant.Definition.ReferencedConstant);
				continue;
			}

			// Otherwise evaluate the current constant
			if (ensure(!OutputConstant.Definition.Expression.IsEmpty()))
			{
				TMap<FName, TArray<float>> ValuePerInvocationPerDependentConstant = BindingConstantsForOutput;

				for (const FOptimusConstant& InputConstant : KernelContainer.InputConstants)
				{
					const TArray<float>& InputValuePerInvocation = GetConstantValuePerInvocation(InputConstant.Identifier);
					FName InputConstantLocalName = InputConstant.Identifier.GetLocalConstantName();

					if (InputConstant.Identifier.GroupName == OutputConstant.Identifier.GroupName)
					{
						ValuePerInvocationPerDependentConstant.Add(InputConstantLocalName) = InputValuePerInvocation;	
					}
					else
					{
						float Sum = 0;
						for (const float& Value : InputValuePerInvocation)
						{
							Sum+=Value;
						}
								
						ValuePerInvocationPerDependentConstant.Add(InputConstantLocalName) = {Sum};	
					}
				}
				
				if (!EvaluateAndSaveResult(
					OutputConstant.Identifier,
					OutputConstant.Definition.Expression,
					ValuePerInvocationPerDependentConstant
					))
				{
					return false;
				}
			}
		}
	}

	return true;
}

TArray<float> FOptimusConstantContainerInstance::GetConstantValuePerInvocation(const FOptimusConstantIdentifier& InIdentifier) const
{
	if (const TArray<float>* Values = ConstantToValuePerInvocation.Find(InIdentifier))
	{
		return *Values;
	}

	return {};
}

bool FOptimusConstantContainerInstance::EvaluateAndSaveResult(const FOptimusConstantIdentifier InIdentifier,
	const FString& InExpression, const TMap<FName, TArray<float>>& InDependentConstants)
{
	TMap<FName, float> DependentConstantToValue;

	int32 NumInvocations = 1;
	for (const TPair<FName, TArray<float>>& DependentConstant : InDependentConstants)
	{
		DependentConstantToValue.Add(DependentConstant.Key, 0);

		check(DependentConstant.Value.Num() != 0);
							
		if (DependentConstant.Value.Num() > 1)
		{
			if (NumInvocations == 1)
			{
				NumInvocations = DependentConstant.Value.Num();
			}
			else if (!ensure(NumInvocations == DependentConstant.Value.Num()))
			{
				return false;
			}
		}
	}

	using namespace Optimus::Expression;

	FEngine Engine;
	TVariant<FExpressionObject, FParseError> ParseResult = Engine.Parse(InExpression, [DependentConstantToValue](FName InName)->TOptional<float>
	{
		if (const float* Value = DependentConstantToValue.Find(InName))
		{
			return *Value;
		};

		return {};
	});

	if (ParseResult.IsType<FParseError>())
	{
		return false;
	}

	TArray<float> ValuePerInvocation;
	ValuePerInvocation.Reset(NumInvocations);
	
	for (int32 Index = 0; Index < NumInvocations; Index++)
	{
		for (TPair<FName, float>& DependentConstant: DependentConstantToValue)
		{
			const FName ConstantName = DependentConstant.Key;
			float& Value = DependentConstant.Value;
					
			const TArray<float>& Values = InDependentConstants[ConstantName];
			if (ensure(Values.Num() >= 1))
			{
				Value = Values.IsValidIndex(Index) ? Values[Index] : Values[0];
			}
			else
			{
				return false;
			}
		}
					
		const float Result = Engine.Execute(ParseResult.Get<FExpressionObject>(),
			[DependentConstantToValue](FName InName)->TOptional<float>
			{
				if (const float* Value = DependentConstantToValue.Find(InName))
				{
					return *Value;
				};

				return {};
			});
		
		ValuePerInvocation.Add(Result);
	}

	ConstantToValuePerInvocation.Add(InIdentifier) = ValuePerInvocation;

	return true;
}

