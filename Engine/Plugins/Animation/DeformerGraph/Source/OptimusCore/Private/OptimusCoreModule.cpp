// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusCoreModule.h"

#include "Interfaces/IPluginManager.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusObjectVersion.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"
#include "UObject/DevObjectVersion.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Nodes/OptimusNode_FunctionReference.h"
#include "RigVMCore/RigVMRegistry.h"

// Unique serialization id for Optimus .
const FGuid FOptimusObjectVersion::GUID(0x93ede1aa, 0x10ca7375, 0x4df98a28, 0x49b157a0);

static FDevVersionRegistration GRegisterOptimusObjectVersion(FOptimusObjectVersion::GUID, FOptimusObjectVersion::LatestVersion, TEXT("Dev-Optimus"));

void FOptimusCoreModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("DeformerGraph"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/Optimus"), PluginShaderDir);

	// Make sure all our types are known at startup.
	FOptimusDataTypeRegistry::RegisterBuiltinTypes();
	FOptimusDataTypeRegistry::RegisterEngineCallbacks();
	UOptimusComputeDataInterface::RegisterAllTypes();

	FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UOptimusDeformer::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
}

void FOptimusCoreModule::ShutdownModule()
{
	FOptimusDataTypeRegistry::UnregisterEngineCallbacks();
	FOptimusDataTypeRegistry::UnregisterAllTypes();
	
}


bool FOptimusCoreModule::RegisterDataInterfaceClass(TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass)
{
	if (InDataInterfaceClass)
	{
		UOptimusComputeDataInterface* DataInterface = Cast<UOptimusComputeDataInterface>(InDataInterfaceClass->GetDefaultObject());
		if (ensure(DataInterface))
		{
			DataInterface->RegisterTypes();
			return true;
		}
	}
	return false;
}


IMPLEMENT_MODULE(FOptimusCoreModule, OptimusCore)

DEFINE_LOG_CATEGORY(LogOptimusCore);
