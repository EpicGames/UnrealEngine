// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusCoreModule.h"
#include "OptimusFunctionNodeGraphHeader.h"


class FOptimusCoreModule : public IOptimusCoreModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;


	/** IOptimusCoreModule implementation */
	bool RegisterDataInterfaceClass(TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass) override;

};

DECLARE_LOG_CATEGORY_EXTERN(LogOptimusCore, Log, All);
