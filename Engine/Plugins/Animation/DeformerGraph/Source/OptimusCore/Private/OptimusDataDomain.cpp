// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataDomain.h"

#include "OptimusExecutionDomain.h"
#include "OptimusExpressionEvaluator.h"
#include "OptimusObjectVersion.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataDomain)


namespace Optimus::DomainName
{
	const FName Singleton("Singleton");
	const FName Vertex("Vertex");
	const FName Triangle("Triangle");
	const FName Bone("Bone");
	const FName UVChannel("UVChannel");
	const FName Index0("Index0");
}

FString Optimus::FormatDimensionNames(const TArray<FName>& InNames)
{
	TArray<FString> NameParts;
	for (FName Name: InNames)
	{
		NameParts.Add(Name.ToString());
	}
	return FString::Join(NameParts, TEXT(" › "));
}


FOptimusDataDomain::FOptimusDataDomain(const FOptimusExecutionDomain& InExecutionDomain)
{
	if (InExecutionDomain.Type == EOptimusExecutionDomainType::DomainName)
	{
		Type = EOptimusDataDomainType::Dimensional;
		DimensionNames = {InExecutionDomain.Name};
	}
	else if (InExecutionDomain.Type == EOptimusExecutionDomainType::Expression)
	{
		Type = EOptimusDataDomainType::Expression;
		Expression = InExecutionDomain.Expression;
	}
	else
	{
		checkNoEntry();
	}
}

FString FOptimusDataDomain::ToString() const
{
	switch(Type)
	{
	case EOptimusDataDomainType::Dimensional:
		{
			if (DimensionNames.IsEmpty())
			{
				return TEXT("Empty");
			}
			else if (DimensionNames.Num() == 1)
			{
				FString Result;
				for (FName Name: DimensionNames)
				{
					Result += Name.ToString();
					Result += TEXT("/");
				}
				Result += FString::FromInt(Multiplier);
				return Result;
			}
		}
		break;
	case EOptimusDataDomainType::Expression:
		return FString::Printf(TEXT("'%s'"), *Expression);
	}
	
	return {};
}


FOptimusDataDomain FOptimusDataDomain::FromString(
	const FString& InString
	)
{
	FString String = InString.TrimStartAndEnd();
	if (String.StartsWith(TEXT("'")))
	{
		return FOptimusDataDomain(String.Mid(1, String.Len() - 2));
	}

	TArray<FStringView> Parts;
	UE::String::ParseTokens(InString, TEXT("/"), Parts);
	if (Parts.Num() < 2)
	{
		return {};
	}

	const int32 Multiplier = FCString::Atoi(*FString(Parts.Pop()));
	TArray<FName> Names;
	for (FStringView Part: Parts)
	{
		Names.Add(FName(Part));
	}
	return FOptimusDataDomain(Names, Multiplier);
}


bool FOptimusDataDomain::operator==(const FOptimusDataDomain& InOtherDomain) const
{
	if (Type != InOtherDomain.Type)
	{
		return false;
	}

	switch(Type)
	{
	case EOptimusDataDomainType::Dimensional:
		return DimensionNames == InOtherDomain.DimensionNames && Multiplier == InOtherDomain.Multiplier;
	case EOptimusDataDomainType::Expression:
		return TStringView(Expression).TrimStartAndEnd().Compare(TStringView(InOtherDomain.Expression).TrimStartAndEnd()) == 0;
	}

	return false;
}

void FOptimusDataDomain::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() &&
		Ar.CustomVer(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::DataDomainExpansion)
	{
		BackCompFixupLevels();
	}
}


void FOptimusDataDomain::BackCompFixupLevels()
{
	if (!LevelNames_DEPRECATED.IsEmpty())
	{
		DimensionNames = LevelNames_DEPRECATED;
		LevelNames_DEPRECATED.Reset();
	}
}

TOptional<FString> FOptimusDataDomain::AsExpression() const
{
	if (IsSingleton())
	{
		return {};
	}

	if (IsMultiDimensional())
	{
		return {};
	}

	if (Type == EOptimusDataDomainType::Dimensional)
	{
		if (Multiplier > 1)
		{
			return FString::Printf(TEXT("%s * %d"), *DimensionNames[0].ToString(), Multiplier);
		}
		
		return DimensionNames[0].ToString();
	}

	if (Type == EOptimusDataDomainType::Expression)
	{
		return Expression;
	}

	return {};
}

TSet<FName> FOptimusDataDomain::GetUsedConstants() const
{
	if (ensure(Type == EOptimusDataDomainType::Expression))
	{
		using namespace Optimus::Expression;
		FEngine Engine;
		TVariant<FExpressionObject, FParseError> ParseResult = Engine.Parse(Expression);

		if (ParseResult.IsType<FParseError>())
		{
			return {};
		}

		return ParseResult.Get<FExpressionObject>().GetUsedConstants();
	}

	return {};
}

FString FOptimusDataDomain::GetDisplayName() const
{
	switch(Type)
	{
	case EOptimusDataDomainType::Dimensional:
		{
			if (DimensionNames.IsEmpty())
			{
				return TEXT("Parameter");
			}
			TArray<FString> Names;
			for (FName DomainLevelName: DimensionNames)
			{
				Names.Add(DomainLevelName.ToString());
			}
			FString DomainName = FString::Join(Names, *FString(UTF8TEXT(" › ")));
			if (Multiplier > 1)
			{
				DomainName += FString::Printf(TEXT(" x %d"), Multiplier);
			}
			return DomainName;
		}
		
	case EOptimusDataDomainType::Expression:
		if (Expression.IsEmpty())
		{
			return TEXT("Undefined");
		}
		return FString::Printf(TEXT("'%s'"), *Expression.TrimStartAndEnd());
	}

	checkNoEntry();
	return TEXT("");	
}

bool FOptimusDataDomain::AreCompatible(const FOptimusDataDomain& InOutput, const FOptimusDataDomain& InInput,
                                       FString* OutReason)
{
	if (!InOutput.IsFullyDefined() || !InInput.IsFullyDefined())
	{
		if (OutReason)
		{
			*OutReason = TEXT("One of the pins has undefined datadomain");
		}
		return false;
	}

	// We don't allow resource -> value connections. All other combos are legit. 
	// Value -> Resource just means the resource gets filled with the value.
	if (!InOutput.IsSingleton() && InInput.IsSingleton())
	{
		if (OutReason)
		{
			*OutReason = TEXT("Can't connect a resource output into a value input.");
		}
		return false;
	}

	// If it's resource -> resource, check that the domains are compatible
	if (!InOutput.IsSingleton() && !InInput.IsSingleton())
	{
		if (InOutput != InInput)
		{
			if (OutReason)
			{
				*OutReason = FString::Printf(TEXT("Can't connect resources with incompatible data domain types (%s vs %s)."),
					*InOutput.GetDisplayName(), *InInput.GetDisplayName());
			}
			return false;
		}
	}

	return true;
}


