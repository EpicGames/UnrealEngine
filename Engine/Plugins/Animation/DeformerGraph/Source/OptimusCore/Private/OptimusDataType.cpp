// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataType.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDataType)



FOptimusDataTypeRef::FOptimusDataTypeRef(
	FOptimusDataTypeHandle InTypeHandle
	)
{
	Set(InTypeHandle);
}


void FOptimusDataTypeRef::Set(
	FOptimusDataTypeHandle InTypeHandle
	)
{
	if (InTypeHandle.IsValid())
	{
		TypeName = InTypeHandle->TypeName;
		TypeObject = InTypeHandle->TypeObject.Get();
		checkSlow(FOptimusDataTypeRegistry::Get().FindType(TypeName) != nullptr);
	}
	else
	{
		TypeName = NAME_None;
		TypeObject.Reset();
	}
}


FOptimusDataTypeHandle FOptimusDataTypeRef::Resolve() const
{
	FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	FOptimusDataTypeHandle TypeHandle = Registry.FindType(TypeName);
	
	// This can happen during asset load, at which point the registry may not have been initialized yet
	// so we have to register these types on demand.
	if (!TypeHandle.IsValid())
	{
		if (TypeObject.LoadSynchronous())
		{
			if (Registry.RegisterStructType(Cast<UScriptStruct>(TypeObject.Get())))
			{
				TypeHandle = Registry.FindType(TypeName);
			}
		}
	}
	
	return TypeHandle;
}


void FOptimusDataTypeRef::PostSerialize(const FArchive& Ar)
{
	// Fix up data types so that the type points to the type object.
	if (Ar.IsLoading())
	{
		const FOptimusDataTypeHandle TypeHandle = FOptimusDataTypeRegistry::Get().FindType(TypeName);
		if (TypeHandle.IsValid())
		{
			TypeObject = TypeHandle->TypeObject.Get();
		}
	}
}


FProperty* FOptimusDataType::CreateProperty(
	UStruct* InScope, 
	FName InName
	) const
{
	const FOptimusDataTypeRegistry::PropertyCreateFuncT PropertyCreateFunc =
		FOptimusDataTypeRegistry::Get().FindPropertyCreateFunc(TypeName);

	if (PropertyCreateFunc)
	{
		return PropertyCreateFunc(InScope, InName);
	}
	else
	{
		return nullptr;
	}
}


bool FOptimusDataType::ConvertPropertyValueToShader(
	TArrayView<const uint8> InValue,
	FShaderValueContainer& OutConvertedValue
	) const
{
	const FOptimusDataTypeRegistry::PropertyValueConvertFuncT PropertyConversionFunc =
		FOptimusDataTypeRegistry::Get().FindPropertyValueConvertFunc(TypeName);
	if (PropertyConversionFunc)
	{
		return PropertyConversionFunc(InValue, OutConvertedValue);
	}
	else
	{
		return false;
	}
}

FShaderValueContainer FOptimusDataType::MakeShaderValue() const
{
	return FShaderValueContainer(ShaderValueSize, GetNumArrays());
}

bool FOptimusDataType::CanCreateProperty() const
{
	return static_cast<bool>(FOptimusDataTypeRegistry::Get().FindPropertyCreateFunc(TypeName));
}

int32 FOptimusDataType::GetNumArrays() const
{
	const TArray<FOptimusDataTypeRegistry::FArrayMetadata> ArrayMetadata =
		FOptimusDataTypeRegistry::Get().FindArrayMetadata(TypeName);
	
	return ArrayMetadata.Num();
}

int32 FOptimusDataType::GetArrayShaderValueOffset(int32 InArrayIndex) const
{
	const TArray<FOptimusDataTypeRegistry::FArrayMetadata> ArrayMetadata =
		FOptimusDataTypeRegistry::Get().FindArrayMetadata(TypeName);

	if (ensure(ArrayMetadata.IsValidIndex(InArrayIndex)))
	{
		return ArrayMetadata[InArrayIndex].ShaderValueOffset;
	}

	return INDEX_NONE;
}

int32 FOptimusDataType::GetArrayElementShaderValueSize(int32 InArrayIndex) const
{
	const TArray<FOptimusDataTypeRegistry::FArrayMetadata> ArrayMetadata =
		FOptimusDataTypeRegistry::Get().FindArrayMetadata(TypeName);

	if (ensure(ArrayMetadata.IsValidIndex(InArrayIndex)))
	{
		return ArrayMetadata[InArrayIndex].ElementShaderValueSize;
	}

	return INDEX_NONE;
}

bool FOptimusDataType::IsArrayType() const
{ 
	return TypeName.ToString().StartsWith("TArray<");
}

