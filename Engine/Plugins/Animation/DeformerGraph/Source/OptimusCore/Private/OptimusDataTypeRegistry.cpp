// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDataTypeRegistry.h"

#include "AssetRegistry/AssetData.h"
#include "OptimusCoreModule.h"
#include "OptimusHelpers.h"
#include "OptimusComponentSource.h"

#include "ShaderParameterMetadata.h"
#include "Animation/AttributeTypes.h"
#include "Animation/BuiltInAttributeTypes.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ComputeFramework/ComputeMetadataBuilder.h"

#include "StructUtils/UserDefinedStruct.h"

#include "UObject/UnrealType.h"

const TCHAR* FOptimusDataTypeRegistry::Matrix34TypeName = TEXT("3x4 float");

static const TMap<FName, UScriptStruct*>& GetBuiltInAttributeTypes()
{
	static const TMap<FName, UScriptStruct*> BuiltInAttributeTypes =
	{
		{FFloatProperty::StaticClass()->GetFName(), FFloatAnimationAttribute::StaticStruct()},
		{FIntProperty::StaticClass()->GetFName(), FIntegerAnimationAttribute::StaticStruct()},
		{*TBaseStructure<FTransform>::Get()->GetStructCPPName(), FTransformAnimationAttribute::StaticStruct()},
		{*TBaseStructure<FVector>::Get()->GetStructCPPName(), FVectorAnimationAttribute::StaticStruct()},
		{*TBaseStructure<FQuat>::Get()->GetStructCPPName(), FQuaternionAnimationAttribute::StaticStruct()},
	};

	return BuiltInAttributeTypes;
}

static bool IsStructHashable(const UScriptStruct* InStructType)
{
	if (InStructType->IsNative())
	{
		return InStructType->GetCppStructOps() && InStructType->GetCppStructOps()->HasGetTypeHash();
	}
	else
	{
		for (TFieldIterator<FProperty> It(InStructType); It; ++It)
		{
			if (CastField<FBoolProperty>(*It))
			{
				continue;
			}
			else if (!It->HasAllPropertyFlags(CPF_HasGetValueTypeHash))
			{
				return false;
			}
		}
		return true;
	}
}

template<typename SourceT, typename DestT>
static bool ConvertPropertyValuePOD(
		TArrayView<const uint8> InRawValue, FShaderValueContainerView OutShaderValue
	)
{
	if (ensure(InRawValue.Num() == sizeof(SourceT)) &&
		ensure(OutShaderValue.ShaderValue.Num() == sizeof(DestT)))
	{
		*reinterpret_cast<DestT*>(OutShaderValue.ShaderValue.GetData()) = static_cast<DestT>(*reinterpret_cast<const SourceT*>(InRawValue.GetData()));
		return true;
	}
	return false;
}

// Special logic for things like StructuredBuffer<float3> which should be packed as buffer of float4 in Vulkan
static FOptimusDataTypeHandle GetArrayElementDataTypeForStructuredBuffer(FOptimusDataTypeHandle InDataType)
{
	FOptimusDataTypeHandle Result = InDataType;
	if (InDataType->ShaderValueType->Type != EShaderFundamentalType::Struct && InDataType->ShaderValueType->DimensionType == EShaderFundamentalDimensionType::Vector && InDataType->ShaderValueType->VectorElemCount == 3)
	{
		Result = FOptimusDataTypeRegistry::Get().FindType(FShaderValueType::Get(InDataType->ShaderValueType->Type, 4));
		
		if (!ensureMsgf(Result.IsValid(), TEXT("Cannot find element type for structured buffer: %s"), *(InDataType->TypeName.ToString())))
		{
			return InDataType;
		}
	}

	return Result;
}

EOptimusDataTypeUsageFlags FOptimusDataTypeRegistry::GetStructTypeUsageFlag(UScriptStruct* InStruct)
{
	struct FTypeValidator 
    {
        EOptimusDataTypeUsageFlags WalkStruct(UScriptStruct* InStruct)
        {
        	// Rotator cannot be mapped to a shader struct member at the moment
        	if (InStruct == TBaseStructure<FRotator>::Get())
        	{
        		return EOptimusDataTypeUsageFlags::None;
        	}
        	
        	FOptimusDataTypeHandle DataType = Get().FindType(InStruct);
        	if (DataType.IsValid())
        	{
        		PropertyValueConvertFuncT ConvertFunc = Get().FindPropertyValueConvertFunc(DataType->TypeName);
        		if (!ConvertFunc)
        		{
        			return EOptimusDataTypeUsageFlags::None;
        		}
        		
        		return ApplicableFlags & DataType->UsageFlags;
        	}

        	if (!InStruct->ChildProperties)
        	{
        		// Empty struct is not support
        		return EOptimusDataTypeUsageFlags::None;
        	}

        	TSet<FName> PropertyNamesSeen;
        	
            for (const FProperty* Property : TFieldRange<FProperty>(InStruct))
            {
				if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStruct))
				{
					FName ShaderMemberName = Optimus::GetMemberPropertyShaderName(UserDefinedStruct, Property);
					if (PropertyNamesSeen.Contains(ShaderMemberName))
					{
						return EOptimusDataTypeUsageFlags::None;
					}

					PropertyNamesSeen.Add(ShaderMemberName);
				}
            	
                if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
                {
                    if (WalkStruct(StructProperty->Struct) == EOptimusDataTypeUsageFlags::None)
                    {
                        return EOptimusDataTypeUsageFlags::None;
                    }
                }
                else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
                {
                    if (WalkArray(ArrayProperty) == EOptimusDataTypeUsageFlags::None)
                    {
                        return EOptimusDataTypeUsageFlags::None;   
                    }
                }
                else if (CheckLeaf(Property) == EOptimusDataTypeUsageFlags::None)
                {
                    return EOptimusDataTypeUsageFlags::None;
                }
            }
        	
            return ApplicableFlags;
        }
		
        EOptimusDataTypeUsageFlags WalkArray(const FArrayProperty* InArrayProperty)
        {
        	EnumRemoveFlags(ApplicableFlags, EOptimusDataTypeUsageFlags::Resource);
        	
            if (!bFoundArray)
            {
            	TGuardValue<bool> NestedArrayGuard(bFoundArray, true);
            }
            else
            {
                // Nested array is not supported
                return EOptimusDataTypeUsageFlags::None;
            }
            
            FProperty* Inner = InArrayProperty->Inner;
            if (const FStructProperty* StructProperty = CastField<FStructProperty>(Inner))
            {
            	if (WalkStruct(StructProperty->Struct) == EOptimusDataTypeUsageFlags::None)
            	{
            		return EOptimusDataTypeUsageFlags::None;
            	}
            }
            else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Inner))
            {
            	if (WalkArray(ArrayProperty) == EOptimusDataTypeUsageFlags::None)
            	{
            		return EOptimusDataTypeUsageFlags::None;   
            	}
            }
                
        	else if (CheckLeaf(Inner) == EOptimusDataTypeUsageFlags::None)
        	{
        		return EOptimusDataTypeUsageFlags::None;
        	}

        	return ApplicableFlags;
        }
		
        EOptimusDataTypeUsageFlags CheckLeaf(const FProperty* InLeafProperty) const
        {
            FOptimusDataTypeHandle DataType = Get().FindType(*InLeafProperty);
			if (!DataType.IsValid())
			{
            	return EOptimusDataTypeUsageFlags::None;
			}
        	
			// Check for special cases that we cannot handle for now
			if (DataType->TypeName == FBoolProperty::StaticClass()->GetFName())
			{
				return EOptimusDataTypeUsageFlags::None;
			}

			PropertyValueConvertFuncT ConvertFunc = Get().FindPropertyValueConvertFunc(DataType->TypeName);
			if (!ConvertFunc)
			{
				return EOptimusDataTypeUsageFlags::None;
			}
            	
        	return ApplicableFlags & DataType->UsageFlags;
        }
		
        bool bFoundArray = false;

		EOptimusDataTypeUsageFlags ApplicableFlags =
			EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType;
    };

	if (FOptimusDataTypeHandle DataType = Get().FindType(InStruct))
	{
		return DataType->UsageFlags;
	}

	FTypeValidator Validator;
	return Validator.WalkStruct(InStruct);
}


void FOptimusDataTypeRegistry::RegisterBuiltinTypes()
{
	// Register standard UE types and their mappings to the compute framework types.
	FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	// NOTE: The pin categories should match the PC_* ones in EdGraphSchema_K2.cpp for the 
	// fundamental types.
	// FIXME: Turn this into an array and separate out to own file.
	// Data subpin in general is not currently supported, disable for now
	constexpr bool bShowElements = false; // true;
	constexpr bool bHideElements = false;


	// bool -> bool
	Registry.RegisterType(
	    *FBoolProperty::StaticClass(),
	    FText::FromString(TEXT("Bool")),
	    FShaderValueType::Get(EShaderFundamentalType::Bool),
		[](FFieldVariant InScope, FName InName) {
		    FBoolProperty* Property = new FBoolProperty(InScope, InName);
		    Property->SetBoolSize(sizeof(bool), true);
			return Property;
		},
		ConvertPropertyValuePOD<bool, int32>,
		FName(TEXT("bool")), {},
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// int -> int
	Registry.RegisterType(
	    *FIntProperty::StaticClass(),
	    FText::FromString(TEXT("Int")),
	    FShaderValueType::Get(EShaderFundamentalType::Int),
	    [](FFieldVariant InScope, FName InName) {
		    FIntProperty* Property = new FIntProperty(InScope, InName);
			Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
		    return Property;
	    },
		ConvertPropertyValuePOD<int32, int32>,
		FName(TEXT("int")), {}, 
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);

	// FIntPoint -> int2
	Registry.RegisterType(
		TBaseStructure<FIntPoint>::Get(),
	    FText::FromString(TEXT("Int Vector 2")),
		FShaderValueType::Get(EShaderFundamentalType::Int, 2),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FIntVector -> int3
	Registry.RegisterType(
		TBaseStructure<FIntVector>::Get(),
	    FText::FromString(TEXT("Int Vector 3")),	 
		FShaderValueType::Get(EShaderFundamentalType::Int, 3),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FIntVector4 -> int4
	Registry.RegisterType(
		TBaseStructure<FIntVector4>::Get(),
		FShaderValueType::Get(EShaderFundamentalType::Int, 4),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// uint -> uint
	Registry.RegisterType(
		*FUInt32Property::StaticClass(),
	    FText::FromString(TEXT("UInt")),
		FShaderValueType::Get(EShaderFundamentalType::Uint),
		[](FFieldVariant InScope, FName InName) {
			FUInt32Property* Property = new FUInt32Property(InScope, InName);
			Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
			return Property;
		},
		ConvertPropertyValuePOD<uint32, uint32>,
		FName(TEXT("uint")), FLinearColor(0.0275f, 0.733f, 0.820f, 1.0f), 
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	/* FIXME: Need the corresponding definitions in UObject/Class.h @ line 3537
	// FIntPoint -> int2
	Registry.RegisterType(
		TBaseStructure<FUintPoint>::Get(),
	    FText::FromString(TEXT("UInt Vector 2")),
		FShaderValueType::Get(EShaderFundamentalType::Uint, 2),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable);

	// FIntVector -> int3
	Registry.RegisterType(
		TBaseStructure<FUintVector>::Get(),
	    FText::FromString(TEXT("UInt Vector 3")),	 
		FShaderValueType::Get(EShaderFundamentalType::Uint, 3),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable);

	// FIntVector4 -> int4
	Registry.RegisterType(
		TBaseStructure<FUintVector4>::Get(),
		FShaderValueType::Get(EShaderFundamentalType::Uint, 4),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable);
	*/
	
	// float -> float
	Registry.RegisterType(
	    *FFloatProperty::StaticClass(),
	    FText::FromString(TEXT("Float")),
	    FShaderValueType::Get(EShaderFundamentalType::Float),
	    [](FFieldVariant InScope, FName InName) {
		    FFloatProperty* Property = new FFloatProperty(InScope, InName);
		    Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
#if WITH_EDITOR
	    	Property->SetMetaData(TEXT("UIMin"), TEXT("0.0"));
	    	Property->SetMetaData(TEXT("UIMax"), TEXT("1.0"));
	    	Property->SetMetaData(TEXT("SupportDynamicSliderMinValue"), TEXT("true"));
	    	Property->SetMetaData(TEXT("SupportDynamicSliderMaxValue"), TEXT("true"));
#endif
		    return Property;
	    },
		ConvertPropertyValuePOD<float, float>,
		FName(TEXT("real")), {}, 
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);

	// double -> float 
	Registry.RegisterType(
	    *FDoubleProperty::StaticClass(),
	    FText::FromString(TEXT("Float")),
	    FShaderValueType::Get(EShaderFundamentalType::Float),
	    [](FFieldVariant InScope, FName InName) {
		    FDoubleProperty* Property = new FDoubleProperty(InScope, InName);
		    Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
#if WITH_EDITOR
	    	Property->SetMetaData(TEXT("UIMin"), TEXT("0.0"));
			Property->SetMetaData(TEXT("UIMax"), TEXT("1.0"));
			Property->SetMetaData(TEXT("SupportDynamicSliderMinValue"), TEXT("true"));
			Property->SetMetaData(TEXT("SupportDynamicSliderMaxValue"), TEXT("true"));
#endif
			return Property;
	    },
		ConvertPropertyValuePOD<double, float>,
		FName(TEXT("real")), {}, 
	    EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FVector2D -> float2
	Registry.RegisterType(
	    TBaseStructure<FVector2D>::Get(),
		FText::FromString(TEXT("Vector 2")),
		FShaderValueType::Get(EShaderFundamentalType::Float, 2),
	    {},
	    bShowElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FVector -> float3
	Registry.RegisterType(
		TBaseStructure<FVector>::Get(),
		FText::FromString(TEXT("Vector 3")),
		FShaderValueType::Get(EShaderFundamentalType::Float, 3),
		{},
		bShowElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);

	// FVector4 -> float4
	Registry.RegisterType(
	    TBaseStructure<FVector4>::Get(),
		FText::FromString(TEXT("Vector 4")),
		FShaderValueType::Get(EShaderFundamentalType::Float, 4),
	    {},
	    bShowElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);
	
	// FLinearColor -> float4
	Registry.RegisterType(
	    TBaseStructure<FLinearColor>::Get(),
	    FShaderValueType::Get(EShaderFundamentalType::Float, 4),
	    {},
	    bShowElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FQuat -> float4
	Registry.RegisterType(
		TBaseStructure<FQuat>::Get(),
		FShaderValueType::Get(EShaderFundamentalType::Float, 4),
		{},
		bShowElements,
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);

	// FRotator -> float3x3
	Registry.RegisterType(
	    TBaseStructure<FRotator>::Get(),
	    FShaderValueType::Get(EShaderFundamentalType::Float, 3, 3),
	    {},
	    bShowElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FTransform -> float4x4
	Registry.RegisterType(
	    TBaseStructure<FTransform>::Get(),
	    FShaderValueType::Get(EShaderFundamentalType::Float, 4, 4),
	    [](
	    	TArrayView<const uint8> InRawValue,
			FShaderValueContainerView OutShaderValue) -> bool
	    {
	    	if (ensure(InRawValue.Num() == TBaseStructure<FTransform>::Get()->GetCppStructOps()->GetSize()) &&
				ensure(OutShaderValue.ShaderValue.Num() == FShaderValueType::Get(EShaderFundamentalType::Float, 4, 4)->GetResourceElementSize()))
	    	{
				uint8* ShaderValuePtr = OutShaderValue.ShaderValue.GetData();

				*((FMatrix44f*)(ShaderValuePtr)) = Optimus::ConvertFTransformToFMatrix44f(*((FTransform*)InRawValue.GetData()));
				return true;
			}
			return false; 
	    },
	    {},
	    bHideElements,
	    EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);

	// HLSL types
	Registry.RegisterType(
		Matrix34TypeName,
		FText::FromString(TEXT("Matrix 3x4")),
		FShaderValueType::Get(EShaderFundamentalType::Float, 3, 4),
		FName("float3x4"),
		nullptr,
		FLinearColor(0.7f, 0.3f, 0.4f, 1.0f),
		EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::DataInterfaceOutput | EOptimusDataTypeUsageFlags::PinType);

	// FIXME: Add type aliases (e.g. "3x4 Float" above should really be "float3x4")

	// Actor Component
	// ComponentSourceBinding pin does not carry concrete data, and is special
	Registry.RegisterType(
		UOptimusComponentSourceBinding::StaticClass(),
		FText::FromString("Component"),
		FLinearColor(0.3f, 0.3f, 0.4f, 1.0f),
		EOptimusDataTypeUsageFlags::PinType
		);

	// Name
	Registry.RegisterType(
		*FNameProperty::StaticClass(),
		FText::FromString(TEXT("Name")),
		FShaderValueTypeHandle(),
		[](FFieldVariant InScope, FName InName) {
			FNameProperty* Property = new FNameProperty(InScope, InName);
			return Property;
		},
		{},
		FName(TEXT("name")), {},
		EOptimusDataTypeUsageFlags::Property | EOptimusDataTypeUsageFlags::PinType);

	Registry.TypeWithAtomicSupport = {FIntProperty::StaticClass()->GetFName()};
	
	// Scan available built-in types to see if we can create array type for them
	// Currently only support variable usage + no nested array

	TMap<FName, FTypeInfo> AlreadyRegisteredTypes = Registry.RegisteredTypes;
	
	for (const TPair<FName, FTypeInfo>& Type : AlreadyRegisteredTypes)
	{
		// This should never happen theoretically, but somehow Linux complained about it once, hope this helps the next time it complains
		if (ensureMsgf(Type.Value.DataType.IsValid(), TEXT("Invalid data type for array type registration: %s"), *Type.Key.ToString()))
		{
			Registry.RegisterArrayTypeIfApplicable(Type.Value.DataType);
		}
	}
}


FOptimusDataTypeRegistry::~FOptimusDataTypeRegistry()
{
}


FName FOptimusDataTypeRegistry::GetTypeName(const FFieldClass& InFieldClass)
{
	return InFieldClass.GetFName();
}

FName FOptimusDataTypeRegistry::GetTypeName(UScriptStruct* InStruct)
{
	return Optimus::GetTypeName(InStruct, true);
}

FName FOptimusDataTypeRegistry::GetTypeName(const FAssetData& InStructAsset)
{
	return Optimus::GetTypeName(InStructAsset);
}

FName FOptimusDataTypeRegistry::GetArrayTypeName(const FFieldClass& InFieldClass)
{
	return GetArrayTypeName(GetTypeName(InFieldClass));
}

FName FOptimusDataTypeRegistry::GetArrayTypeName(UScriptStruct* InStruct)
{
	return GetArrayTypeName(GetTypeName(InStruct));
}

FName FOptimusDataTypeRegistry::GetArrayTypeName(FName InElementTypeName)
{
	return *(TEXT("TArray<") + InElementTypeName.ToString() + TEXT(">"));	
};

FOptimusDataTypeRegistry& FOptimusDataTypeRegistry::Get()
{
	static FOptimusDataTypeRegistry Singleton;

	return Singleton;
}

bool FOptimusDataTypeRegistry::RegisterType(
	FName InTypeName, 
	TFunction<void(FOptimusDataType&)> InFillFunc,
	PropertyCreateFuncT InPropertyCreateFunc,
	PropertyValueConvertFuncT InPropertyValueConvertFunc,
	TArray<FArrayMetadata> InArrayMetadata
	)
{
	if (InTypeName == NAME_None)
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid type name.");
		return false;
	}

	if (RegisteredTypes.Contains(InTypeName))
	{
		UE_LOGF(LogOptimusCore, Error, "Type '%ls' is already registered.", *InTypeName.ToString());
		return false;
	}

	TSharedRef<FOptimusDataType> DataType = MakeShared<FOptimusDataType>();

	const FTypeInfo Info
	{
		DataType,
		InPropertyCreateFunc,
		InPropertyValueConvertFunc,
		InArrayMetadata
	};
	
	InFillFunc(*DataType);
	RegisteredTypes.Add(InTypeName, Info);
	RegistrationOrder.Add(InTypeName);
	return true;
}

bool FOptimusDataTypeRegistry::RegisterStructType(UScriptStruct* InStructType)
{
	if (!InStructType)
	{
		return false;
	}

	EOptimusDataTypeUsageFlags UsageFlags = GetStructTypeUsageFlag(InStructType);

	// Disable the use of structs in variables and resources for now
	UsageFlags &= ~EOptimusDataTypeUsageFlags::Variable;
	UsageFlags &= ~EOptimusDataTypeUsageFlags::Resource;

	FText DisplayName = Optimus::GetTypeDisplayName(InStructType);

	FName TypeName = GetTypeName(InStructType);

	const bool bIsHashable = IsStructHashable(InStructType);

	PropertyCreateFuncT PropertyCreateFunc;
	PropertyCreateFunc = [bIsHashable, InStructType](FFieldVariant InScope, FName InName) -> FProperty *
	{
		auto Property = new FStructProperty(InScope, InName);
		Property->Struct = InStructType;
		Property->SetElementSize(InStructType->GetStructureSize());
		if (bIsHashable)
		{
			Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		return Property;
	};
	
	if (UsageFlags != EOptimusDataTypeUsageFlags::None)
	{
		struct FPropertyInfo
		{
			const FProperty* Property;
			FOptimusDataTypeHandle DataType;
			PropertyValueConvertFuncT ConvertFunc;
		};

		int32 NumArrayProperties = 0;

		TArray<FPropertyInfo> PropertyInfos;

		TArray<FShaderValueType::FStructElement> StructMembers;
		for (const FProperty* Property : TFieldRange<FProperty>(InStructType))
		{
			const FProperty* PropertyForValidation = Property;

			bool bIsArrayMember = false;

			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				bIsArrayMember = true;
				NumArrayProperties++;
				PropertyForValidation = ArrayProperty->Inner;
			}

			FOptimusDataTypeHandle DataType = FindType(*PropertyForValidation);

			if (!DataType.IsValid())
			{
				const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyForValidation);
				if (ensure(StructProperty))
				{
					RegisterStructType(StructProperty->Struct);
				}
			}

			DataType = FindType(*PropertyForValidation);

			// Special logic for things like StructuredBuffer<float3> which should be packed as buffer of float4 in Vulkan
			if (bIsArrayMember)
			{
				DataType = GetArrayElementDataTypeForStructuredBuffer(DataType);
			}
			
			if (DataType->GetNumArrays() > 0)
			{
				NumArrayProperties += DataType->GetNumArrays();
			}

			PropertyValueConvertFuncT ConvertFunc;

			ConvertFunc = FindPropertyValueConvertFunc(DataType->TypeName);

			FName PropertyShaderName = Optimus::GetMemberPropertyShaderName(InStructType, Property);
			
			FShaderValueType::FStructElement StructElement(PropertyShaderName, DataType->ShaderValueType);
			
			if (bIsArrayMember)
			{
				StructElement = {PropertyShaderName, FShaderValueType::MakeDynamicArrayType(DataType->ShaderValueType)};
			}
			
			StructMembers.Add(StructElement);
			
			FPropertyInfo PropertyInfo;
			PropertyInfo.Property = Property;
			PropertyInfo.DataType = DataType;
			PropertyInfo.ConvertFunc = ConvertFunc;
			PropertyInfos.Add(PropertyInfo);
		}
		
		FShaderValueTypeHandle ShaderValueType = FShaderValueType::Get(TypeName, StructMembers);

		
		PropertyValueConvertFuncT PropertyValueConvertFunc;
		int32 ExpectedShaderValueSize = 0;
		TArray<FArrayMetadata> ArrayMetadata;
		

		ComputeFramework::FTypeMetaData TypeMetaData(ShaderValueType);
		ExpectedShaderValueSize = TypeMetaData.Metadata->GetSize();
		int32 ExpectedPropertySize = 
			InStructType->GetCppStructOps() ? 
			InStructType->GetCppStructOps()->GetSize() : InStructType->GetStructureSize();


		// Cache shader value offsets of all array properties within the type
		ArrayMetadata.Reserve(NumArrayProperties);
		for (int32 Index = 0; Index < PropertyInfos.Num(); Index++)
		{
			const FProperty* Property = PropertyInfos[Index].Property;
			const FOptimusDataTypeHandle DataType = PropertyInfos[Index].DataType;
			const FShaderParametersMetadata::FMember& ShaderStructMember = TypeMetaData.Metadata->GetMembers()[Index];

			if (CastField<FArrayProperty>(Property))
			{
				FArrayMetadata Metadata;
				Metadata.ElementShaderValueSize= DataType->ShaderValueSize;
				Metadata.ShaderValueOffset = ShaderStructMember.GetOffset();
				
				ArrayMetadata.Add(Metadata);
			}
			else if (DataType->GetNumArrays() > 0)
			{
				for (int32 ArrayIndex = 0; ArrayIndex < DataType->GetNumArrays(); ArrayIndex++)
				{
					int32 ArrayOffset = DataType->GetArrayShaderValueOffset(ArrayIndex);
					
					FArrayMetadata Metadata;
					Metadata.ElementShaderValueSize = DataType->GetArrayElementShaderValueSize(ArrayIndex);
					Metadata.ShaderValueOffset = ShaderStructMember.GetOffset() + ArrayOffset;
					
					ArrayMetadata.Add(Metadata);
				}
			}
		}

		struct FPropertyConversionInfo
		{
			FPropertyInfo PropertyInfo;
			int32 ShaderValueOffset;
			int32 ShaderValueSize;
			int32 ShaderValueInlineSize;
			int32 ArrayIndex;
		};
		
		TArray<FPropertyConversionInfo> ConversionEntries;

		// Build conversion info
		int32 ArrayIndex = 0;
		for (int32 Index = 0; Index < PropertyInfos.Num(); Index++)
		{
			const FProperty* Property = PropertyInfos[Index].Property;
			const FOptimusDataTypeHandle DataType = PropertyInfos[Index].DataType;
			const FShaderParametersMetadata::FMember& ShaderStructMember = TypeMetaData.Metadata->GetMembers()[Index];

			FPropertyConversionInfo ConversionInfo;
			ConversionInfo.PropertyInfo = PropertyInfos[Index];
			ConversionInfo.ShaderValueOffset = ShaderStructMember.GetOffset();
			ConversionInfo.ShaderValueSize = DataType->ShaderValueSize;
			ConversionInfo.ArrayIndex = ArrayIndex;
			
			if (CastField<FArrayProperty>(Property))
			{
				ArrayIndex++;
			}
			else if (DataType->GetNumArrays() > 0)
			{
				ArrayIndex += DataType->GetNumArrays();
			}

			ConversionEntries.Add(ConversionInfo);
		}

		// Infer inline sizes from offsets
		for (int32 Index = 0; Index < ConversionEntries.Num(); Index++)
		{
			FPropertyConversionInfo& ConversionInfo = ConversionEntries[Index];

			if (Index + 1 < ConversionEntries.Num())
			{
				FPropertyConversionInfo& NextInfo = ConversionEntries[Index + 1];
				ConversionInfo.ShaderValueInlineSize = NextInfo.ShaderValueOffset - ConversionInfo.ShaderValueOffset;
			}
			else
			{
				ConversionInfo.ShaderValueInlineSize = ExpectedShaderValueSize - ConversionInfo.ShaderValueOffset;
			}
		}
		
		PropertyValueConvertFunc = [ConversionEntries, ExpectedPropertySize, ExpectedShaderValueSize](
				TArrayView<const uint8> InRawValue, FShaderValueContainerView OutShaderValue
			) -> bool
		{
			if (ensure(InRawValue.Num() == ExpectedPropertySize) &&
				ensure(OutShaderValue.ShaderValue.Num() == ExpectedShaderValueSize))
			{
				for (const FPropertyConversionInfo& ConversionInfo: ConversionEntries)
				{
					const FProperty* Property = ConversionInfo.PropertyInfo.Property;
					const uint8* PropertyRawValue = Property->ContainerPtrToValuePtr<uint8>(InRawValue.GetData());
					if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						check(ConversionInfo.ArrayIndex != INDEX_NONE);
						// Memzero the part in the shader struct where the buffer handle will be stored
						FMemory::Memzero(
							OutShaderValue.ShaderValue.GetData() + ConversionInfo.ShaderValueOffset,
							ConversionInfo.ShaderValueInlineSize);

						// Convert each element, store them in a separate buffer that is to be uploaded
						FScriptArrayHelper ArrayHelper(ArrayProperty, PropertyRawValue);
						TArray<uint8>& Buffer = OutShaderValue.ArrayList[ConversionInfo.ArrayIndex].ArrayOfValues;
						Buffer.AddZeroed(ConversionInfo.ShaderValueSize * ArrayHelper.Num());
						
						for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
						{
							uint8* ElementPtr = ArrayHelper.GetRawPtr(Index);
							uint8* ShaderValuePtr = Buffer.GetData() + ConversionInfo.ShaderValueSize * Index;

							// Nested buffer is not possible
							TArray<FArrayShaderValue> DummyBufferList;

							if (!ConversionInfo.PropertyInfo.ConvertFunc(
									{ElementPtr, ArrayProperty->Inner->GetSize()},
									{
										{ShaderValuePtr, ConversionInfo.ShaderValueSize},
										DummyBufferList
									}))
							{
								return false;
							}
						}	
					}
					else
					{
						
						const FOptimusDataTypeHandle DataType = ConversionInfo.PropertyInfo.DataType;
						
						check(ConversionInfo.ShaderValueSize <= ConversionInfo.ShaderValueInlineSize);

						if (!ConversionInfo.PropertyInfo.ConvertFunc(
								{
									PropertyRawValue,
									Property->GetSize()
								},
								{
									{
										OutShaderValue.ShaderValue.GetData() + ConversionInfo.ShaderValueOffset,
										ConversionInfo.ShaderValueSize
									},
									{
										OutShaderValue.ArrayList.GetData() + ConversionInfo.ArrayIndex,
										DataType->GetNumArrays() 
									}
								}))
						{
							return false;
						}
					}
				}
				return true;
			}
			return false;
		};

		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
			InDataType.TypeName = TypeName;
			InDataType.DisplayName = DisplayName;
			InDataType.ShaderValueType = ShaderValueType;
			InDataType.ShaderValueSize = ExpectedShaderValueSize;
			InDataType.TypeCategory = FName(TEXT("struct"));
			InDataType.TypeObject = InStructType;
			InDataType.UsageFlags = UsageFlags;
			InDataType.TypeFlags |= EOptimusDataTypeFlags::IsStructType;
			},
			PropertyCreateFunc,
			PropertyValueConvertFunc,
			ArrayMetadata);
	}

	// For user defined structs we still register them even if it is unsupported
	// since they can be changed to be compatible with our type system sometime later
	if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(InStructType))
	{
		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
				InDataType.TypeName = TypeName;
				InDataType.DisplayName = DisplayName;
				InDataType.TypeCategory = FName(TEXT("struct"));
				InDataType.TypeObject = InStructType;
				InDataType.UsageFlags = UsageFlags;
				InDataType.TypeFlags |= EOptimusDataTypeFlags::IsStructType;
				},
				PropertyCreateFunc
				);
	}

	return false;
}

bool FOptimusDataTypeRegistry::RegisterArrayTypeIfApplicable(FOptimusDataTypeHandle InElementDataType)
{
	check(InElementDataType.IsValid());
	// For now only allow array type for variables
	if (!EnumHasAnyFlags(InElementDataType->UsageFlags, EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::Property))
	{
		return false;
	}

	// Nested array is not supported at the moment for array variables
	if (InElementDataType->GetNumArrays() != 0)
	{
		return false;
	}
	
	PropertyCreateFuncT ElementPropertyCreateFunc = FindPropertyCreateFunc(InElementDataType->TypeName);
	PropertyCreateFuncT	ArrayPropertyCreateFunc = [ElementPropertyCreateFunc](FFieldVariant InScope, FName InName) -> FProperty*
	{
		FArrayProperty* ArrayProperty = new FArrayProperty(InScope, InName);
		ArrayProperty->Inner = ElementPropertyCreateFunc(ArrayProperty, TEXT("Inner"));
		
		return ArrayProperty;
	};
	
	PropertyValueConvertFuncT ArrayPropertyValueConvertFunc;
	TArray<FArrayMetadata> ArrayMetadata;
	FShaderValueTypeHandle ArrayShaderType;
	if (InElementDataType->ShaderValueType.IsValid())
	{
		// Making sure we are copying property value of a type into shader value of a equal or larger type, see comment for GetArrayElementDataTypeForStructuredBuffer
		FOptimusDataTypeHandle InnerDataTypeForStructuredBuffer = GetArrayElementDataTypeForStructuredBuffer(InElementDataType);
		// This should never happen theoretically, but somehow Linux complained about it, hope this helps the next time it complains
		if (!ensureMsgf(InnerDataTypeForStructuredBuffer.IsValid(), TEXT("Cannot find matching element type for array type registration: %s"), *(InElementDataType->TypeName.ToString())))
		{
			return false;
		}
		check(InnerDataTypeForStructuredBuffer->ShaderValueSize >= InElementDataType->ShaderValueSize);
		
		ArrayShaderType = FShaderValueType::MakeDynamicArrayType(InnerDataTypeForStructuredBuffer->ShaderValueType);
		
		PropertyValueConvertFuncT ElementPropertyValueConvertFunc = FindPropertyValueConvertFunc(InnerDataTypeForStructuredBuffer->TypeName);
		check(ElementPropertyValueConvertFunc);
		int32 ElementShaderValueSize = InnerDataTypeForStructuredBuffer->ShaderValueSize;

		// Nested array is not supported at the moment for array variables
		check(InnerDataTypeForStructuredBuffer->GetNumArrays() == 0);
		
		ArrayPropertyValueConvertFunc= [
			ArrayPropertyCreateFunc,
			ElementShaderValueSize,
			ElementPropertyValueConvertFunc
			] (
			TArrayView<const uint8> InRawValue,
			FShaderValueContainerView OutShaderValue
			) -> bool
		{
			const TUniquePtr<FArrayProperty> LocalArrayProperty(CastField<FArrayProperty>(ArrayPropertyCreateFunc(nullptr, NAME_None)));

			const FArrayProperty* ArrayProperty = LocalArrayProperty.Get();
			// Convert each element, store them in a separate buffer that is to be uploaded
			FScriptArrayHelper ArrayHelper(ArrayProperty, InRawValue.GetData());
			TArray<uint8>& Buffer = OutShaderValue.ArrayList[0].ArrayOfValues;
			Buffer.Reset();
			Buffer.AddZeroed(ElementShaderValueSize * ArrayHelper.Num());
					
			for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
			{
				uint8* ElementPtr = ArrayHelper.GetRawPtr(Index);
				uint8* ShaderValuePtr = Buffer.GetData() + ElementShaderValueSize * Index;

				// Nested buffer is not possible
				TArray<FArrayShaderValue> DummyBufferList;

				if (!ElementPropertyValueConvertFunc(
						{ElementPtr, ArrayProperty->Inner->GetSize()},
						{
							{ShaderValuePtr, ElementShaderValueSize},
							DummyBufferList
						}))
				{
					return false;
				}
			}	
			return true;
		};

		ArrayMetadata = {{ElementShaderValueSize, 0}};
	}
	

	TSharedRef<FOptimusDataType> ArrayDataType = MakeShared<FOptimusDataType>();
	
	const FTypeInfo Info
	{
		ArrayDataType,
		ArrayPropertyCreateFunc,
		ArrayPropertyValueConvertFunc,
		ArrayMetadata
	};

	{
		*ArrayDataType = *InElementDataType;
		ArrayDataType->TypeName = GetArrayTypeName(InElementDataType->TypeName);
		ArrayDataType->DisplayName = FText::FromString(InElementDataType->DisplayName.ToString() + TEXT(" Array"));
		ArrayDataType->ShaderValueType = ArrayShaderType;
		
		EnumRemoveFlags(ArrayDataType->UsageFlags, EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::AnimAttributes | EOptimusDataTypeUsageFlags::PerBoneAnimAttribute);
		
		// Unused field
		ArrayDataType->ShaderValueSize = 0;	
	}
	
	RegisteredTypes.Add(ArrayDataType->TypeName, Info);
	RegistrationOrder.Add(ArrayDataType->TypeName);

	return true;
}

void FOptimusDataTypeRegistry::RefreshStructType(UUserDefinedStruct* InStructType)
{
	FName TypeName = GetTypeName(InStructType);
	if (RegisteredTypes.Contains(TypeName))
	{
		UnregisterType(TypeName);
		RegisterStructType(InStructType);
		OnDataTypeChanged.Broadcast(TypeName);
	}
}

bool FOptimusDataTypeRegistry::RegisterType(
	const FFieldClass &InFieldType,
	const FText& InDisplayName,
	FShaderValueTypeHandle InShaderValueType,
    PropertyCreateFuncT InPropertyCreateFunc,
	PropertyValueConvertFuncT InPropertyValueConvertFunc,
	FName InPinCategory,
	TOptional<FLinearColor> InPinColor,
	EOptimusDataTypeUsageFlags InUsageFlags
	)
{
	return RegisterType(GetTypeName(InFieldType), [&](FOptimusDataType& InDataType) {
		InDataType.TypeName = GetTypeName(InFieldType);
		InDataType.DisplayName = InDisplayName;
		InDataType.ShaderValueType = InShaderValueType;
		InDataType.ShaderValueSize = InShaderValueType.IsValid() ? InShaderValueType->GetResourceElementSize() : 0;
		InDataType.TypeCategory = InPinCategory;
		if (InPinColor.IsSet())
		{
			InDataType.bHasCustomPinColor = true;
			InDataType.CustomPinColor = *InPinColor;
		}
		InDataType.UsageFlags = InUsageFlags;
	},
	InPropertyCreateFunc,
	InPropertyValueConvertFunc
	);
}

bool FOptimusDataTypeRegistry::RegisterType(
    UScriptStruct *InStructType,
    FShaderValueTypeHandle InShaderValueType,
	TOptional<FLinearColor> InPinColor,
	bool bInShowElements,
    EOptimusDataTypeUsageFlags InUsageFlags
	)
{
#if WITH_EDITOR
	FText DisplayName = InStructType->GetDisplayNameText();
#else
	FText DisplayName = FText::FromName(InStructType->GetFName());
#endif
	
	return RegisterType(InStructType, DisplayName, InShaderValueType, InPinColor, bInShowElements, InUsageFlags);
}

bool FOptimusDataTypeRegistry::RegisterType(
    UScriptStruct *InStructType,
	const FText& InDisplayName,
    FShaderValueTypeHandle InShaderValueType,
    TOptional<FLinearColor> InPinColor,
    bool bInShowElements,
    EOptimusDataTypeUsageFlags InUsageFlags
	)
{
	if (ensure(InStructType))
	{
		// If showing elements, the sub-elements have to be registered already.
		if (bInShowElements)
		{
			for (const FProperty* Property : TFieldRange<FProperty>(InStructType))
			{
				if (FindType(*Property) == nullptr)
				{
					UE_LOGF(LogOptimusCore, Error, "Found un-registered sub-element '%ls' when registering '%ls'",
						*(Property->GetClass()->GetName()), *InStructType->GetName());
					return false;
				}
			}
		}

		const FName TypeName = GetTypeName(InStructType);

		PropertyCreateFuncT PropertyCreateFunc;
		PropertyValueConvertFuncT PropertyValueConvertFunc;
		int32 ExpectedShaderValueSize = 0;
		
		if (EnumHasAnyFlags(InUsageFlags, EOptimusDataTypeUsageFlags::Variable))
		{
			const bool bIsHashable = IsStructHashable(InStructType);

			PropertyCreateFunc = [bIsHashable, InStructType](FFieldVariant InScope, FName InName) -> FProperty *
			{
				auto Property = new FStructProperty(InScope, InName);
				Property->Struct = InStructType;
				Property->SetElementSize(InStructType->GetStructureSize());
				if (bIsHashable)
				{
					Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
				}
				return Property;
			};

			struct FPropertyConversionInfo
			{
				PropertyValueConvertFuncT ConversionFunc;
				int32 PropertyOffset;
				int32 PropertySize;
				int32 ShaderValueSize;
			};
			
			TArray<FPropertyConversionInfo> ConversionEntries;
			int32 ExpectedPropertySize = InStructType->GetCppStructOps()->GetSize();
			
			for (const FProperty* Property : TFieldRange<FProperty>(InStructType))
			{
				FOptimusDataTypeHandle TypeHandle = FindType(*Property);
				if (!TypeHandle.IsValid())
				{
					UE_LOGF(LogOptimusCore, Error, "Found un-registered sub-element '%ls' when converting '%ls'",
						*(Property->GetClass()->GetName()), *InStructType->GetName());
					return false;
				}
				
				FPropertyConversionInfo ConversionInfo;

				ConversionInfo.ConversionFunc = FindPropertyValueConvertFunc(TypeHandle->TypeName);
				if (!ConversionInfo.ConversionFunc)
				{
					UE_LOGF(LogOptimusCore, Error, "Sub-element '%ls' has no conversion when converting '%ls'",
						*(Property->GetClass()->GetName()), *InStructType->GetName());
					return false;
				}

				ConversionInfo.PropertyOffset = Property->GetOffset_ForInternal();
				ConversionInfo.PropertySize = Property->GetSize();
				ConversionInfo.ShaderValueSize = TypeHandle->ShaderValueSize;
				
				ExpectedShaderValueSize += ConversionInfo.ShaderValueSize; 

				ConversionEntries.Add(ConversionInfo);
			}

			PropertyValueConvertFunc = [ConversionEntries, ExpectedPropertySize, ExpectedShaderValueSize](
				TArrayView<const uint8> InRawValue, 
				FShaderValueContainerView OutShaderValue
				) -> bool
			{
				// we can be copying a smaller property into a larger shader side array element
				// for example, FVector3 -> StructuredBuffer<float4>, see special logic in RegisterStructType
				if (ensure(InRawValue.Num() <= ExpectedPropertySize) && 
					ensure(OutShaderValue.ShaderValue.Num() == ExpectedShaderValueSize))
				{
					uint8* ShaderValuePtr = OutShaderValue.ShaderValue.GetData();
					for (const FPropertyConversionInfo& ConversionInfo: ConversionEntries)
					{
						if (ConversionInfo.PropertyOffset + ConversionInfo.PropertySize <= InRawValue.Num())
						{
							TArrayView<const uint8> PropertyData(InRawValue.GetData() + ConversionInfo.PropertyOffset, ConversionInfo.PropertySize);
							TArrayView<uint8> ShaderValueData(ShaderValuePtr, ConversionInfo.ShaderValueSize);
							if (!ConversionInfo.ConversionFunc(PropertyData, ShaderValueData))
							{
								return false;
							}

							ShaderValuePtr += ConversionInfo.ShaderValueSize;
						}
					}
					return true;
				}
				return false;
			};
		}

		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
			InDataType.TypeName = TypeName;
			InDataType.DisplayName = InDisplayName;
			InDataType.ShaderValueType = InShaderValueType;
			InDataType.ShaderValueSize = ExpectedShaderValueSize;
			InDataType.TypeCategory = FName(TEXT("struct"));
			InDataType.TypeObject = InStructType;
			if (InPinColor.IsSet())
			{
				InDataType.bHasCustomPinColor = true;
				InDataType.CustomPinColor = *InPinColor;
			}
			InDataType.UsageFlags = InUsageFlags;
			InDataType.TypeFlags |= EOptimusDataTypeFlags::IsStructType;
			if (bInShowElements)
			{
				InDataType.TypeFlags |= EOptimusDataTypeFlags::ShowElements;
			}
			}, PropertyCreateFunc, PropertyValueConvertFunc);
	}
	else
	{
		return false;
	}
}

bool FOptimusDataTypeRegistry::RegisterType(
	UScriptStruct* InStructType,
	FShaderValueTypeHandle InShaderValueType,
	PropertyValueConvertFuncT InPropertyValueConvertFunc,
	TOptional<FLinearColor> InPinColor,
	bool bInShowElements,
	EOptimusDataTypeUsageFlags InUsageFlags)
{
#if WITH_EDITOR
	FText DisplayName = InStructType->GetDisplayNameText();
#else
	FText DisplayName = FText::FromName(InStructType->GetFName());
#endif
	
	return RegisterType(InStructType, DisplayName, InShaderValueType, InPropertyValueConvertFunc, InPinColor, bInShowElements, InUsageFlags);	
}

bool FOptimusDataTypeRegistry::RegisterType(
	UScriptStruct* InStructType,
	const FText& InDisplayName,
	FShaderValueTypeHandle InShaderValueType,
	PropertyValueConvertFuncT InPropertyValueConvertFunc,
	TOptional<FLinearColor> InPinColor,
	bool bInShowElements,
	EOptimusDataTypeUsageFlags InUsageFlags)
{
	if (ensure(InStructType))
	{
		// If showing elements, the sub-elements have to be registered already.
		if (bInShowElements)
		{
			for (const FProperty* Property : TFieldRange<FProperty>(InStructType))
			{
				if (FindType(*Property) == nullptr)
				{
					UE_LOGF(LogOptimusCore, Error, "Found un-registered sub-element '%ls' when registering '%ls'",
						*(Property->GetClass()->GetName()), *InStructType->GetName());
					return false;
				}
			}
		}

		const FName TypeName = GetTypeName(InStructType);

		PropertyCreateFuncT PropertyCreateFunc;
		const int32 ExpectedShaderValueSize = InShaderValueType->GetResourceElementSize();
		
		if (EnumHasAnyFlags(InUsageFlags, EOptimusDataTypeUsageFlags::Variable))
		{
			const bool bIsHashable = IsStructHashable(InStructType);

			PropertyCreateFunc = [bIsHashable, InStructType](FFieldVariant InScope, FName InName) -> FProperty *
			{
				FStructProperty* Property = new FStructProperty(InScope, InName);
				Property->Struct = InStructType;
				Property->SetElementSize(InStructType->GetStructureSize());
				if (bIsHashable)
				{
					Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
				}
				return Property;
			};
		}

		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
			InDataType.TypeName = TypeName;
			InDataType.DisplayName = InDisplayName;
			InDataType.ShaderValueType = InShaderValueType;
			InDataType.ShaderValueSize = ExpectedShaderValueSize;
			InDataType.TypeCategory = FName(TEXT("struct"));
			InDataType.TypeObject = InStructType;
			if (InPinColor.IsSet())
			{
				InDataType.bHasCustomPinColor = true;
				InDataType.CustomPinColor = *InPinColor;
			}
			InDataType.UsageFlags = InUsageFlags;
			InDataType.TypeFlags |= EOptimusDataTypeFlags::IsStructType;
			if (bInShowElements)
			{
				InDataType.TypeFlags |= EOptimusDataTypeFlags::ShowElements;
			}
			}, PropertyCreateFunc, InPropertyValueConvertFunc);
	}
	else
	{
		return false;
	}
}


bool FOptimusDataTypeRegistry::RegisterType(
	UScriptStruct *InStructType,
	const FText& InDisplayName,
    TOptional<FLinearColor> InPinColor,
	EOptimusDataTypeUsageFlags InUsageFlags
	)
{
	if (ensure(InStructType))
	{
		const FName TypeName = GetTypeName(InStructType);

		PropertyCreateFuncT PropertyCreateFunc;
		if (EnumHasAnyFlags(InUsageFlags, EOptimusDataTypeUsageFlags::Variable))
		{
			const bool bIsHashable = IsStructHashable(InStructType);
			
			PropertyCreateFunc = [bIsHashable, InStructType](FFieldVariant InScope, FName InName) -> FProperty* {
				FStructProperty* Property = new FStructProperty(InScope, InName);
				Property->Struct = InStructType;
				Property->SetElementSize(InStructType->GetStructureSize());
				if (bIsHashable)
				{
					Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
				}
				return Property;
			};
		}

		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
				InDataType.TypeName = TypeName;
				InDataType.DisplayName = InDisplayName;
				InDataType.TypeCategory = FName(TEXT("struct"));
				InDataType.TypeObject = InStructType;
				InDataType.bHasCustomPinColor = true;
			    if (InPinColor.IsSet())
			    {
				    InDataType.bHasCustomPinColor = true;
				    InDataType.CustomPinColor = *InPinColor;
			    }
				InDataType.UsageFlags = InUsageFlags;
			}, PropertyCreateFunc);
	}
	else
	{
		return false;
	}
}


bool FOptimusDataTypeRegistry::RegisterType(
	UClass* InClassType, 
	const FText& InDisplayName,
	TOptional<FLinearColor> InPinColor,
	EOptimusDataTypeUsageFlags InUsageFlags
	)
{
	if (ensure(InClassType))
	{
		const FName TypeName(*FString::Printf(TEXT("U%s"), *InClassType->GetName()));

		PropertyCreateFuncT PropertyCreateFunc;
		if (EnumHasAnyFlags(InUsageFlags, EOptimusDataTypeUsageFlags::Variable))
		{
			PropertyCreateFunc = [InClassType](FFieldVariant InScope, FName InName) -> FProperty* {
				FObjectProperty* Property = new FObjectProperty(InScope, InName);
				Property->SetPropertyClass(InClassType);
				Property->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Property;
			};
		}

		return RegisterType(TypeName, [&](FOptimusDataType& InDataType) {
				InDataType.TypeName = TypeName;
				InDataType.DisplayName = InDisplayName;
				InDataType.TypeCategory = FName(TEXT("object"));
				InDataType.TypeObject = InClassType;
				InDataType.bHasCustomPinColor = true;
				if (InPinColor.IsSet())
				{
					InDataType.bHasCustomPinColor = true;
					InDataType.CustomPinColor = *InPinColor;
				}
				InDataType.UsageFlags = InUsageFlags;
			}, PropertyCreateFunc);
	}
	else
	{
		return false;
	}
}


bool FOptimusDataTypeRegistry::RegisterType(
    FName InTypeName,
	const FText& InDisplayName,
    FShaderValueTypeHandle InShaderValueType,
    FName InPinCategory,
    UObject* InPinSubCategory,
    FLinearColor InPinColor,
    EOptimusDataTypeUsageFlags InUsageFlags
	)
{
	if (EnumHasAnyFlags(InUsageFlags, EOptimusDataTypeUsageFlags::Variable))
	{
		UE_LOGF(LogOptimusCore, Error, "Can't register '%ls' for use in variables when there is no associated native type.", *InTypeName.ToString());
		return false;
	}

	return RegisterType(InTypeName, [&](FOptimusDataType& InDataType) {
		InDataType.TypeName = InTypeName;
		InDataType.DisplayName = InDisplayName;
		InDataType.ShaderValueType = InShaderValueType;
		InDataType.ShaderValueSize = InShaderValueType->GetResourceElementSize();
		InDataType.TypeCategory = InPinCategory;
		InDataType.TypeObject = InPinSubCategory;
		InDataType.bHasCustomPinColor = true;
		InDataType.CustomPinColor = InPinColor;
		InDataType.UsageFlags = InUsageFlags;
	});
}


TArray<FOptimusDataTypeHandle> FOptimusDataTypeRegistry::GetAllTypes() const
{
	TArray<FOptimusDataTypeHandle> Result;
	for (const FName& TypeName : RegistrationOrder)
	{
		Result.Add(RegisteredTypes[TypeName].DataType);
	}
	return Result;
}


FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(const FProperty& InProperty) const
{
	if (const FStructProperty* StructProperty = CastField<const FStructProperty>(&InProperty))
	{
		const FName TypeName = GetTypeName(StructProperty->Struct);
		return FindType(TypeName);
	}
	else if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(&InProperty))
	{
		const FName TypeName(*FString::Printf(TEXT("U%s"), *ObjectProperty->PropertyClass->GetName()));
		return FindType(TypeName);
	}
	else if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(&InProperty))
	{
		const FProperty* PropertyForType = ArrayProperty->Inner;
		return FindArrayType(*PropertyForType);
	}
	else
	{
		return FindType(*InProperty.GetClass());
	}

#if 0
	const FProperty* PropertyForType = &InProperty;
	const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(PropertyForType);
	if (ArrayProperty)
	{
		PropertyForType = ArrayProperty->Inner;
	}
	else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(PropertyForType))
	{
		TypeClass = EnumProperty->GetEnum()->GetClass();
	}
	else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(PropertyForType))
	{
		TypeClass = ByteProperty->Enum->GetClass();
	}
#endif
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindArrayType(const FProperty& InProperty) const
{
	if(FOptimusDataTypeHandle Type = FindType(InProperty))
	{
		return FindType(GetArrayTypeName(Type->TypeName));
	}

	return {};	
}


FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(const FFieldClass& InFieldType) const
{
	return FindType(InFieldType.GetFName());
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindArrayType(const FFieldClass& InFieldType) const
{
	if(FOptimusDataTypeHandle Type = FindType(InFieldType))
	{
		return FindType(GetArrayTypeName(Type->TypeName));
	}

	return {};	
}


FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(const UClass& InClassType) const
{
	const FName TypeName(*FString::Printf(TEXT("U%s"), *InClassType.GetName()));
	return FindType(TypeName);
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindArrayType(const UClass& InClassType) const
{
	if(FOptimusDataTypeHandle Type = FindType(InClassType))
	{
		return FindType(GetArrayTypeName(Type->TypeName));
	}

	return {};	
}


FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(FName InTypeName) const
{
	const FTypeInfo* InfoPtr = RegisteredTypes.Find(InTypeName);
	return InfoPtr ? InfoPtr->DataType : FOptimusDataTypeHandle();
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindArrayType(FName InTypeName) const
{
	return FindType(GetArrayTypeName(InTypeName));
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(UScriptStruct* InStruct) const
{
	return FindType(GetTypeName(InStruct));
}

FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindArrayType(UScriptStruct* InStruct) const
{
	return FindArrayType(GetTypeName(InStruct));
}


FOptimusDataTypeHandle FOptimusDataTypeRegistry::FindType(FShaderValueTypeHandle InValueType) const
{
	for (const FName& TypeName : RegistrationOrder)
	{
		const FOptimusDataTypeHandle Handle = RegisteredTypes[TypeName].DataType;
		if (Handle->ShaderValueType == InValueType)
		{
			return Handle;
		}	
	}
	
	return {};
}


void FOptimusDataTypeRegistry::UnregisterAllTypes()
{
	FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	Registry.RegisteredTypes.Reset();
}

void FOptimusDataTypeRegistry::RegisterEngineCallbacks()
{
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		AssetRegistryModule->Get().OnFilesLoaded().AddRaw(&Get(), &FOptimusDataTypeRegistry::OnFilesLoaded);
		AssetRegistryModule->Get().OnAssetRemoved().AddRaw(&Get(), &FOptimusDataTypeRegistry::OnAssetRemoved);
		AssetRegistryModule->Get().OnAssetRenamed().AddRaw(&Get(), &FOptimusDataTypeRegistry::OnAssetRenamed);
	}

	UE::Anim::AttributeTypes::GetOnAttributeTypesChanged().AddRaw(&Get(), &FOptimusDataTypeRegistry::OnAnimationAttributeRegistryChanged);
}

void FOptimusDataTypeRegistry::UnregisterEngineCallbacks()
{
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		AssetRegistryModule->Get().OnFilesLoaded().RemoveAll(&Get());
		AssetRegistryModule->Get().OnAssetRemoved().RemoveAll(&Get());
		AssetRegistryModule->Get().OnAssetRenamed().RemoveAll(&Get());
	}
	
	UE::Anim::AttributeTypes::GetOnAttributeTypesChanged().RemoveAll(&Get());
}


FOptimusDataTypeRegistry::PropertyCreateFuncT FOptimusDataTypeRegistry::FindPropertyCreateFunc(
	FName InTypeName
	) const
{
	const FTypeInfo* InfoPtr = RegisteredTypes.Find(InTypeName);
	if (!InfoPtr)
	{
		UE_LOGF(LogOptimusCore, Fatal, "CreateProperty: Invalid type name.");
		return nullptr;
	}

	return InfoPtr->PropertyCreateFunc;
}

FOptimusDataTypeRegistry::FOptimusDataTypeRegistry()
{
}

void FOptimusDataTypeRegistry::OnFilesLoaded()
{
	TArray<TWeakObjectPtr<const UScriptStruct>> AttributeTypes = UE::Anim::AttributeTypes::GetRegisteredTypes();
	
	for (TWeakObjectPtr<const UScriptStruct> Type : AttributeTypes)
	{
		UScriptStruct* ScriptStruct = const_cast<UScriptStruct*>(Type.Get());

		TArray<UScriptStruct*> BuiltInAttributeTypeArray;
		GetBuiltInAttributeTypes().GenerateValueArray(BuiltInAttributeTypeArray);
		
		if (ScriptStruct && !BuiltInAttributeTypeArray.Contains(ScriptStruct))
		{
			FOptimusDataTypeHandle DataType = FindType(ScriptStruct);

			if (!DataType.IsValid())
			{
				RegisterStructType(ScriptStruct);
			}
		}
	}
}

void FOptimusDataTypeRegistry::OnAssetRemoved(const FAssetData& InAssetData)
{
	if (InAssetData.AssetClassPath == UUserDefinedStruct::StaticClass()->GetClassPathName())
	{
		// Avoid using InAssetData.GetAsset() here, the asset's module/plugin may have just been unloaded
		UnregisterType(GetTypeName(InAssetData));
	}
}

void FOptimusDataTypeRegistry::OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldName)
{
	if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InAssetData.GetAsset()))
	{
		FName TypeName = GetTypeName(UserDefinedStruct);

		if (FOptimusDataTypeHandle DataTypeHandle = FindType(TypeName))
		{
			FOptimusDataType* MutableDataType = const_cast<FOptimusDataType*>(DataTypeHandle.Get());
			MutableDataType->DisplayName = Optimus::GetTypeDisplayName(UserDefinedStruct);
			
			OnDataTypeChanged.Broadcast(TypeName);
		}
	}
}

void FOptimusDataTypeRegistry::OnAnimationAttributeRegistryChanged(const UScriptStruct* InScriptStruct,
	bool bIsAdded)
{
	if (ensure(InScriptStruct) && ensure(InScriptStruct->IsA<UUserDefinedStruct>()))
	{
		
		UScriptStruct* ScriptStruct = const_cast<UScriptStruct*>(InScriptStruct);
		FOptimusDataTypeHandle DataType = FindType(ScriptStruct);
		
		if (bIsAdded)
		{
			if (!DataType.IsValid())
			{
				RegisterStructType(ScriptStruct);
			}
		}
	}
}

FOptimusDataTypeRegistry::PropertyValueConvertFuncT FOptimusDataTypeRegistry::FindPropertyValueConvertFunc(
	FName InTypeName
	) const
{
	const FTypeInfo* InfoPtr = RegisteredTypes.Find(InTypeName);
	if (!InfoPtr)
	{
		UE_LOGF(LogOptimusCore, Fatal, "CreateProperty: Invalid type name.");
		return nullptr;
	}

	return InfoPtr->PropertyValueConvertFunc;
}

TArray<FOptimusDataTypeRegistry::FArrayMetadata> FOptimusDataTypeRegistry::FindArrayMetadata(
	FName InTypeName) const
{
	const FTypeInfo* InfoPtr = RegisteredTypes.Find(InTypeName);
	if (!InfoPtr)
	{
		UE_LOGF(LogOptimusCore, Fatal, "CreateProperty: Invalid type name.");
		return {};
	}

	return InfoPtr->ArrayMetadata;
}

UScriptStruct* FOptimusDataTypeRegistry::FindAttributeType(FName InTypeName) const
{
	if (UScriptStruct* const* AttributeType = GetBuiltInAttributeTypes().Find(InTypeName))
	{
		return *AttributeType;
	}
	
	const FTypeInfo* InfoPtr = RegisteredTypes.Find(InTypeName);
	if (!InfoPtr)
	{
		UE_LOGF(LogOptimusCore, Fatal, "CreateProperty: Invalid type name.");
		return nullptr;
	}

	if (ensure(EnumHasAnyFlags(InfoPtr->DataType->UsageFlags, EOptimusDataTypeUsageFlags::AnimAttributes)))
	{
		return Cast<UScriptStruct>(InfoPtr->DataType->TypeObject);
	}

	return nullptr;
}

TArray<FOptimusDataTypeHandle> FOptimusDataTypeRegistry::GetAllTypesWithAtomicSupport() const
{
	TArray<FOptimusDataTypeHandle> Types;
	for (FName TypeName : TypeWithAtomicSupport)
	{
		FOptimusDataTypeHandle Type = FindType(TypeName);
		if (Type.IsValid())
		{
			Types.Add(Type);
		}
	}

	return Types;
}

bool FOptimusDataTypeRegistry::DoesTypeSupportAtomic(FOptimusDataTypeHandle InType)
{
	return GetAllTypesWithAtomicSupport().Contains(InType);
}

FOptimusDataTypeRegistry::FOnDataTypeChanged& FOptimusDataTypeRegistry::GetOnDataTypeChanged()
{
	return OnDataTypeChanged;
}

void FOptimusDataTypeRegistry::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const TTuple<FName, FTypeInfo>& TypeItem: RegisteredTypes)
	{
		TSharedPtr<FOptimusDataType> DataType = TypeItem.Value.DataType;

		if (DataType->TypeObject.IsValid())
		{
			Collector.AddReferencedObject(DataType->TypeObject);
		}
		
	}
}

void FOptimusDataTypeRegistry::UnregisterType(FName InTypeName)
{
	RegisteredTypes.Remove(InTypeName);
	RegistrationOrder.Remove(InTypeName);
}
