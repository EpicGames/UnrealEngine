// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeformer.h"

#include "Actions/OptimusNodeActions.h"
#include "Actions/OptimusNodeGraphActions.h"
#include "Actions/OptimusResourceActions.h"
#include "Actions/OptimusVariableActions.h"
#include "Components/MeshComponent.h"
#include "ComputeFramework/ComputeKernel.h"
#include "Containers/Queue.h"
#include "DataInterfaces/OptimusDataInterfaceGraph.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "DataInterfaces/OptimusDataInterfaceLoopTerminal.h"
#include "DataInterfaces/OptimusDataInterfaceCopyKernel.h"
#include "IOptimusComputeKernelProvider.h"
#include "IOptimusDataInterfaceProvider.h"
#include "IOptimusValueProvider.h"
#include "OptimusActionStack.h"
#include "OptimusComputeGraph.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerInstance.h"
#include "OptimusCoreModule.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusHelpers.h"
#include "OptimusKernelSource.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusObjectVersion.h"
#include "OptimusResourceDescription.h"
#include "OptimusSettings.h"
#include "OptimusVariableDescription.h"
#include "RenderingThread.h"
#include "SceneInterface.h"
#include "ShaderCore.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "Engine/World.h"

// FIXME: We should not be accessing nodes directly.
#include "OptimusValueContainer.h"
#include "Actions/OptimusComponentBindingActions.h"
#include "ComponentSources/OptimusSkeletalMeshComponentSource.h"
#include "Nodes/OptimusNode_ComponentSource.h"
#include "Nodes/OptimusNode_ConstantValue.h"
#include "Nodes/OptimusNode_DataInterface.h"
#include "Nodes/OptimusNode_GetVariable.h"
#include "Nodes/OptimusNode_ResourceAccessorBase.h"
#include "Nodes/OptimusNode_LoopTerminal.h"

#include "IOptimusDeprecatedExecutionDataInterface.h"
#include "Nodes/OptimusNode_CustomComputeKernel.h"
#include "Nodes/OptimusNode_Resource.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDeformer)

#include <limits>

#include "OptimusDeformerDynamicInstanceManager.h"
#include "OptimusFunctionNodeGraphHeader.h"
#include "OptimusFunctionNodeGraphHeaderWithGuid.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Nodes/OptimusNode_FunctionReference.h"
#include "Nodes/OptimusNode_SubGraphReference.h"

#define PRINT_COMPILED_OUTPUT 1

#define LOCTEXT_NAMESPACE "OptimusDeformer"

static const FName DefaultResourceName("Resource");
static const FName DefaultVariableName("Variable");

const TCHAR* UOptimusDeformer::PublicFunctionsAssetTagName = TEXT("PublicFunctions");
const TCHAR* UOptimusDeformer::PublicFunctionsWithGuidAssetTagName = TEXT("PublicFunctionsWithGuid");

UOptimusDeformer::UOptimusDeformer()
{
	UOptimusNodeGraph *UpdateGraph = CreateDefaultSubobject<UOptimusNodeGraph>(UOptimusNodeGraph::UpdateGraphName);
	UpdateGraph->SetGraphType(EOptimusNodeGraphType::Update);
	Graphs.Add(UpdateGraph);

	Bindings = CreateDefaultSubobject<UOptimusComponentSourceBindingContainer>(TEXT("@Bindings"));
	Variables = CreateDefaultSubobject<UOptimusVariableContainer>(TEXT("@Variables"));
	Resources = CreateDefaultSubobject<UOptimusResourceContainer>(TEXT("@Resources"));

#if WITH_EDITOR
	FOptimusDataTypeRegistry::Get().GetOnDataTypeChanged().AddUObject(this, &UOptimusDeformer::OnDataTypeChanged);
#endif
}


UOptimusActionStack* UOptimusDeformer::GetActionStack()
{
	if (ActionStack == nullptr)
	{
		ActionStack = NewObject<UOptimusActionStack>(this, TEXT("@ActionStack"));
	}
	return ActionStack;
}


UOptimusNodeGraph* UOptimusDeformer::AddSetupGraph()
{
	FOptimusNodeGraphAction_AddGraph* AddGraphAction = 
		new FOptimusNodeGraphAction_AddGraph(GetCollectionPath(), EOptimusNodeGraphType::Setup, UOptimusNodeGraph::SetupGraphName, 0);

	if (GetActionStack()->RunAction(AddGraphAction))
	{
		return AddGraphAction->GetGraph(this);
	}
	else
	{
		return nullptr;
	}
}


UOptimusNodeGraph* UOptimusDeformer::AddTriggerGraph(const FString &InName)
{
	if (!UOptimusNodeGraph::IsValidUserGraphName(InName))
	{
		return nullptr;
	}

	FOptimusNodeGraphAction_AddGraph* AddGraphAction =
	    new FOptimusNodeGraphAction_AddGraph(GetCollectionPath(), EOptimusNodeGraphType::ExternalTrigger, *InName, INDEX_NONE);

	if (GetActionStack()->RunAction(AddGraphAction))
	{
		return AddGraphAction->GetGraph(this);
	}
	else
	{
		return nullptr;
	}
}


UOptimusNodeGraph* UOptimusDeformer::GetUpdateGraph() const
{
	for (UOptimusNodeGraph* Graph: Graphs)
	{
		if (Graph->GetGraphType() == EOptimusNodeGraphType::Update)
		{
			return Graph;
		}
	}
	UE_LOGF(LogOptimusCore, Fatal, "No upgrade graph on deformer (%ls).", *GetPathName());
	return nullptr;
}


bool UOptimusDeformer::RemoveGraph(UOptimusNodeGraph* InGraph)
{
	// Plain subgraph maps 1:1 to a subgraph reference node
	if (InGraph->GetGraphType() == EOptimusNodeGraphType::SubGraph)
	{
		UOptimusNodeSubGraph* SubGraph = CastChecked<UOptimusNodeSubGraph>(InGraph);
		if (UOptimusNode* Node = GetSubGraphReferenceNode(SubGraph))
		{
			FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove SubGraph"));
			// Remove node also triggers the removal of the graph
			Node->GetOwningGraph()->RemoveNode(Node);
			return true;
		}
		
		return false;
	}

	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove Graph"));
	InGraph->RemoveNodes(InGraph->GetAllNodes());
	GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveGraph>(InGraph);
	
	return true;
}

UOptimusNode* UOptimusDeformer::GetSubGraphReferenceNode(const UOptimusNodeSubGraph* InSubGraph) const
{
	UOptimusNode* UsedNode = nullptr;
	
	TArray<UOptimusNode*> AllSubGraphNodes = GetAllNodesOfClass(UOptimusNode_SubGraphReference::StaticClass());
	for (UOptimusNode* Node: AllSubGraphNodes)
	{
		const UOptimusNode_SubGraphReference* SubGraphReference = Cast<UOptimusNode_SubGraphReference>(Node);
		if (SubGraphReference->GetReferencedSubGraph() == InSubGraph)
		{
			UsedNode = Node;
			break;
		}
	}
	return UsedNode;
}

TArray<UOptimusFunctionNodeGraph*> UOptimusDeformer::GetFunctionGraphs(FName InAccessSpecifier) const
{
	TArray<UOptimusFunctionNodeGraph*> FunctionGraphs;
	for (UOptimusNodeGraph* Graph : Graphs)
	{
		if (UOptimusFunctionNodeGraph* FunctionNodeGraph = Cast<UOptimusFunctionNodeGraph>(Graph))
		{
			if (InAccessSpecifier.IsNone())
			{
				FunctionGraphs.Add(FunctionNodeGraph);
			}
			else if (FunctionNodeGraph->AccessSpecifier == InAccessSpecifier)
			{
				FunctionGraphs.Add(FunctionNodeGraph);
			}
		}
	}

	return FunctionGraphs;
}

UOptimusFunctionNodeGraph* UOptimusDeformer::FindFunctionByGuid(FGuid InFunctionGraphGuid)
{
	for (UOptimusFunctionNodeGraph* FunctionNodeGraph : GetFunctionGraphs())
	{
		if (FunctionNodeGraph->GetGuid() == InFunctionGraphGuid)
		{
			return FunctionNodeGraph;
		}
	}

	return nullptr;
}

UOptimusVariableDescription* UOptimusDeformer::AddVariable(
	FOptimusDataTypeRef InDataTypeRef, 
	FName InName /*= NAME_None */
	)
{
	if (InName.IsNone())
	{
		InName = DefaultVariableName;
	}

	if (!InDataTypeRef.IsValid())
	{
		// Default to float.
		InDataTypeRef.Set(FOptimusDataTypeRegistry::Get().FindType(*FDoubleProperty::StaticClass()));
	}

	// Is this data type compatible with variables?
	FOptimusDataTypeHandle DataType = InDataTypeRef.Resolve();
	if (!DataType.IsValid() || !EnumHasAnyFlags(DataType->UsageFlags, EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::Property))
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid data type for variables.");
		return nullptr;
	}

	// Ensure the name is unique.
	InName = Optimus::GetUniqueNameForScope(Variables, InName);

	FOptimusVariableAction_AddVariable* AddVariabAction =
	    new FOptimusVariableAction_AddVariable(InDataTypeRef, InName);

	if (GetActionStack()->RunAction(AddVariabAction))
	{
		return AddVariabAction->GetVariable(this);
	}
	else
	{
		return nullptr;
	}
}


bool UOptimusDeformer::RemoveVariable(
	UOptimusVariableDescription* InVariableDesc
	)
{
	if (!ensure(InVariableDesc))
	{
		return false;
	}
	if (InVariableDesc->GetOuter() != Variables)
	{
		UE_LOGF(LogOptimusCore, Error, "Variable not owned by this deformer.");
		return false;
	}
	
	TMap<UOptimusNodeGraph*, TArray<UOptimusNode*>> NodesByGraph;
	for (UOptimusNode* Node: GetNodesUsingVariable(InVariableDesc))
	{
		UOptimusNode_GetVariable* VariableNode = Cast<UOptimusNode_GetVariable>(Node);
		NodesByGraph.FindOrAdd(VariableNode->GetOwningGraph()).Add(VariableNode);
	}

	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove Variable"));
	
	for (const TTuple<UOptimusNodeGraph*, TArray<UOptimusNode*>>& GraphNodes: NodesByGraph)
	{
		UOptimusNodeGraph* Graph = GraphNodes.Key;
		Graph->RemoveNodes(GraphNodes.Value);
	}

	GetActionStack()->RunAction<FOptimusVariableAction_RemoveVariable>(InVariableDesc);

	return true;
}


bool UOptimusDeformer::RenameVariable(	
	UOptimusVariableDescription* InVariableDesc, 
	FName InNewName,
	bool bInForceChange
	)
{
	if (!ensure(InVariableDesc))
	{
		return false;
	}
	if (InVariableDesc->GetOuter() != Variables)
	{
		UE_LOGF(LogOptimusCore, Error, "Variable not owned by this deformer.");
		return false;
	}
	if (InNewName.IsNone())
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid resource name.");
		return false;
	}

	if (!bInForceChange && InNewName == InVariableDesc->VariableName)
	{
		return true;
	}
	
	// Ensure we can rename to that name, update the name if necessary.
	if (InNewName != InVariableDesc->VariableName)
	{
		InNewName = Optimus::GetUniqueNameForScope(Variables, InNewName);
	}
	
	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Rename Variable"));
	
	for (UOptimusNode* Node: GetNodesUsingVariable(InVariableDesc))
	{
		const UOptimusNode_GetVariable* VariableNode = Cast<UOptimusNode_GetVariable>(Node);
		if (ensure(VariableNode->GetPins().Num() == 1))
		{
			Action->AddSubAction<FOptimusNodeAction_SetPinName>(VariableNode->GetPins()[0], InNewName);
		}
	}

	if (InNewName == InVariableDesc->VariableName)
	{
		Notify(EOptimusGlobalNotifyType::VariableRenamed, InVariableDesc);
	}
	else
	{
		Action->AddSubAction<FOptimusVariableAction_RenameVariable>(InVariableDesc, InNewName);
	}
	return GetActionStack()->RunAction(Action);
}



bool UOptimusDeformer::SetVariableDataType(
	UOptimusVariableDescription* InVariableDesc,
	FOptimusDataTypeRef InDataType,
	bool bInForceChange
	)
{
	if (!ensure(InVariableDesc))
	{
		return false;
	}
	if (InVariableDesc->GetOuter() != Variables)
	{
		UE_LOGF(LogOptimusCore, Error, "Resource not owned by this deformer.");
		return false;
	}
	
	if (!InDataType.IsValid())
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid data type");
		return false;
	}

	if (!bInForceChange && InDataType == InVariableDesc->DataType)
	{
		return true;
	}
	
	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Set Variable Type"));

	TSet<TTuple<UOptimusNodePin* /*OutputPin*/, UOptimusNodePin* /*InputPin*/>> Links;
	
	for (UOptimusNode* Node: GetNodesUsingVariable(InVariableDesc))
	{
		const UOptimusNode_GetVariable* VariableNode = CastChecked<UOptimusNode_GetVariable>(Node);
		if (ensure(VariableNode->GetPins().Num() == 1))
		{
			UOptimusNodePin* Pin = VariableNode->GetPins()[0];

			// Update the pin type to match.
			Action->AddSubAction<FOptimusNodeAction_SetPinType>(VariableNode->GetPins()[0], InDataType);

			// Collect _unique_ links (in case there's a resource->resource link, since that would otherwise
			// show up twice).
			const UOptimusNodeGraph* Graph = Pin->GetOwningNode()->GetOwningGraph();

			for (UOptimusNodePin* ConnectedPin: Graph->GetConnectedPins(Pin))
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
				{
					Links.Add({Pin, ConnectedPin});
				}
				else
				{
					Links.Add({ConnectedPin, Pin});
				}
			}
		}
	}

	for (auto [OutputPin, InputPin]: Links)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(OutputPin, InputPin);
	}

	if (InVariableDesc->DataType != InDataType)
	{
		Action->AddSubAction<FOptimusVariableAction_SetDataType>(InVariableDesc, InDataType);
	}
	else
	{
		Notify(EOptimusGlobalNotifyType::VariableTypeChanged, InVariableDesc);
	}

	if (!GetActionStack()->RunAction(Action))
	{
		return false;
	}
	
	return true;
}

TArray<UOptimusNode*> UOptimusDeformer::GetNodesUsingVariable(
	const UOptimusVariableDescription* InVariableDesc
	) const
{
	TArray<UOptimusNode*> UsedNodes;
	TArray<UOptimusNode*> AllVariableNodes = GetAllNodesOfClass(UOptimusNode_GetVariable::StaticClass());
	for (UOptimusNode* Node: AllVariableNodes)
	{
		const UOptimusNode_GetVariable* VariableNode = Cast<UOptimusNode_GetVariable>(Node);
		if (VariableNode->GetVariableDescription() == InVariableDesc)
		{
			UsedNodes.Add(Node);
		}
	}
	return UsedNodes;
}


UOptimusVariableDescription* UOptimusDeformer::ResolveVariable(
	FName InVariableName
	) const
{
	for (UOptimusVariableDescription* Variable : GetVariables())
	{
		if (Variable->GetFName() == InVariableName)
		{
			return Variable;
		}
	}
	return nullptr;
}


UOptimusVariableDescription* UOptimusDeformer::CreateVariableDirect(
	FName InName
	)
{
	if (!ensure(!InName.IsNone()))
	{
		return nullptr;
	}
	
	UOptimusVariableDescription* Variable = NewObject<UOptimusVariableDescription>(
		Variables, 
		UOptimusVariableDescription::StaticClass(), 
		InName, 
		RF_Transactional);

	// Make sure to give this variable description a unique GUID. We use this when updating the
	// class.
	Variable->Guid = FGuid::NewGuid();
	
	(void)MarkPackageDirty();

	return Variable;
}


bool UOptimusDeformer::AddVariableDirect(
	UOptimusVariableDescription* InVariableDesc,
	const int32 InIndex
	)
{
	if (!ensure(InVariableDesc))
	{
		return false;
	}

	if (!ensure(InVariableDesc->GetOuter() == Variables))
	{
		return false;
	}

	if (Variables->Descriptions.IsValidIndex(InIndex))
	{
		Variables->Descriptions.Insert(InVariableDesc, InIndex);
	}
	else
	{
		Variables->Descriptions.Add(InVariableDesc);
	}


	Notify(EOptimusGlobalNotifyType::VariableAdded, InVariableDesc);

	return true;
}


bool UOptimusDeformer::RemoveVariableDirect(
	UOptimusVariableDescription* InVariableDesc
	)
{
	// Do we actually own this variable?
	const int32 VariableIndex = Variables->Descriptions.IndexOfByKey(InVariableDesc);
	if (VariableIndex == INDEX_NONE)
	{
		return false;
	}

	Variables->Descriptions.RemoveAt(VariableIndex);

	Notify(EOptimusGlobalNotifyType::VariableRemoved, InVariableDesc);

	Optimus::RemoveObject(InVariableDesc);

	(void)MarkPackageDirty();

	return true;
}


bool UOptimusDeformer::RenameVariableDirect(
	UOptimusVariableDescription* InVariableDesc, 
	FName InNewName
	)
{
	// Do we actually own this variable?
	if (Variables->Descriptions.IndexOfByKey(InVariableDesc) == INDEX_NONE)
	{
		return false;
	}

	
	if (Optimus::RenameObject(InVariableDesc, *InNewName.ToString(), nullptr))
	{
		InVariableDesc->VariableName = InNewName;
		Notify(EOptimusGlobalNotifyType::VariableRenamed, InVariableDesc);
		(void)MarkPackageDirty();
		return true;
	}
	
	return false;
}


bool UOptimusDeformer::SetVariableDataTypeDirect(
	UOptimusVariableDescription* InVariableDesc,
	FOptimusDataTypeRef InDataType
	)
{
	// Do we actually own this variable?
	if (Variables->Descriptions.IndexOfByKey(InVariableDesc) == INDEX_NONE)
	{
		return false;
	}
	
	if (InVariableDesc->DataType != InDataType)
	{
		InVariableDesc->SetDataType(InDataType);
		Notify(EOptimusGlobalNotifyType::VariableTypeChanged, InVariableDesc);
		(void)MarkPackageDirty();
	}

	return true;
}


UOptimusResourceDescription* UOptimusDeformer::AddResource(
	FOptimusDataTypeRef InDataTypeRef,
	FName InName
	)
{
	if (InName.IsNone())
	{
		InName = DefaultResourceName;
	}

	if (!InDataTypeRef.IsValid())
	{
		// Default to float.
		InDataTypeRef.Set(FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass()));
	}

	// Is this data type compatible with resources?
	FOptimusDataTypeHandle DataType = InDataTypeRef.Resolve();
	if (!DataType.IsValid() || !EnumHasAnyFlags(DataType->UsageFlags, EOptimusDataTypeUsageFlags::Resource))
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid data type for resources.");
		return nullptr;
	}

	// Ensure the name is unique.
	InName = Optimus::GetUniqueNameForScope(Resources, InName);

	FOptimusResourceAction_AddResource *AddResourceAction = 	
	    new FOptimusResourceAction_AddResource(InDataTypeRef, InName);

	if (GetActionStack()->RunAction(AddResourceAction))
	{
		return AddResourceAction->GetResource(this);
	}
	else
	{
		return nullptr;
	}
}


bool UOptimusDeformer::RemoveResource(UOptimusResourceDescription* InResourceDesc)
{
	if (!ensure(InResourceDesc))
	{
		return false;
	}
	if (InResourceDesc->GetOuter() != Resources)
	{
		UE_LOGF(LogOptimusCore, Error, "Resource not owned by this deformer.");
		return false;
	}

	TMap<UOptimusNodeGraph*, TArray<UOptimusNode*>> NodesByGraph;
	for (UOptimusNode* Node: GetNodesUsingResource(InResourceDesc))
	{
		UOptimusNode_ResourceAccessorBase* ResourceNode = Cast<UOptimusNode_ResourceAccessorBase>(Node);
		NodesByGraph.FindOrAdd(ResourceNode->GetOwningGraph()).Add(ResourceNode);
	}

	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove Resource"));
	
	for (const TTuple<UOptimusNodeGraph*, TArray<UOptimusNode*>>& GraphNodes: NodesByGraph)
	{
		UOptimusNodeGraph* Graph = GraphNodes.Key;
		Graph->RemoveNodes(GraphNodes.Value);
	}

	GetActionStack()->RunAction<FOptimusResourceAction_RemoveResource>(InResourceDesc);

	return true;
}


bool UOptimusDeformer::RenameResource(
	UOptimusResourceDescription* InResourceDesc, 
	FName InNewName,
	bool bInForceChange
	)
{
	if (!ensure(InResourceDesc))
	{
		return false;
	}
	if (InResourceDesc->GetOuter() != Resources)
	{
		UE_LOGF(LogOptimusCore, Error, "Resource not owned by this deformer.");
		return false;
	}
	
	if (InNewName.IsNone())
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid resource name");
		return false;
	}

	if (!bInForceChange && InNewName == InResourceDesc->ResourceName)
	{
		return true;
	}

	// Ensure we can rename to that name, update the name if necessary.
	if (InNewName != InResourceDesc->ResourceName)
	{
		InNewName = Optimus::GetUniqueNameForScope(Resources, InNewName);
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Rename Resource"));

	for (UOptimusNode* Node: GetNodesUsingResource(InResourceDesc))
	{
		const UOptimusNode_ResourceAccessorBase* ResourceNode = CastChecked<UOptimusNode_ResourceAccessorBase>(Node);
		for (int32 PinIndex = 0; PinIndex < ResourceNode->GetPins().Num(); ++PinIndex)
		{
			Action->AddSubAction<FOptimusNodeAction_SetPinName>(ResourceNode->GetPins()[PinIndex], ResourceNode->GetResourcePinName(PinIndex, InNewName));
		}
	}	

	if (InNewName == InResourceDesc->ResourceName)
	{
		// Make sure we update the explorer.
		Notify(EOptimusGlobalNotifyType::ResourceRenamed, InResourceDesc);
	}
	else
	{
		Action->AddSubAction<FOptimusResourceAction_RenameResource>(InResourceDesc, InNewName);
	}
	return GetActionStack()->RunAction(Action);
}


bool UOptimusDeformer::SetResourceDataType(
	UOptimusResourceDescription* InResourceDesc,
	FOptimusDataTypeRef InDataType,
	bool bInForceChange
	)
{
	if (!ensure(InResourceDesc))
	{
		return false;
	}
	if (InResourceDesc->GetOuter() != Resources)
	{
		UE_LOGF(LogOptimusCore, Error, "Resource not owned by this deformer.");
		return false;
	}

	if (!InDataType.IsValid())
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid data type");
		return false;
	}

	if (!bInForceChange && InDataType == InResourceDesc->DataType)
	{
		return true;
	}
	
	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Set Resource Data Type"));

	TSet<TTuple<UOptimusNodePin* /*OutputPin*/, UOptimusNodePin* /*InputPin*/>> Links;
	
	for (UOptimusNode* Node: GetNodesUsingResource(InResourceDesc))
	{
		const UOptimusNode_ResourceAccessorBase* ResourceNode = CastChecked<UOptimusNode_ResourceAccessorBase>(Node);
		for (UOptimusNodePin* Pin : ResourceNode->GetPins())
		{
			// Update the pin type to match.
			Action->AddSubAction<FOptimusNodeAction_SetPinType>(Pin, InDataType);

			// Collect _unique_ links (in case there's a resource->resource link, since that would otherwise show up twice).
			const UOptimusNodeGraph* Graph = Pin->GetOwningNode()->GetOwningGraph();

			for (UOptimusNodePin* ConnectedPin: Graph->GetConnectedPins(Pin))
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
				{
					Links.Add({Pin, ConnectedPin});
				}
				else
				{
					Links.Add({ConnectedPin, Pin});
				}
			}
		}
	}

	for (auto [OutputPin, InputPin]: Links)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(OutputPin, InputPin);
	}

	if (InResourceDesc->DataType != InDataType)
	{
		Action->AddSubAction<FOptimusResourceAction_SetDataType>(InResourceDesc, InDataType);
	}

	return GetActionStack()->RunAction(Action);
}

bool UOptimusDeformer::SetResourceDataDomain(
	UOptimusResourceDescription* InResourceDesc,
	const FOptimusDataDomain& InDataDomain,
	bool bInForceChange
	)
{
	if (!ensure(InResourceDesc))
	{
		return false;
	}
	if (InResourceDesc->GetOuter() != Resources)
	{
		UE_LOGF(LogOptimusCore, Error, "Resource not owned by this deformer.");
		return false;
	}

	if (!bInForceChange && InDataDomain == InResourceDesc->DataDomain)
	{
		return true;
	}
	
	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Set Resource Data Domain"));

	TSet<TTuple<UOptimusNodePin* /*OutputPin*/, UOptimusNodePin* /*InputPin*/>> Links;
	
	for (UOptimusNode* Node: GetNodesUsingResource(InResourceDesc))
	{
		const UOptimusNode_ResourceAccessorBase* ResourceNode = CastChecked<UOptimusNode_ResourceAccessorBase>(Node);
		for (UOptimusNodePin* Pin : ResourceNode->GetPins())
		{
			// Update the pin type to match.
			Action->AddSubAction<FOptimusNodeAction_SetPinDataDomain>(Pin, InDataDomain);

			// Collect _unique_ links (in case there's a resource->resource link, since that would otherwise
			// show up twice).
			const UOptimusNodeGraph* Graph = Pin->GetOwningNode()->GetOwningGraph();

			for (UOptimusNodePin* ConnectedPin: Graph->GetConnectedPins(Pin))
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
				{
					Links.Add({Pin, ConnectedPin});
				}
				else
				{
					Links.Add({ConnectedPin, Pin});
				}
			}
		}	
	}

	for (auto [OutputPin, InputPin]: Links)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(OutputPin, InputPin);
	}

	if (InResourceDesc->DataDomain != InDataDomain)
	{
		Action->AddSubAction<FOptimusResourceAction_SetDataDomain>(InResourceDesc, InDataDomain);
	}

	return GetActionStack()->RunAction(Action);
}


TArray<UOptimusNode*> UOptimusDeformer::GetNodesUsingResource(
	const UOptimusResourceDescription* InResourceDesc
	) const
{
	TArray<UOptimusNode*> UsedNodes;
	TArray<UOptimusNode*> AllResourceNodes = GetAllNodesOfClass(UOptimusNode_ResourceAccessorBase::StaticClass());
	for (UOptimusNode* Node: AllResourceNodes)
	{
		const UOptimusNode_ResourceAccessorBase* ResourceNode = Cast<UOptimusNode_ResourceAccessorBase>(Node);
		if (ResourceNode->GetResourceDescription() == InResourceDesc)
		{
			UsedNodes.Add(Node);
		}
	}
	return UsedNodes;
}


UOptimusResourceDescription* UOptimusDeformer::ResolveResource(
	FName InResourceName
	) const
{
	for (UOptimusResourceDescription* Resource : GetResources())
	{
		if (Resource->GetFName() == InResourceName)
		{
			return Resource;
		}
	}
	return nullptr;
}


UOptimusResourceDescription* UOptimusDeformer::CreateResourceDirect(
	FName InName
	)
{
	if (!ensure(!InName.IsNone()))
	{
		return nullptr;
	}
	
	// The resource is actually owned by the "Resources" container to avoid name clashing as
	// much as possible.
	UOptimusResourceDescription* Resource = NewObject<UOptimusResourceDescription>(
		Resources, 
		UOptimusResourceDescription::StaticClass(),
		InName, 
		RF_Transactional);

	return Resource;
}


bool UOptimusDeformer::AddResourceDirect(
	UOptimusResourceDescription* InResourceDesc,
	const int32 InIndex
	)
{
	if (!ensure(InResourceDesc))
	{
		return false;
	}

	if (!ensure(InResourceDesc->GetOuter() == Resources))
	{
		return false;
	}

	if (Resources->Descriptions.IsValidIndex(InIndex))
	{
		Resources->Descriptions.Insert(InResourceDesc, InIndex);
	}
	else
	{
		Resources->Descriptions.Add(InResourceDesc);
	}
	
	Notify(EOptimusGlobalNotifyType::ResourceAdded, InResourceDesc);
	(void)MarkPackageDirty();
	
	return true;
}


bool UOptimusDeformer::RemoveResourceDirect(
	UOptimusResourceDescription* InResourceDesc
	)
{
	// Do we actually own this resource?
	int32 ResourceIndex = Resources->Descriptions.IndexOfByKey(InResourceDesc);
	if (ResourceIndex == INDEX_NONE)
	{
		return false;
	}

	Resources->Descriptions.RemoveAt(ResourceIndex);

	Notify(EOptimusGlobalNotifyType::ResourceRemoved, InResourceDesc);

	Optimus::RemoveObject(InResourceDesc);

	(void)MarkPackageDirty();

	return true;
}


bool UOptimusDeformer::RenameResourceDirect(
	UOptimusResourceDescription* InResourceDesc, 
	FName InNewName
	)
{
	// Do we actually own this resource?
	int32 ResourceIndex = Resources->Descriptions.IndexOfByKey(InResourceDesc);
	if (ResourceIndex == INDEX_NONE)
	{
		return false;
	}

	// Rename in a non-transactional manner, since we're handling undo/redo.
	if (Optimus::RenameObject(InResourceDesc, *InNewName.ToString(), nullptr))
	{
		InResourceDesc->ResourceName = InNewName;
		Notify(EOptimusGlobalNotifyType::ResourceRenamed, InResourceDesc);
		(void)MarkPackageDirty();
		return true;
	}

	return false;
}


bool UOptimusDeformer::SetResourceDataTypeDirect(
	UOptimusResourceDescription* InResourceDesc,
	FOptimusDataTypeRef InDataType
	)
{
	// Do we actually own this resource?
	const int32 ResourceIndex = Resources->Descriptions.IndexOfByKey(InResourceDesc);
	if (ResourceIndex == INDEX_NONE)
	{
		return false;
	}
	
	if (InResourceDesc->DataType != InDataType)
	{
		InResourceDesc->DataType = InDataType;
		Notify(EOptimusGlobalNotifyType::ResourceTypeChanged, InResourceDesc);
		(void)MarkPackageDirty();
	}
	
	return true;
}


bool UOptimusDeformer::SetResourceDataDomainDirect(
	UOptimusResourceDescription* InResourceDesc,
	const FOptimusDataDomain& InDataDomain
	)
{
	// Do we actually own this resource?
	const int32 ResourceIndex = Resources->Descriptions.IndexOfByKey(InResourceDesc);
	if (ResourceIndex == INDEX_NONE)
	{
		return false;
	}
	
	if (InResourceDesc->DataDomain != InDataDomain)
	{
		InResourceDesc->DataDomain = InDataDomain;
		Notify(EOptimusGlobalNotifyType::ResourceTypeChanged, InResourceDesc);
		(void)MarkPackageDirty();
	}
	
	return true;
}


// === Component bindings
UOptimusComponentSourceBinding* UOptimusDeformer::AddComponentBinding(
	const UOptimusComponentSource* InComponentSource,
	FName InName
	)
{
	if (!InComponentSource)
	{
		InComponentSource = UOptimusSkeletalMeshComponentSource::StaticClass()->GetDefaultObject<UOptimusComponentSource>();		
	}

	if (InName.IsNone())
	{
		InName = InComponentSource->GetBindingName();
	}

	InName = Optimus::GetUniqueNameForScope(Bindings, InName);

	FOptimusComponentBindingAction_AddBinding *AddComponentBindingAction = 	
		new FOptimusComponentBindingAction_AddBinding(InComponentSource, InName);

	if (!GetActionStack()->RunAction(AddComponentBindingAction))
	{
		return nullptr;
	}
	
	return AddComponentBindingAction->GetComponentBinding(this);
}


UOptimusComponentSourceBinding* UOptimusDeformer::CreateComponentBindingDirect(
	const UOptimusComponentSource* InComponentSource,
	FName InName
	)
{
	if (!ensure(InComponentSource) || !ensure(!InName.IsNone()))
	{
		return nullptr;
	}

	UOptimusComponentSourceBinding* Binding = NewObject<UOptimusComponentSourceBinding>(
		Bindings, 
		UOptimusComponentSourceBinding::StaticClass(), 
		InName, 
		RF_Transactional);

	Binding->ComponentType = InComponentSource->GetClass();
	Binding->BindingName = InName;

	return Binding;
}


bool UOptimusDeformer::AddComponentBindingDirect(
	UOptimusComponentSourceBinding* InComponentBinding,
	const int32 InIndex
	)
{
	if (!ensure(InComponentBinding))
	{
		return false;
	}
	if (!ensure(InComponentBinding->GetOuter() == Bindings))
	{
		return false;
	}

	if (Bindings->Bindings.IsEmpty())
	{
		InComponentBinding->bIsPrimaryBinding = true;
	}

	if (Bindings->Bindings.IsValidIndex(InIndex))
	{
		Bindings->Bindings.Insert(InComponentBinding, InIndex);
	}
	else
	{
		Bindings->Bindings.Add(InComponentBinding);
	}

	Notify(EOptimusGlobalNotifyType::ComponentBindingAdded, InComponentBinding);

	(void)MarkPackageDirty();

	return true;
}


bool UOptimusDeformer::RemoveComponentBinding(
	UOptimusComponentSourceBinding* InBinding
	)
{
	if (!ensure(InBinding))
	{
		return false;
	}
	if (InBinding->GetOuter() != Bindings)
	{
		UE_LOGF(LogOptimusCore, Error, "Component binding not owned by this deformer.");
		return false;
	}
	if (InBinding->bIsPrimaryBinding)
	{
		UE_LOGF(LogOptimusCore, Error, "The primary binding cannot be removed.");
		return false;
	}

	TMap<UOptimusNodeGraph*, TArray<UOptimusNode*>> NodesByGraph;
	
	for (UOptimusNode* Node: GetNodesUsingComponentBinding(InBinding))
	{
		NodesByGraph.FindOrAdd(Node->GetOwningGraph()).Add(Node);
	}

	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove Binding"));

	for (const TTuple<UOptimusNodeGraph*, TArray<UOptimusNode*>>& GraphNodes: NodesByGraph)
	{
		UOptimusNodeGraph* Graph = GraphNodes.Key;
		Graph->RemoveNodes(GraphNodes.Value);
	}

	GetActionStack()->RunAction<FOptimusComponentBindingAction_RemoveBinding>(InBinding);

	return true;
}


bool UOptimusDeformer::RemoveComponentBindingDirect(UOptimusComponentSourceBinding* InBinding)
{
	// Do we actually own this binding?
	const int32 BindingIndex = Bindings->Bindings.IndexOfByKey(InBinding);
	if (BindingIndex == INDEX_NONE)
	{
		return false;
	}

	Bindings->Bindings.RemoveAt(BindingIndex);

	Notify(EOptimusGlobalNotifyType::ComponentBindingRemoved, InBinding);

	Optimus::RemoveObject(InBinding);
	
	(void)MarkPackageDirty();
	
	return true;
}


bool UOptimusDeformer::RenameComponentBinding(
	UOptimusComponentSourceBinding* InBinding, 
	FName InNewName,
	bool bInForceChange
	)
{
	if (!ensure(InBinding))
	{
		return false;
	}
	if (InBinding->GetOuter() != Bindings)
	{
		UE_LOGF(LogOptimusCore, Error, "Binding not owned by this deformer.");
		return false;
	}
	
	if (InNewName.IsNone())
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid binding name");
		return false;
	}

	if (!bInForceChange && InNewName == InBinding->BindingName)
	{
		return true;
	}

	// Ensure we can rename to that name, update the name if necessary.
	if (InNewName != InBinding->BindingName)
	{
		InNewName = Optimus::GetUniqueNameForScope(Resources, InNewName);
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Rename Component Binding"));

	for (UOptimusNode* Node: GetNodesUsingComponentBinding(InBinding))
	{
		Action->AddSubAction<FOptimusNodeAction_RenameNode>(Node, FText::FromName(InNewName));
	}	

	if (InNewName == InBinding->BindingName)
	{
		Notify(EOptimusGlobalNotifyType::ComponentBindingRenamed, InBinding);
	}
	else
	{
		Action->AddSubAction<FOptimusComponentBindingAction_RenameBinding>(InBinding, InNewName);
	}
	
	return GetActionStack()->RunAction(Action);
}



bool UOptimusDeformer::RenameComponentBindingDirect(
	UOptimusComponentSourceBinding* InBinding,
	FName InNewName
	)
{
	// Do we actually own this binding?
	if (Bindings->Bindings.IndexOfByKey(InBinding) == INDEX_NONE)
	{
		return false;
	}
	
	if (Optimus::RenameObject(InBinding, *InNewName.ToString(), nullptr))
	{
		InBinding->BindingName = InNewName;
		Notify(EOptimusGlobalNotifyType::ComponentBindingRenamed, InBinding);
		(void)MarkPackageDirty();
		return true;
	}
	
	return false;
}


bool UOptimusDeformer::SetComponentBindingSource(
	UOptimusComponentSourceBinding* InBinding,
	const UOptimusComponentSource* InComponentSource,
	bool bInForceChange
	)
{
	if (!ensure(InBinding))
	{
		return false;
	}
	if (InBinding->GetOuter() != Bindings)
	{
		UE_LOGF(LogOptimusCore, Error, "Binding not owned by this deformer.");
		return false;
	}
	
	if (!InComponentSource)
	{
		UE_LOGF(LogOptimusCore, Error, "Invalid component source");
		return false;
	}

	if (!bInForceChange && InBinding->ComponentType == InComponentSource->GetClass())
	{
		return true;
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction(TEXT("Set Component Binding Source"));
	TSet<TTuple<UOptimusNodePin* /*OutputPin*/, UOptimusNodePin* /*InputPin*/>> Links;
	for (UOptimusNode* Node: GetNodesUsingComponentBinding(InBinding))
	{
		const UOptimusNode_ComponentSource* ComponentSourceNode = CastChecked<UOptimusNode_ComponentSource>(Node);
		UOptimusNodePin* ComponentSourcePin = ComponentSourceNode->GetComponentPin();

		const UOptimusNodeGraph* Graph = ComponentSourceNode->GetOwningGraph();

		for (UOptimusNodePin* ConnectedPin: Graph->GetConnectedPins(ComponentSourcePin))
		{
			// Will this connection be invalid once the source is changed?
			if (const UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(ConnectedPin->GetOwningNode()))
			{
				if (!DataInterfaceNode->IsComponentSourceCompatible(InComponentSource))
				{
					Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(ComponentSourcePin, ConnectedPin);
				}
			}
		}

		// Change the pin name _after_ the links are removed, since the link remove action uses the pin path, including
		// the old name to resolve the pin.
		Action->AddSubAction<FOptimusNodeAction_SetPinName>(ComponentSourcePin, InComponentSource->GetBindingName());
	}
	
	if (InBinding->ComponentType != InComponentSource->GetClass())
	{
		Action->AddSubAction<FOptimusComponentBindingAction_SetComponentSource>(InBinding, InComponentSource);
	}
	else
	{
		Notify(EOptimusGlobalNotifyType::ComponentBindingSourceChanged, InBinding);
	}

	return GetActionStack()->RunAction(Action);
}


TArray<UOptimusNode*> UOptimusDeformer::GetNodesUsingComponentBinding(
	const UOptimusComponentSourceBinding* InBinding
	) const
{
	TArray<UOptimusNode*> UsedNodes;
	TArray<UOptimusNode*> AllComponentSourceNode = GetAllNodesOfClass(UOptimusNode_ComponentSource::StaticClass());
	for (UOptimusNode* Node: AllComponentSourceNode)
	{
		const UOptimusNode_ComponentSource* ComponentSourceNode = Cast<UOptimusNode_ComponentSource>(Node);
		if (ComponentSourceNode->GetComponentBinding() == InBinding)
		{
			UsedNodes.Add(Node);
		}
	}
	return UsedNodes;
}


bool UOptimusDeformer::SetComponentBindingSourceDirect(
	UOptimusComponentSourceBinding* InBinding,
	const UOptimusComponentSource* InComponentSource
	)
{
	// Do we actually own this binding?
	if (Bindings->Bindings.IndexOfByKey(InBinding) == INDEX_NONE)
	{
		return false;
	}

	if (InBinding->ComponentType != InComponentSource->GetClass())
	{
		InBinding->ComponentType = InComponentSource->GetClass();
		Notify(EOptimusGlobalNotifyType::ComponentBindingSourceChanged, InBinding);
		(void)MarkPackageDirty();
	}
	
	return true;
}

void UOptimusDeformer::SetStatusFromDiagnostic(EOptimusDiagnosticLevel InDiagnosticLevel)
{
	if (InDiagnosticLevel == EOptimusDiagnosticLevel::Error)
	{
		Status = EOptimusDeformerStatus::HasErrors;
	}
	else if (InDiagnosticLevel == EOptimusDiagnosticLevel::Warning && Status == EOptimusDeformerStatus::Compiled)
	{
		Status = EOptimusDeformerStatus::CompiledWithWarnings;
	}
}


UOptimusComponentSourceBinding* UOptimusDeformer::GetPrimaryComponentBinding() const
{
	for (UOptimusComponentSourceBinding* Binding : GetComponentBindings())
	{
		if (Binding->bIsPrimaryBinding)
		{
			return Binding;
		}
	}
	return nullptr;
}


UOptimusComponentSourceBinding* UOptimusDeformer::ResolveComponentBinding(
	FName InBindingName
	) const
{
	for (UOptimusComponentSourceBinding* Binding : GetComponentBindings())
	{
		if (Binding->GetFName() == InBindingName)
		{
			return Binding;
		}
	}
	return nullptr;
}



// Do a breadth-first collection of nodes starting from the seed nodes (terminal data interfaces).
static void CollectNodes(
	const TArray<const UOptimusNode*>& InSeedNodes,
	TArray<FOptimusRoutedConstNode>& OutCollectedNodes,
	TMap<FOptimusRoutedConstNode, TArray<FOptimusRoutedConstNode>>& OutNodeToInputNodes,
	TMap<FOptimusRoutedConstNode, TArray<FOptimusRoutedConstNode>>& OutNodeToOutputNodes 
	)
{
	TSet<FOptimusRoutedConstNode> VisitedNodes;
	TSet<FOptimusRoutedConstNode> UniqueNeighborNodes;
	TQueue<FOptimusRoutedConstNode> WorkingSet;

	for (const UOptimusNode* Node: InSeedNodes)
	{
		WorkingSet.Enqueue({Node, FOptimusPinTraversalContext{}});
		VisitedNodes.Add({Node, FOptimusPinTraversalContext{}});
		OutCollectedNodes.Add({Node, FOptimusPinTraversalContext{}});
	}

	auto CollectFromInputPins = [&WorkingSet, &VisitedNodes, &UniqueNeighborNodes, &OutCollectedNodes, &OutNodeToInputNodes, &OutNodeToOutputNodes](const FOptimusRoutedConstNode& InWorkItem, const UOptimusNodePin* InPin)
	{
		for (const FOptimusRoutedNodePin& ConnectedPin: InPin->GetConnectedPinsWithRouting(InWorkItem.TraversalContext))
		{
			if (ensure(ConnectedPin.NodePin != nullptr))
			{
				const UOptimusNode *NextNode = ConnectedPin.NodePin->GetOwningNode();
				FOptimusRoutedConstNode CollectedNode{NextNode, ConnectedPin.TraversalContext};
				if (!UniqueNeighborNodes.Contains(CollectedNode))
				{
					UniqueNeighborNodes.Add(CollectedNode);
					
					OutNodeToInputNodes.FindOrAdd(InWorkItem).Add(CollectedNode);
					OutNodeToOutputNodes.FindOrAdd(CollectedNode).Add(InWorkItem);
					
					WorkingSet.Enqueue(CollectedNode);
					
					if (!VisitedNodes.Contains(CollectedNode))
					{
						VisitedNodes.Add(CollectedNode);
						OutCollectedNodes.Add(CollectedNode);
					}
					else
					{
						// Push the node to the back because to ensure that it is scheduled  earlier then it's referencing node.
						OutCollectedNodes.RemoveSingle(CollectedNode);
						OutCollectedNodes.Add(CollectedNode);
					}
				}
			}
		}
	};

	FOptimusRoutedConstNode WorkItem;
	while (WorkingSet.Dequeue(WorkItem))
	{
		UniqueNeighborNodes.Reset();
		
		// Traverse in the direction of input pins (up the graph).
		for (const UOptimusNodePin* Pin: WorkItem.Node->GetPins())
		{
			if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
			{
				if (Pin->IsGroupingPin())
				{
					for (const UOptimusNodePin* SubPin: Pin->GetSubPins())
					{
						CollectFromInputPins(WorkItem, SubPin);
					}
				}
				else
				{
					CollectFromInputPins(WorkItem, Pin);
				}
			}
		}	
	}	
}

#if WITH_EDITOR
bool UOptimusDeformer::Compile()
{
	if (!GetUpdateGraph())
	{
		FOptimusCompilerDiagnostic Diagnostic;
		Diagnostic.Level = EOptimusDiagnosticLevel::Error;
		Diagnostic.Message = LOCTEXT("NoGraphFound", "No update graph found. Compilation aborted.");

		CompileBeginDelegate.Broadcast(this);
		CompileMessageDelegate.Broadcast(Diagnostic);
		CompileEndDelegate.Broadcast(this);

		Status = EOptimusDeformerStatus::HasErrors;
		
		return false;
	}

	auto ClearCompiledData = [&]()
	{
		for (FOptimusComputeGraphInfo& GraphInfo : ComputeGraphs )
		{
			Optimus::RemoveObject(GraphInfo.ComputeGraph);
		}
		ComputeGraphs.Reset();
		DataInterfacePropertyOverrideMap.Reset();
		ValueMap.Reset();
	};

	ClearCompiledData();
	
	CompileBeginDelegate.Broadcast(this);
	
	// Wait for rendering to be done.
	FlushRenderingCommands();

	Status = EOptimusDeformerStatus::Compiled;

	auto ErrorReporter = [this](
		EOptimusDiagnosticLevel InDiagnosticLevel, 
		FText InMessage,
		const UObject* InObject)
	{
		FOptimusCompilerDiagnostic Diagnostic;
		Diagnostic.Level = InDiagnosticLevel;
		Diagnostic.Message = InMessage;
		Diagnostic.Object = InObject;
		CompileMessageDelegate.Broadcast(Diagnostic);

		SetStatusFromDiagnostic(InDiagnosticLevel);
	};

	for (const UOptimusNodeGraph* Graph: Graphs)
	{
		if (Graph->GetGraphType() != EOptimusNodeGraphType::Function)
		{
			FOptimusNodeGraphCompilationResult Result = CompileNodeGraphToComputeGraphs(Graph, ErrorReporter);
			ComputeGraphs.Append(Result.ComputeGraphInfos);

			DataInterfacePropertyOverrideMap.Append(Result.DataInterfacePropertyOverrideMap);

			// Merge Value Maps
			for (const TPair<FOptimusValueIdentifier, FOptimusValueDescription>& Pair : Result.ValueMap)
			{
				const FOptimusValueIdentifier& ValueId = Pair.Key;
				const FOptimusValueDescription& ValueDescription = Pair.Value;
				
				if (FOptimusValueDescription* ExistingDescription = ValueMap.Find(ValueId))
				{
					if (EnumHasAnyFlags(ValueDescription.ValueUsage, EOptimusValueUsage::CPU) &&
						(!EnumHasAnyFlags(ExistingDescription->ValueUsage, EOptimusValueUsage::CPU)))
					{
						ExistingDescription->ValueUsage |= EOptimusValueUsage::CPU;
						ExistingDescription->Value = ValueDescription.Value;
					}
					else if (EnumHasAnyFlags(ValueDescription.ValueUsage, EOptimusValueUsage::GPU) &&
						(!EnumHasAnyFlags(ExistingDescription->ValueUsage, EOptimusValueUsage::GPU)))
					{
						ExistingDescription->ValueUsage |= EOptimusValueUsage::GPU;
						ExistingDescription->ShaderValue = ValueDescription.ShaderValue;
					}
				}
				else
				{
					ValueMap.Add(Pair);
				}
			}
		}
	}

	CompileEndDelegate.Broadcast(this);

	if (Status == EOptimusDeformerStatus::HasErrors)
	{
		ClearCompiledData();
		return false;
	}
	
#if WITH_EDITOR
	// Flush the shader file cache in case we are editing engine or data interface shaders.
	// We could make the user do this manually, but that makes iterating on data interfaces really painful.
	FlushShaderFileCache();
#endif

	for (const FOptimusComputeGraphInfo& ComputeGraphInfo: ComputeGraphs)
	{
		ComputeGraphInfo.ComputeGraph->UpdateResources();
	}
	
	return true;
}
#endif // WITH_EDITOR

TArray<UOptimusNode*> UOptimusDeformer::GetAllNodesOfClass(UClass* InNodeClass) const
{
	if (!ensure(InNodeClass->IsChildOf<UOptimusNode>()))
	{
		return {};
	}

	TArray<UOptimusNodeGraph*> GraphsToSearch = Graphs;
	TArray<UOptimusNode*> NodesFound;
	
	while(!GraphsToSearch.IsEmpty())
	{
		const UOptimusNodeGraph* CurrentGraph = GraphsToSearch.Pop(EAllowShrinking::No);

		for (UOptimusNode* Node: CurrentGraph->GetAllNodes())
		{
			if (Node->GetClass()->IsChildOf(InNodeClass))
			{
				NodesFound.Add(Node);
			}
		}

		GraphsToSearch.Append(CurrentGraph->GetGraphs());
	}

	return NodesFound;
}

void UOptimusDeformer::OnGraphRenamedOrRemoved(UOptimusNodeGraph* InGraph) const
{
	if (UOptimusFunctionNodeGraph* FunctionNodeGraph = Cast<UOptimusFunctionNodeGraph>(InGraph))
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry&	AssetRegistry = AssetRegistryModule.Get();

		TArray<FAssetIdentifier> Referencers;
		FAssetIdentifier Source(InGraph->GetPackage()->GetFName(), NAME_None);
		AssetRegistry.GetReferencers(Source, Referencers);

		Referencers.Add(Source);
		
		for (const FAssetIdentifier& Referencer : Referencers)
		{
			TArray<FAssetData> Assets;
			AssetRegistry.GetAssetsByPackageName(Referencer.PackageName, Assets);

			for (const FAssetData& Asset : Assets)
			{
				if (Asset.IsInstanceOf(UOptimusDeformer::StaticClass()))
				{
					constexpr bool bLoad = false;
					if (UOptimusDeformer* ReferencerDeformer = Cast<UOptimusDeformer>(Asset.FastGetAsset(bLoad)))
					{
						ReferencerDeformer->UpdateFunctionReferenceNodeDisplayName(FunctionNodeGraph->GetGraphIdentifier());
					}
				}
			}
		}	
	}
}

void UOptimusDeformer::UpdateFunctionReferenceNodeDisplayName(const FOptimusFunctionGraphIdentifier& InRenamedFunction)
{
	TArray<UOptimusNode*> FunctionNodes = GetAllNodesOfClass(UOptimusNode_FunctionReference::StaticClass());
	for (UOptimusNode* Node: FunctionNodes)
	{
		UOptimusNode_FunctionReference* ReferenceNode = CastChecked<UOptimusNode_FunctionReference>(Node);
		if (ReferenceNode->GetReferencedFunctionGraphIdentifier() == InRenamedFunction)
		{
			ReferenceNode->UpdateDisplayName();
		}
	}
}

struct FOptimusInstancedNode
{
	FOptimusRoutedConstNode	RoutedNode;
	int32 LoopIndex = 0;

	FOptimusInstancedNode() = default;
	FOptimusInstancedNode(const FOptimusRoutedConstNode& InRoutedNode, const int32 InLoopIndex) :
		RoutedNode(InRoutedNode), LoopIndex(InLoopIndex)
	{
	}
	
	friend uint32 GetTypeHash(const FOptimusInstancedNode& InNode)
	{
		return HashCombineFast(GetTypeHash(InNode.RoutedNode), InNode.LoopIndex);
	}

	bool operator==(const FOptimusInstancedNode& InOther) const
	{
		return RoutedNode == InOther.RoutedNode && LoopIndex == InOther.LoopIndex;
	}
};

struct FOptimusInstancedPin
{
	FOptimusInstancedNode InstancedNode;
	const UOptimusNodePin* Pin = nullptr;

	friend uint32 GetTypeHash(const FOptimusInstancedPin& InPin)
	{
		return HashCombineFast(GetTypeHash(InPin.InstancedNode), GetTypeHash(InPin.Pin));
	}

	bool operator==(const FOptimusInstancedPin& InOther) const
	{
		return InstancedNode == InOther.InstancedNode && Pin == InOther.Pin;
	}
};

#if WITH_EDITOR
FOptimusNodeGraphCompilationResult UOptimusDeformer::CompileNodeGraphToComputeGraphs(
	const UOptimusNodeGraph* InNodeGraph,
	TFunction<void(EOptimusDiagnosticLevel, FText, const UObject*)> InErrorReporter
	)
{
	auto AddDiagnostic = [ErrorReporter=InErrorReporter](
		EOptimusDiagnosticLevel InLevel, 
		FText InMessage,
		const UOptimusNode* InNode = nullptr)
	{
		// Only raise the diagnostic level.
		if (InNode && InNode->GetDiagnosticLevel() < InLevel)
		{
			const_cast<UOptimusNode*>(InNode)->SetDiagnosticLevel(InLevel);
		}

		ErrorReporter(InLevel, InMessage, InNode);
	};
	
	// No nodes in the graph, nothing to do.
	if (InNodeGraph->GetAllNodes().IsEmpty())
	{
		return {};
	}

	// Clear the error state of all nodes.
	for (UOptimusNode* Node: InNodeGraph->GetAllNodes())
	{
		Node->SetDiagnosticLevel(EOptimusDiagnosticLevel::None);
	}

	// Terminal nodes are data providers that contain only input pins. Any graph with no
	// written output is a null graph.
	TArray<const UOptimusNode*> TerminalNodes;
	
	for (const UOptimusNode* Node: InNodeGraph->GetAllNodes())
	{
		bool bConnectedInput = false;

		const IOptimusDataInterfaceProvider* DataInterfaceProviderNode = Cast<const IOptimusDataInterfaceProvider>(Node);

		if (DataInterfaceProviderNode)
		{
			for (const UOptimusNodePin* Pin: Node->GetPins())
			{
				// NOTE: No grouping pins on data interfaces (yet).
				if (!ensure(!Pin->IsGroupingPin()))
				{
					continue;
				}
				
				if (Pin->GetDirection() == EOptimusNodePinDirection::Input && Pin->GetConnectedPins().Num())
				{
					bConnectedInput = true;
				}
				if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
				{
					DataInterfaceProviderNode = nullptr;
					break;
				}
			}
		}
		if (DataInterfaceProviderNode && bConnectedInput)
		{
			TerminalNodes.Add(Node);
		}
	}

	if (TerminalNodes.IsEmpty())
	{
		FText WarnMessage = FText::Format(LOCTEXT("NoOutputDataInterfaceFound", "No connected output data interface nodes found. Compilation for Graph: {0} aborted."), 
			FText::FromString(InNodeGraph->GetCollectionPath()));
		
		AddDiagnostic(EOptimusDiagnosticLevel::Warning, WarnMessage);
		return {};
	}

	TArray<FOptimusRoutedConstNode> ConnectedNodes;
	TMap<FOptimusRoutedConstNode, TArray<FOptimusRoutedConstNode>> NodeToInputNodes;
	TMap<FOptimusRoutedConstNode, TArray<FOptimusRoutedConstNode>> NodeToOutputNodes; 
	CollectNodes(TerminalNodes, ConnectedNodes, NodeToInputNodes, NodeToOutputNodes);

	// Since we now have the connected nodes in a breadth-first list, reverse the list which
	// will give use the same list but topologically sorted in kernel execution order.
	Algo::Reverse(ConnectedNodes.GetData(), ConnectedNodes.Num());

	// Go through all the nodes and check if their state is valid for compilation.
	bool bValidationFailed = false;
	for (FOptimusRoutedConstNode ConnectedNode: ConnectedNodes)
	{
		TOptional<FText> ErrorMessage = ConnectedNode.Node->ValidateForCompile(ConnectedNode.TraversalContext);
		if (ErrorMessage.IsSet())
		{
			bValidationFailed = true;
			AddDiagnostic(EOptimusDiagnosticLevel::Error, *ErrorMessage, ConnectedNode.Node);
		}
	}
	if (bValidationFailed)
	{
		return {};
	}

	// Mark zero-count loops as skippable
	TSet<FOptimusRoutedConstNode> LoopTerminalToSkip; 
	for (const FOptimusRoutedConstNode& ConnectedNode : ConnectedNodes )
	{
		if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(ConnectedNode.Node);
			LoopTerminal && LoopTerminal->GetLoopCount() == 0)
		{
			LoopTerminalToSkip.Add(ConnectedNode);
		}
	}

	// Collect looping kernels and skippable kernels
	TMap<FOptimusRoutedConstNode, TArray<FOptimusRoutedConstNode>> LoopEntryToKernelNodes; 
	TMap<FOptimusRoutedConstNode, FOptimusRoutedConstNode> KernelNodeToLoopEntry; 
	TSet<FOptimusRoutedConstNode> KernelToSkip; 

	for (const FOptimusRoutedConstNode& ConnectedNode : ConnectedNodes )
	{
		if (Cast<const IOptimusComputeKernelProvider>(ConnectedNode.Node))
		{
			TSet<FOptimusRoutedConstNode> LoopTerminals = ConnectedNode.Node->GetOwningGraph()->GetLoopEntryTerminalForNode(ConnectedNode.Node, ConnectedNode.TraversalContext);
			if (LoopTerminals.Num() > 0)
			{
				FOptimusRoutedConstNode LoopEntry= *LoopTerminals.CreateConstIterator();

				LoopEntryToKernelNodes.FindOrAdd(LoopEntry).Add(ConnectedNode);
				KernelNodeToLoopEntry.Add(ConnectedNode) = LoopEntry;

				if (LoopTerminalToSkip.Contains(LoopEntry))
				{
					KernelToSkip.Add(ConnectedNode);
				}
			}
		}
	}

	// Mark additional kernels with no meaningful output as skippable
	for (int32 Index = ConnectedNodes.Num() - 1; Index >= 0 ; Index-- )
	{
		const FOptimusRoutedConstNode& ConnectedNode = ConnectedNodes[Index];

		if (Cast<const IOptimusComputeKernelProvider>(ConnectedNode.Node))
		{
			bool bShouldSkip = true;
			for (const FOptimusRoutedConstNode& OutputNode : NodeToOutputNodes[ConnectedNode])
			{
				// Do not skip if this node has at least one meaningful output
				if (!KernelToSkip.Contains(OutputNode))
				{
					bShouldSkip = false;
				}
			}

			if (bShouldSkip)
			{
				KernelToSkip.Add(ConnectedNode);
			}
		}
	}

	TMap<FOptimusRoutedConstNodePin, FOptimusRoutedConstNodePin> LoopTerminalInputPinToSource;
	
	for (FOptimusRoutedConstNode ConnectedNode : ConnectedNodes)
	{
		if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(ConnectedNode.Node))
		{
			if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Return && LoopTerminalToSkip.Contains(ConnectedNode))
			{
				continue;
			}
			
			for (UOptimusNodePin* InputPin : LoopTerminal->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
			{
				FOptimusRoutedConstNodePin InputRoutedPin = {InputPin, ConnectedNode.TraversalContext};
				TOptional<FOptimusRoutedConstNodePin> SourcePin;
				
				FOptimusRoutedConstNodePin WorkPin = InputRoutedPin;	
				TQueue<FOptimusRoutedConstNodePin> PinQueue;
				TSet<FOptimusRoutedConstNodePin> VisitedPin;
				PinQueue.Enqueue(WorkPin);
				while(PinQueue.Dequeue(WorkPin))
				{
					if (!ensure(!VisitedPin.Contains(WorkPin)))
					{
						// Should not hit a cycle
						continue;
					}
					VisitedPin.Add(WorkPin);

					TArray<FOptimusRoutedNodePin> NextRoutedPin = WorkPin.NodePin->GetConnectedPinsWithRouting(WorkPin.TraversalContext);

					if (NextRoutedPin.Num() == 1)
					{
						UOptimusNodePin* NextPin = NextRoutedPin[0].NodePin;
						FOptimusRoutedConstNode NextRoutedNode = {NextPin->GetOwningNode(), NextRoutedPin[0].TraversalContext};
						if (const UOptimusNode_LoopTerminal* NextLoopTerminal = Cast<UOptimusNode_LoopTerminal>(NextRoutedNode.Node))
						{
							// Entry hitting a return
							if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry)
							{
								if (ensure(NextLoopTerminal->GetTerminalType() == EOptimusTerminalType::Return))
								{
									if (LoopTerminalToSkip.Contains(NextRoutedNode))
									{
										NextPin = NextLoopTerminal->GetPinCounterpart(NextPin, EOptimusTerminalType::Entry);
									}
									else
									{
										NextPin = NextLoopTerminal->GetPinCounterpart(NextPin, EOptimusTerminalType::Return);
									}
								}
							}
							// Return hitting an entry
							else
							{
								if (ensure(NextLoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry))
								{
									if (ensure(!LoopTerminalToSkip.Contains(NextRoutedNode)))
									{
										NextPin = NextLoopTerminal->GetPinCounterpart(NextPin, EOptimusTerminalType::Entry);		
									}
								}
							}

							PinQueue.Enqueue({NextPin,NextRoutedPin[0].TraversalContext});
						}
						else
						{
							SourcePin = {NextRoutedPin[0].NodePin,NextRoutedPin[0].TraversalContext};
						}
					}
				}

				if (SourcePin)
				{
					LoopTerminalInputPinToSource.Add(InputRoutedPin, *SourcePin);
				}
			}
		}
	}

	TSet<EOptimusNodeGraphType> GraphTypes;
	TMap<FOptimusRoutedConstNode, EOptimusNodeGraphType> KernelToGraphType; 
	
	for (FOptimusRoutedConstNode ConnectedNode : ConnectedNodes)
	{
		if (Cast<const IOptimusComputeKernelProvider>(ConnectedNode.Node))
		{
			if (InNodeGraph->GraphType != EOptimusNodeGraphType::Update)
			{
				KernelToGraphType.Add(ConnectedNode) = InNodeGraph->GraphType;
				GraphTypes.Add(InNodeGraph->GraphType);
			}
			else
			{
				if (ConnectedNode.Node->GetOwningGraph()->DoesNodeHaveMutableInput(ConnectedNode.Node, ConnectedNode.TraversalContext))
				{
					KernelToGraphType.Add(ConnectedNode) = EOptimusNodeGraphType::Update;
					GraphTypes.Add(EOptimusNodeGraphType::Update);
				}
				else
				{
					KernelToGraphType.Add(ConnectedNode) = EOptimusNodeGraphType::Setup;
					GraphTypes.Add(EOptimusNodeGraphType::Setup);
				}
			}
		}
	}

	// Instance looped nodes
	TArray<FOptimusInstancedNode> InstancedNodes;
	TMap<FOptimusRoutedConstNode, int32> NodeToMaxLoopIndex;

	for (const FOptimusRoutedConstNode& ConnectedNode : ConnectedNodes)
	{
		if (KernelToSkip.Contains(ConnectedNode))
		{
			continue;
		}

		if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(ConnectedNode.Node))
		{
			if (!LoopTerminalToSkip.Contains(ConnectedNode))
			{
				if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Return)
				{
					FOptimusRoutedConstNode LoopEntry = {LoopTerminal->GetOtherTerminal(), ConnectedNode.TraversalContext};

					// When the entry is disconnected from the return, finding looped kernels using LoopEntryToKernelNodes would fail
					if (const TArray<FOptimusRoutedConstNode>* LoopedKernelNodes = LoopEntryToKernelNodes.Find(LoopEntry))
					{
						for (int32 Index = 1; Index < LoopTerminal->GetLoopCount(); Index++)
						{
							for (const FOptimusRoutedConstNode& KernelNode : *LoopedKernelNodes)
							{
								InstancedNodes.Add({ KernelNode , Index });
							}
						}
						
						for (const FOptimusRoutedConstNode& KernelNode : *LoopedKernelNodes)
						{
							NodeToMaxLoopIndex.FindOrAdd(KernelNode) = LoopTerminal->GetLoopCount() - 1;
						}
					}
				}
			}
		}

		InstancedNodes.Add({ConnectedNode, 0});
		NodeToMaxLoopIndex.FindOrAdd(ConnectedNode) = 0;
	}

	// Create instanced links
	TMap<FOptimusInstancedPin, FOptimusInstancedPin> TargetPinToSourcePin;
	
	for (const FOptimusInstancedNode& InstancedNode : InstancedNodes )
	{
		const FOptimusRoutedConstNode& ThisRoutedNode = InstancedNode.RoutedNode;
		const UOptimusNode* ThisNode = ThisRoutedNode.Node;

		if (Cast<const IOptimusComputeKernelProvider>(ThisNode) || Cast<const IOptimusDataInterfaceProvider>(ThisNode))
		{
			for (const UOptimusNodePin* Pin: ThisRoutedNode.Node->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
			{
				if (Pin->IsGroupingPin())
				{
					continue;
				}

				FOptimusInstancedPin InstancedTargetPin = {InstancedNode, Pin};
				TArray<FOptimusRoutedNodePin> OtherPins = Pin->GetConnectedPinsWithRouting(ThisRoutedNode.TraversalContext);
				
				if (OtherPins.Num() == 1)
				{
					const UOptimusNodePin* OtherPin = OtherPins[0].NodePin;
					const UOptimusNode* OtherNode = OtherPin->GetOwningNode();

					if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(OtherNode);
						LoopTerminal && !OtherPin->GetDataDomain().IsSingleton())
					{
						// Looped resource pins require additional routing
						
						enum class EAddType
						{
							LastInstance,
							PreviousInstance,
							AllButLastInstance,
						};
						
						auto AddConnections = [&LoopTerminalInputPinToSource, &NodeToMaxLoopIndex, &TargetPinToSourcePin, InstancedTargetPin, &TraversalContext = OtherPins[0].TraversalContext] (UOptimusNodePin* InLoopTerminalInputPin, EAddType InType)
						{
							if (const FOptimusRoutedConstNodePin* SourcePin = LoopTerminalInputPinToSource.Find({InLoopTerminalInputPin, TraversalContext}))
							{
								const FOptimusRoutedConstNode SourceRoutedNode = {SourcePin->NodePin->GetOwningNode(), SourcePin->TraversalContext};

								if (InType != EAddType::AllButLastInstance)
								{
									int32 SourceLoopIndex = NodeToMaxLoopIndex[SourceRoutedNode];
									if (InType == EAddType::LastInstance)
									{
										SourceLoopIndex = NodeToMaxLoopIndex[SourceRoutedNode];
									}
									else if (InType == EAddType::PreviousInstance)
									{
										SourceLoopIndex = FMath::Clamp(InstancedTargetPin.InstancedNode.LoopIndex - 1, 0, NodeToMaxLoopIndex[SourceRoutedNode]);
									}
									const FOptimusInstancedPin InstancedSourcePin = {FOptimusInstancedNode(SourceRoutedNode, SourceLoopIndex), SourcePin->NodePin};
									TargetPinToSourcePin.Add(InstancedTargetPin) = InstancedSourcePin;
								}
								else if (InType == EAddType::AllButLastInstance) 
								{
									// The last Instance is excluded since something out of the loop links to it 
									for (int32 SourceLoopIndex = 0; SourceLoopIndex < NodeToMaxLoopIndex[SourceRoutedNode]; SourceLoopIndex++)
									{
										const FOptimusInstancedPin InstancedSourcePin = {FOptimusInstancedNode(SourceRoutedNode, SourceLoopIndex), SourcePin->NodePin};
										TargetPinToSourcePin.Add(InstancedTargetPin) = InstancedSourcePin;		
									} 
								}
								
							}	
						};

						FOptimusRoutedConstNode OtherRoutedNode = {OtherNode, OtherPins[0].TraversalContext};

						UOptimusNodePin* EntryInputPin = LoopTerminal->GetPinCounterpart(OtherPin, EOptimusTerminalType::Entry);
						UOptimusNodePin* ReturnInputPin = LoopTerminal->GetPinCounterpart(OtherPin, EOptimusTerminalType::Return);
						
						if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry)
						{
							if (!LoopTerminalToSkip.Contains(OtherRoutedNode))
							{
								if (Cast<const IOptimusDataInterfaceProvider>(ThisNode))
								{
									AddConnections(EntryInputPin, EAddType::LastInstance);

									if (LoopTerminal->GetLoopCount() > 1)
									{
										AddConnections(ReturnInputPin, EAddType::AllButLastInstance);	
									}
								
								}
								else if (Cast<const IOptimusComputeKernelProvider>(ThisNode))
								{
									if (InstancedNode.LoopIndex == 0)
									{
										AddConnections(EntryInputPin, EAddType::LastInstance);	
									}
									else
									{
										AddConnections(ReturnInputPin, EAddType::PreviousInstance);		
									}
								}	
							}
						}
						else
						{
							// InstancedNode.LoopIndex should be at its max (i.e. outside of a loop);
							
							UOptimusNodePin* LoopTerminalInputPin = nullptr;
							if (LoopTerminalToSkip.Contains(OtherRoutedNode))
							{
								LoopTerminalInputPin = EntryInputPin;
							}
							else
							{
								LoopTerminalInputPin = ReturnInputPin;
							}

							AddConnections(LoopTerminalInputPin, EAddType::LastInstance);
						}	
					}
					else
					{
						// Plain connections
						// 1. Kernel <-> Kernel, validation should make sure the source kernel is not looped or both kernel belong to the same loop
						// 2. Kernel <-> Data Interface
						// 3. Kernel -> index/count pin on Loop Terminals 
						const FOptimusRoutedConstNode SourceRoutedNode = {OtherNode, OtherPins[0].TraversalContext};
						const int32 SourceLoopIndex = FMath::Clamp(InstancedNode.LoopIndex, 0, NodeToMaxLoopIndex[SourceRoutedNode]);
						
						const FOptimusInstancedPin InstancedSourcePin = {FOptimusInstancedNode(SourceRoutedNode, SourceLoopIndex), OtherPin};
						TargetPinToSourcePin.Add(InstancedTargetPin) = InstancedSourcePin;	
					}
				}
			}
		}
	}
	
	TMap<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> SourcePinToTargetPins;
	for (TPair<FOptimusInstancedPin, FOptimusInstancedPin> LinkedPins : TargetPinToSourcePin)
	{
		SourcePinToTargetPins.FindOrAdd(LinkedPins.Value).Add(LinkedPins.Key);
	}

	TMap<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> LinksToInsertCopyKernel;
	for (TPair<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> OutputLink : SourcePinToTargetPins)
	{
		const FOptimusInstancedPin& SourcePin = OutputLink.Key;
		const UOptimusNode* SourceNode = SourcePin.InstancedNode.RoutedNode.Node;

		for (const FOptimusInstancedPin& TargetPin : OutputLink.Value)
		{
			const UOptimusNode* TargetNode = TargetPin.InstancedNode.RoutedNode.Node;

			if (Cast<const IOptimusDataInterfaceProvider>(TargetNode))
			{
				if (Cast<const IOptimusDataInterfaceProvider>(SourceNode))
				{
					LinksToInsertCopyKernel.FindOrAdd(SourcePin).Add(TargetPin);
				}
				else if (Cast<const IOptimusValueProvider>(SourceNode))
				{
					if (const IOptimusPropertyPinProvider* PropertyPinProvider = Cast<const IOptimusPropertyPinProvider>(TargetNode))
					{
						// Property pins are CPU only, no need to involve a kernel
						if (PropertyPinProvider->GetPropertyPins().Contains(TargetPin.Pin))
						{
							continue;
						}
					}
					
					LinksToInsertCopyKernel.FindOrAdd(SourcePin).Add(TargetPin);
				}
			}
		}
	}

	// Find all value nodes (constant and variable) 
	TArray<const UOptimusNode *> ActiveValueNodes;
	TMap<const UOptimusNode*, EOptimusValueUsage> ValueNodeUsageMap;
	TMap<FOptimusRoutedConstNode, FOptimusRoutedConstNode> ConstantNodeOverrideMap;

	// Propagate usage and override info backwards
	for (int32 Index = ConnectedNodes.Num() - 1; Index >= 0; Index--)
	{
		const FOptimusRoutedConstNode& ConnectedNode = ConnectedNodes[Index];
		const UOptimusNode* Node = ConnectedNode.Node;
		if (Cast<IOptimusValueProvider>(Node))
		{
			EOptimusValueUsage ValueUsage = EOptimusValueUsage::None;

			if (Cast<UOptimusNode_ConstantValue>(Node))
			{
				TArray<UOptimusNodePin*> InputPins = Node->GetPinsByDirection(EOptimusNodePinDirection::Input, true);
				TArray<FOptimusRoutedNodePin> SourcePins = InputPins[0]->GetConnectedPinsWithRouting(ConnectedNode.TraversalContext);
				if (SourcePins.Num() == 0)
				{
					// No overrider, this value node is active
					if (!ActiveValueNodes.Contains(Node))
					{
						ActiveValueNodes.Add(Node);
					}
				}
				else
				{
					// Save overrider info
					FOptimusRoutedConstNode SourceNode = {SourcePins[0].NodePin->GetOwningNode(), SourcePins[0].TraversalContext};
					ConstantNodeOverrideMap.Add(ConnectedNode, SourceNode);	
				}
			}
			else
			{
				// No overrider, this value node is active	
				if (!ActiveValueNodes.Contains(Node))
				{
					ActiveValueNodes.Add(Node);
				}	
			}
			
			TArray<UOptimusNodePin*> OutputPins = Node->GetPinsByDirection(EOptimusNodePinDirection::Output, false);
			check(OutputPins.Num() == 1);

			const UOptimusNodePin* OutputPin = OutputPins[0];
			TArray<FOptimusRoutedNodePin> OtherPins = OutputPin->GetConnectedPinsWithRouting(ConnectedNode.TraversalContext);

			for (const FOptimusRoutedNodePin& RoutedOtherPin : OtherPins)
			{
				const UOptimusNodePin* OtherPin = RoutedOtherPin.NodePin;
				const UOptimusNode* OtherNode = OtherPin->GetOwningNode();
				if (const IOptimusPropertyPinProvider* PropertyPinProvider = Cast<const IOptimusPropertyPinProvider>(OtherNode))
				{
					if (PropertyPinProvider->GetPropertyPins().Contains(OtherPin))
					{
						// At least one connection requesting value on CPU
						ValueUsage |= EOptimusValueUsage::CPU;
						continue;
					}
				}

				// Inherit usage from the nodes that this node overrides
				if (Cast<IOptimusValueProvider>(OtherNode))
				{
					ValueUsage |= ValueNodeUsageMap[OtherNode];
					continue;
				}
				
				// At least one connection requesting this value on GPU
				ValueUsage |= EOptimusValueUsage::GPU;
			}

			EOptimusValueUsage& ValueUsageRef = ValueNodeUsageMap.FindOrAdd(Node);
			ValueUsageRef |= ValueUsage;
		}
	}
	
	// Create all the data interfaces: node, graph, kernel outputs, loop terminal data
	
	// The component binding for the graph data is the primary binding on the deformer.
	UOptimusComponentSourceBinding* GraphDataComponentBinding = Bindings->Bindings[0];
	
	TMap<const UComputeDataInterface*, int32> DataInterfaceToBindingIndexMap;
	
	// Find all data interface nodes and create their data interfaces.
	TMap<FOptimusRoutedConstNode, UOptimusComputeDataInterface*> NodeDataInterfaceMap;
	
	TMap<FOptimusRoutedConstNode, TArray<UOptimusComputeDataInterface*>> LoopEntryToLoopDataInterfaces;
	
	for (const FOptimusRoutedConstNode& ConnectedNode : ConnectedNodes)
	{
		if (const IOptimusDataInterfaceProvider* NodeDataInterfaceProvider = Cast<const IOptimusDataInterfaceProvider>(ConnectedNode.Node))
		{
			// Gets a copy of node's data interface
			UOptimusComputeDataInterface* DataInterface = NodeDataInterfaceProvider->GetDataInterface(this);
			if (!DataInterface)
			{
				AddDiagnostic(EOptimusDiagnosticLevel::Error, LOCTEXT("NoDataInterfaceOnProvider", "No data interface object returned from node. Compilation aborted."));
				return {};
			}

			NodeDataInterfaceMap.Add(ConnectedNode, DataInterface);
			DataInterfaceToBindingIndexMap.Add(DataInterface) = NodeDataInterfaceProvider->GetComponentBinding(ConnectedNode.TraversalContext)->GetIndex();
		}
		else if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(ConnectedNode.Node))
		{
			if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry)
			{
				TArray<UOptimusComputeDataInterface*>& LoopDataInterfaces = LoopEntryToLoopDataInterfaces.Add(ConnectedNode);
				for (int32 LoopIndex = 0; LoopIndex < LoopTerminal->GetLoopCount(); LoopIndex++)
				{
					UOptimusLoopTerminalDataInterface* LoopDataInterface = NewObject<UOptimusLoopTerminalDataInterface>(this);
					LoopDataInterface->Index = LoopIndex;
					LoopDataInterface->Count = LoopTerminal->GetLoopCount();
					LoopDataInterfaces.Add(LoopDataInterface);
					// Loop Data are constant values for now so it does not matter which binding is used
					DataInterfaceToBindingIndexMap.Add(LoopDataInterface) = GraphDataComponentBinding->GetIndex();
				}
			}
		}
	}

	TMap<FOptimusValueIdentifier, FOptimusValueDescription> NodeGraphValueMap;

	// Create the graph data interface and fill it with the value nodes.
	UOptimusGraphDataInterface* GraphDataInterface = NewObject<UOptimusGraphDataInterface>(this);
	DataInterfaceToBindingIndexMap.Add(GraphDataInterface) = GraphDataComponentBinding->GetIndex();

	TArray<FOptimusGraphVariableDescription> ValueNodeDescriptions;
	ValueNodeDescriptions.Reserve(ActiveValueNodes.Num());	

	for (const UOptimusNode* ValueNode : ActiveValueNodes)
	{
		const IOptimusValueProvider* ValueProvider = CastChecked<IOptimusValueProvider>(ValueNode);
		FOptimusValueIdentifier ValueId = ValueProvider->GetValueIdentifier();
		EOptimusValueUsage ValueUsage = ValueNodeUsageMap[ValueNode];

		FOptimusValueDescription& Description = NodeGraphValueMap.FindOrAdd(ValueId);
		Description.DataType = ValueProvider->GetValueDataType();
		Description.ValueUsage |= ValueUsage;

		if (EnumHasAnyFlags(Description.ValueUsage, EOptimusValueUsage::CPU) && !Description.Value.IsInitialized())
		{
			Description.Value = ValueProvider->GetValue();
		}
		if (EnumHasAnyFlags(Description.ValueUsage, EOptimusValueUsage::GPU) && !Description.ShaderValue.IsValid())
		{
			Description.ShaderValue = ValueProvider->GetValue().GetShaderValue(ValueProvider->GetValueDataType());

			// Add GPU values to the graph data interface too, which receives value from the deformer instance
			// and we only want to add each value once, even when there are multiple value nodes all referencing the same value (mainly variable nodes)
			FOptimusGraphVariableDescription ValueNodeDescription;;
			ValueNodeDescription.ValueId = ValueProvider->GetValueIdentifier();
			ValueNodeDescription.Name = ValueNodeDescription.ValueId.Name.ToString();
			ValueNodeDescription.ValueType = ValueProvider->GetValueDataType()->ShaderValueType;

			ValueNodeDescriptions.Add(MoveTemp(ValueNodeDescription));
		}
	}
	GraphDataInterface->Init(ValueNodeDescriptions);

	
	TMap<FOptimusInstancedNode, UComputeDataInterface*> KernelDataInterfaceMap;
	TMap<FOptimusInstancedNode, FOptimus_KernelInputMap> KernelInputMap;
	TMap<FOptimusInstancedNode, FOptimus_KernelOutputMap> KernelOutputMap;
	
	// Kernel can either write to a buffer it creates on demand (transient, implicit-persistent raw buffer DIs), 
	// or directly write to some other data interfaces, this TMap tracks the latter and is used to make sure
	// copy kernels that copy data from these DIs only start after the writing kernel is done
	TMap<FOptimusInstancedNode, TArray<FOptimusInstancedNode>> KernelToDirectlyWrittenDataInterfaceNodeMap;
	
	TMap<FOptimusInstancedPin, UOptimusComputeDataInterface*> KernelOutputDataInterfaceMap;

	TMap<TWeakObjectPtr<const UComputeDataInterface>, FOptimusDataInterfacePropertyOverrideInfo> NodeGraphDataInterfacePropertyOverrideMap;

	auto GetRootValueProviderPin = [&ConstantNodeOverrideMap](const FOptimusRoutedConstNode& InStartNode)
	{
		const FOptimusRoutedConstNode* WorkItem = &InStartNode;
		while(const FOptimusRoutedConstNode* NextWorkItem = ConstantNodeOverrideMap.Find(*WorkItem))
		{
			WorkItem = NextWorkItem;
		}

		const TArray<UOptimusNodePin*> NewSourcePins = WorkItem->Node->GetPinsByDirection(EOptimusNodePinDirection::Output, false);
		check(NewSourcePins.Num() == 1);

		return FOptimusRoutedConstNodePin{NewSourcePins[0], WorkItem->TraversalContext};
	};
	
	for (const FOptimusInstancedNode& InstancedNode : InstancedNodes )
	{
		const FOptimusRoutedConstNode& RoutedNode = InstancedNode.RoutedNode;
		const UOptimusNode* Node = RoutedNode.Node;
		
		if (const IOptimusComputeKernelProvider* KernelProvider = Cast<const IOptimusComputeKernelProvider>(Node))
		{
			UComputeDataInterface* KernelDataInterface = KernelProvider->MakeKernelDataInterface(this);
			TSet<UOptimusComponentSourceBinding*> KernelPrimaryBindings = KernelProvider->GetPrimaryGroupPin()->GetComponentSourceBindingsRecursively(RoutedNode.TraversalContext);

			if (!ensure(KernelPrimaryBindings.Num() == 1))
			{
				AddDiagnostic(EOptimusDiagnosticLevel::Error,
					FText::Format(LOCTEXT("InvalidComponentBindingForKernel", "Missing or multiple component bindings found in primary group of a kernel ({0}). Compilation aborted."),
					Node->GetDisplayName()),
					Node);
				return {};
			}
			
			UOptimusComponentSourceBinding* KernelPrimaryBinding = *KernelPrimaryBindings.CreateConstIterator();
			int32 PrimaryBindingIndex = KernelPrimaryBinding->GetIndex();

			KernelDataInterfaceMap.Add(InstancedNode, KernelDataInterface);
			DataInterfaceToBindingIndexMap.Add(KernelDataInterface) = PrimaryBindingIndex;	

			KernelInputMap.Add(InstancedNode);
			KernelOutputMap.Add(InstancedNode);
			KernelToDirectlyWrittenDataInterfaceNodeMap.Add(InstancedNode);
			
			for (const UOptimusNodePin* Pin: Node->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
			{
				if (Pin->IsGroupingPin())
				{
					continue;
				}
				
				FOptimusInstancedPin InstancedPin = {InstancedNode, Pin};
				if (FOptimusInstancedPin* SourceInstancedPin = TargetPinToSourcePin.Find(InstancedPin))
				{
					const UOptimusNodePin* SourcePin = SourceInstancedPin->Pin;
					const FOptimusRoutedConstNode& SourceRoutedNode = SourceInstancedPin->InstancedNode.RoutedNode;
					
					if (Cast<const IOptimusValueProvider>(SourcePin->GetOwningNode()))
					{
						KernelInputMap[InstancedNode].Add(Pin) = {GraphDataInterface, GetRootValueProviderPin(SourceRoutedNode).NodePin};
					}
					else if (UOptimusComputeDataInterface** NodeDataInterface = NodeDataInterfaceMap.Find(SourceRoutedNode))
					{
						KernelInputMap[InstancedNode].Add(Pin) = {*NodeDataInterface, SourcePin};
					}
					else if (UOptimusComputeDataInterface** KernelOutputDataInterface = KernelOutputDataInterfaceMap.Find(*SourceInstancedPin))
					{
						KernelInputMap[InstancedNode].Add(Pin) = {*KernelOutputDataInterface, SourcePin};
					}
					else if (Cast<const UOptimusNode_LoopTerminal>(SourcePin->GetOwningNode()) && Pin->GetDataDomain().IsSingleton())
					{
						KernelInputMap[InstancedNode].Add(Pin) = {LoopEntryToLoopDataInterfaces[SourceRoutedNode][InstancedNode.LoopIndex] , SourcePin};
					}
				}
			}
			
			for (const UOptimusNodePin* Pin: Node->GetPinsByDirection(EOptimusNodePinDirection::Output, true))
			{
				if (ensure(!Pin->GetDataDomain().IsSingleton()))
				{
					FOptimusInstancedPin InstancedPin = {InstancedNode, Pin};

					bool bShouldCreateRawBuffer = false;
					bool bShouldUseImplicitPersistentDI = false;
					bool bShouldCopyToDataInterface = false;

					if (KernelProvider->DoesOutputPinSupportAtomic(Pin) || KernelProvider->DoesOutputPinSupportRead(Pin))
					{
						bShouldCreateRawBuffer = true;
						bShouldCopyToDataInterface = true;
					}

					TArray<FOptimusInstancedPin> TargetDataInterfacePins;

					if (TArray<FOptimusInstancedPin>* TargetInstancedPins = SourcePinToTargetPins.Find(InstancedPin))
					{
						for (const FOptimusInstancedPin& TargetInstancedPin : *TargetInstancedPins)
						{
							const FOptimusRoutedConstNode TargetRoutedNode = TargetInstancedPin.InstancedNode.RoutedNode;
							const UOptimusNode* TargetNode = TargetRoutedNode.Node;

							if (Cast<const IOptimusDataInterfaceProvider>(TargetNode))
							{
								if (InNodeGraph->GetGraphType() == EOptimusNodeGraphType::Update && 
									KernelToGraphType[RoutedNode] == EOptimusNodeGraphType::Setup)
								{
									bShouldCreateRawBuffer = true;
									bShouldUseImplicitPersistentDI = true;
									bShouldCopyToDataInterface = true;
								}

								TargetDataInterfacePins.Add(TargetInstancedPin);
							}
							else if (Cast<const IOptimusComputeKernelProvider>(TargetNode))
							{
								bShouldCreateRawBuffer = true;
								if (KernelToGraphType[RoutedNode] != KernelToGraphType[TargetRoutedNode])
								{
									bShouldUseImplicitPersistentDI = true;
								}
							}
						}
					}

					if (bShouldCreateRawBuffer)
					{
						UOptimusRawBufferDataInterface* RawBufferDI = nullptr;
						if (bShouldUseImplicitPersistentDI)
						{
							RawBufferDI = NewObject<UOptimusImplicitPersistentBufferDataInterface>(this);
							if (KernelProvider->DoesOutputPinSupportAtomic(Pin))
							{
								CastChecked<UOptimusImplicitPersistentBufferDataInterface>(RawBufferDI)->bZeroInitForAtomicWrites = true;
							}
						}
						else
						{
							RawBufferDI = NewObject<UOptimusTransientBufferDataInterface>(this);
							if (KernelProvider->DoesOutputPinSupportAtomic(Pin))
							{
								CastChecked<UOptimusTransientBufferDataInterface>(RawBufferDI)->bZeroInitForAtomicWrites = true;
							}
						}

						RawBufferDI->ValueType = Pin->GetDataType()->ShaderValueType;
						RawBufferDI->DataDomain = Pin->GetDataDomain();
						RawBufferDI->ComponentSourceBinding = KernelPrimaryBinding;

						KernelOutputDataInterfaceMap.Add(InstancedPin) = RawBufferDI;
						DataInterfaceToBindingIndexMap.Add(RawBufferDI) = PrimaryBindingIndex;

						// All connected kernels share the same raw buffer data interface
						KernelOutputMap[InstancedNode].FindOrAdd(Pin).Add({RawBufferDI, nullptr});
					}

					for (const FOptimusInstancedPin& TargetInstancedPin : TargetDataInterfacePins)
					{
						const FOptimusInstancedNode& TargetInstancedNode = TargetInstancedPin.InstancedNode;
						const FOptimusRoutedConstNode& TargetRoutedNode = TargetInstancedNode.RoutedNode;
						const UOptimusNodePin* TargetPin = TargetInstancedPin.Pin;

						if (bShouldCopyToDataInterface)
						{
							check(bShouldCreateRawBuffer);
							LinksToInsertCopyKernel.FindOrAdd(InstancedPin).Add(TargetInstancedPin);
						}
						else
						{
							if (UOptimusComputeDataInterface** NodeDataInterface = NodeDataInterfaceMap.Find(TargetRoutedNode))
							{
								KernelOutputMap[InstancedNode].FindOrAdd(Pin).Add({*NodeDataInterface, TargetPin});
								KernelToDirectlyWrittenDataInterfaceNodeMap[InstancedNode].Add(TargetInstancedNode);
							}
						}	
					}
				}
			}
		}
		
		if (const IOptimusPropertyPinProvider* PropertyPinProvider = Cast<const IOptimusPropertyPinProvider>(Node))
		{
			const UComputeDataInterface* ProviderDataInterface = nullptr;
			if (Cast<const IOptimusDataInterfaceProvider>(Node))
			{
				ProviderDataInterface = NodeDataInterfaceMap[RoutedNode];
			}
			else
			{
				// To be implemented when we have actual use cases
				check(false);
			}
		
			if (ensure(ProviderDataInterface))
			{
				for (UOptimusNodePin* Pin : PropertyPinProvider->GetPropertyPins())
				{
					FOptimusInstancedPin InstancedPin = {InstancedNode, Pin};
					if (FOptimusInstancedPin* SourceInstancedPin = TargetPinToSourcePin.Find(InstancedPin))
					{
						const UOptimusNode* SourceNode = SourceInstancedPin->Pin->GetOwningNode();

						if (Cast<IOptimusValueProvider>(SourceNode))
						{
							FOptimusRoutedConstNodePin RootValueProviderPin = GetRootValueProviderPin(SourceInstancedPin->InstancedNode.RoutedNode);

							IOptimusValueProvider* RootValueProvider = CastChecked<IOptimusValueProvider>(RootValueProviderPin.NodePin->GetOwningNode());
							
							NodeGraphDataInterfacePropertyOverrideMap.FindOrAdd(ProviderDataInterface)
									.PinNameToValueIdMap
									.Add(Pin->GetFName(),
										RootValueProvider->GetValueIdentifier());	
						}
					}
				}
			}
		}
	}

	if (!LinksToInsertCopyKernel.IsEmpty())
	{
		GraphTypes.Add(InNodeGraph->GetGraphType());
	}
	
	struct FDataInterfaceFunctionBinding
	{
		UComputeDataInterface* DataInterface;
		int32 FunctionIndex;
	};
	
	TMap<FOptimusInstancedPin, UComputeDataInterface*> CopyKernelDataInterfaceMap;
	TMap<FOptimusInstancedPin, FDataInterfaceFunctionBinding> CopyFromDataInterfaceMap;
	TMap<FOptimusInstancedPin, FDataInterfaceFunctionBinding> CopyToDataInterfaceMap;

	for (TPair<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> OutputLink : LinksToInsertCopyKernel)
	{
		const FOptimusInstancedPin& SourceInstancedPin = OutputLink.Key;
		const TArray<FOptimusInstancedPin>& TargetInstancedPins = OutputLink.Value;
		
		const UOptimusNode* SourceNode = SourceInstancedPin.Pin->GetOwningNode();
		const FOptimusRoutedConstNode& SourceRoutedNode = SourceInstancedPin.InstancedNode.RoutedNode;
		if (const IOptimusDataInterfaceProvider* InterfaceProvider = Cast<const IOptimusDataInterfaceProvider>(SourceNode))
		{
			FDataInterfaceFunctionBinding DataInterfaceBinding;
			DataInterfaceBinding.DataInterface = NodeDataInterfaceMap[SourceRoutedNode];
			DataInterfaceBinding.FunctionIndex = InterfaceProvider->GetDataFunctionIndexFromPin(SourceInstancedPin.Pin);
			CopyFromDataInterfaceMap.Add(SourceInstancedPin) = DataInterfaceBinding;
		}
		else if (const IOptimusValueProvider* ValueProvider = Cast<const IOptimusValueProvider>(SourceNode))
		{
			FDataInterfaceFunctionBinding DataInterfaceBinding;
			DataInterfaceBinding.DataInterface = GraphDataInterface;
			DataInterfaceBinding.FunctionIndex = GraphDataInterface->FindFunctionIndex(ValueProvider->GetValueIdentifier());
			CopyFromDataInterfaceMap.Add(SourceInstancedPin) = DataInterfaceBinding;	
		}
		else if (Cast<const IOptimusComputeKernelProvider>(SourceNode))
		{
			FDataInterfaceFunctionBinding DataInterfaceBinding;
			DataInterfaceBinding.DataInterface = KernelOutputDataInterfaceMap[SourceInstancedPin];
			DataInterfaceBinding.FunctionIndex = UOptimusRawBufferDataInterface::GetReadValueInputIndex(EOptimusBufferReadType::Default);

			CopyFromDataInterfaceMap.Add(SourceInstancedPin) = DataInterfaceBinding;
		}

		bool bIsCopyKernelDataInterfaceCreated = false;
		for (const FOptimusInstancedPin& TargetInstancedPin : TargetInstancedPins)
		{
			const UOptimusNode* TargetNode = TargetInstancedPin.Pin->GetOwningNode();
			const FOptimusRoutedConstNode& TargetRoutedNode = TargetInstancedPin.InstancedNode.RoutedNode;

			if (const IOptimusDataInterfaceProvider* InterfaceProvider = Cast<const IOptimusDataInterfaceProvider>(TargetNode);
				ensure(InterfaceProvider))
			{
				// One-time Initialization of the copy kernel based on the first target pin, because if source is a value provider, it does not
				// have a meaningful data domain and a meaningful component source binding
				if (!bIsCopyKernelDataInterfaceCreated)
				{
					bIsCopyKernelDataInterfaceCreated = true;
					
					UOptimusCopyKernelDataInterface* CopyKernelDataInterface = NewObject<UOptimusCopyKernelDataInterface>(this);
					CopyKernelDataInterface->SetExecutionDomain(*TargetInstancedPin.Pin->GetDataDomain().AsExpression());
					
					UOptimusComponentSourceBinding* Binding = InterfaceProvider->GetComponentBinding(TargetInstancedPin.InstancedNode.RoutedNode.TraversalContext);
					CopyKernelDataInterface->SetComponentBinding(Binding);
					CopyKernelDataInterfaceMap.Add(SourceInstancedPin) = CopyKernelDataInterface;
					DataInterfaceToBindingIndexMap.Add(CopyKernelDataInterface) = Binding->GetIndex();
				}
				
				
				FDataInterfaceFunctionBinding DataInterfaceBinding;
				DataInterfaceBinding.DataInterface = NodeDataInterfaceMap[TargetRoutedNode];
				DataInterfaceBinding.FunctionIndex = InterfaceProvider->GetDataFunctionIndexFromPin(TargetInstancedPin.Pin);
				CopyToDataInterfaceMap.Add(TargetInstancedPin) = DataInterfaceBinding;
			}
		}
	}

	FOptimusNodeGraphCompilationResult Result;
	Result.DataInterfacePropertyOverrideMap = NodeGraphDataInterfacePropertyOverrideMap;
	Result.ValueMap = NodeGraphValueMap;
	TArray<FOptimusComputeGraphInfo>& GraphInfos = Result.ComputeGraphInfos;
	for (EOptimusNodeGraphType GraphType : GraphTypes)
	{
		FString GraphName = InNodeGraph->GetName();
		if (GraphType != InNodeGraph->GraphType)
		{
			check(GraphType == EOptimusNodeGraphType::Setup);
			// Using "$" to avoid name clash with user provided graph name, see UOptimusNodeGraph::IsValidUserGraphName
			GraphName += TEXT("$Setup");
		}
		
		FOptimusComputeGraphInfo GraphInfo;
		// For trigger graph, this graph name needs to match the node graph name so that user can use the node graph name to trigger it.
		GraphInfo.GraphName = *GraphName;
		GraphInfo.GraphType = GraphType;
		// Avoid node graph and compute graph using the same name
		FString ComputeGraphName = GraphName + TEXT("_ComputeGraph");
		GraphInfo.ComputeGraph = NewObject<UOptimusComputeGraph>(this, *ComputeGraphName);

		if (GraphType != InNodeGraph->GraphType)
		{
			// Make sure generated graphs run before the user created graph
			check(GraphType == EOptimusNodeGraphType::Setup);
			GraphInfos.Insert(GraphInfo, 0);
		}
		else
		{
			GraphInfos.Add(GraphInfo);
		}
		
	}

	for (int32 GraphIndex = 0 ; GraphIndex < GraphInfos.Num(); GraphIndex++)
	{
		FOptimusComputeGraphInfo& GraphInfo = GraphInfos[GraphIndex];
		
		UOptimusComputeGraph* ComputeGraph = GraphInfo.ComputeGraph;
	
		// Create the binding objects.
		for (const UOptimusComponentSourceBinding* Binding: Bindings->Bindings)
		{
			ComputeGraph->Bindings.Add(Binding->GetComponentSource()->GetComponentClass());
		}

		// Now that we've collected all the pieces, time to line them up.
		for (const FOptimusInstancedNode& InstancedNode : InstancedNodes )
		{
			const FOptimusRoutedConstNode& ConnectedNode = InstancedNode.RoutedNode;

			if (Cast<const IOptimusComputeKernelProvider>(ConnectedNode.Node))
			{
				if (KernelToGraphType[ConnectedNode] != GraphInfo.GraphType)
				{
					continue;
				}
				
				const FOptimus_KernelInputMap& KernelInputs = KernelInputMap[InstancedNode];
				const FOptimus_KernelOutputMap& KernelOutputs = KernelOutputMap[InstancedNode];

				for (const TPair<const UOptimusNodePin*, FOptimus_KernelConnection>& Item : KernelInputs)
				{
					UComputeDataInterface* DataInterface = Item.Value.DataInterface;
					if (!ComputeGraph->DataInterfaces.Contains(DataInterface))
					{
						ComputeGraph->DataInterfaces.Add(DataInterface);
						ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[DataInterface]);
					}
				}

				for (const TPair<const UOptimusNodePin*, TArray<FOptimus_KernelConnection>>& Item : KernelOutputs)
				{
					for (const FOptimus_KernelConnection& Connection : Item.Value)
					{
						UComputeDataInterface* DataInterface = Connection.DataInterface;
						if (!ComputeGraph->DataInterfaces.Contains(DataInterface))
						{
							ComputeGraph->DataInterfaces.Add(DataInterface);
							ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[DataInterface]);
						}	
					}
				}

				UComputeDataInterface* KernelDataInterface = KernelDataInterfaceMap[InstancedNode];

				ComputeGraph->DataInterfaces.Add(KernelDataInterface);
				ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[KernelDataInterface]);
			}
		}

		if (InNodeGraph->GetGraphType() == GraphInfo.GraphType)
		{
			for (TPair<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> OutputLink : LinksToInsertCopyKernel)
			{
				const FOptimusInstancedPin& SourceInstancedPin = OutputLink.Key;	
				UComputeDataInterface* CopyKernelDataInterface = CopyKernelDataInterfaceMap[SourceInstancedPin];

				if (!ComputeGraph->DataInterfaces.Contains(CopyKernelDataInterface))
				{
					ComputeGraph->DataInterfaces.Add(CopyKernelDataInterface);
					ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[CopyKernelDataInterface]);
				}	

				const FDataInterfaceFunctionBinding& CopyFromBinding = CopyFromDataInterfaceMap[SourceInstancedPin];
				if (!ComputeGraph->DataInterfaces.Contains(CopyFromBinding.DataInterface))
				{
					ComputeGraph->DataInterfaces.Add(CopyFromBinding.DataInterface);
					ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[CopyFromBinding.DataInterface]);
				}

				for (int32 TargetIndex = 0 ; TargetIndex < OutputLink.Value.Num(); TargetIndex++)
				{
					const FOptimusInstancedPin& TargetInstancedPin = OutputLink.Value[TargetIndex];
					const FDataInterfaceFunctionBinding& CopyToBinding = CopyToDataInterfaceMap[TargetInstancedPin];

					if (!ComputeGraph->DataInterfaces.Contains(CopyToBinding.DataInterface))
					{
						ComputeGraph->DataInterfaces.Add(CopyToBinding.DataInterface);
						ComputeGraph->DataInterfaceToBinding.Add(DataInterfaceToBindingIndexMap[CopyToBinding.DataInterface]);
					}	
				}
			}
		}

		// Create bound kernels
		struct FKernelWithDataBindings
		{
			UComputeKernel *Kernel;
			FOptimus_InterfaceBindingMap InputDataBindings;
			FOptimus_InterfaceBindingMap OutputDataBindings;
		};

		TArray<FKernelWithDataBindings> BoundKernels;
		// Copy Kernel uses this map to look up the earliest point it can dispatch
		// It needs to be dispatched after the kernels producing its inputs and before the kernels consuming its outputs
		// This map tracks which kernel produces the input that the copy kernel is copying from
		TMap<FOptimusInstancedNode, UComputeKernel*> ReadableNodeToProducerComputeKernel;
		
		for (const FOptimusInstancedNode& InstancedNode : InstancedNodes )
		{
			const FOptimusRoutedConstNode& ConnectedNode = InstancedNode.RoutedNode;

			if (const IOptimusComputeKernelProvider* KernelProvider = Cast<const IOptimusComputeKernelProvider>(ConnectedNode.Node))
			{
				if (KernelToGraphType[ConnectedNode] != GraphInfo.GraphType)
				{
					continue;
				}
				
				FKernelWithDataBindings BoundKernel;

				BoundKernel.Kernel = NewObject<UComputeKernel>(this);

				UComputeDataInterface* KernelDataInterface = KernelDataInterfaceMap[InstancedNode];

				const FOptimus_KernelInputMap& KernelInputs = KernelInputMap[InstancedNode];
				const FOptimus_KernelOutputMap& KernelOutputs = KernelOutputMap[InstancedNode];
				
				FOptimus_ComputeKernelResult KernelSourceResult = KernelProvider->CreateComputeKernel(	
					BoundKernel.Kernel, ConnectedNode.TraversalContext,
					KernelInputs, KernelOutputs,
					KernelDataInterface,
					BoundKernel.InputDataBindings, BoundKernel.OutputDataBindings
				);

				ReadableNodeToProducerComputeKernel.Add(InstancedNode, BoundKernel.Kernel);

				const TArray<FOptimusInstancedNode>& WrittenDataInterfaceNodes = KernelToDirectlyWrittenDataInterfaceNodeMap[InstancedNode];
				for (const FOptimusInstancedNode& WrittenNode : WrittenDataInterfaceNodes)
				{
					// This node is the only data interface provider that has both GPU input and GPU output, and only one of each
					if (Cast<UOptimusNode_Resource>(WrittenNode.RoutedNode.Node))
					{
						check(!ReadableNodeToProducerComputeKernel.Contains(WrittenNode));
						ReadableNodeToProducerComputeKernel.Add(WrittenNode, BoundKernel.Kernel);
					}
				}
				
				if (FText* ErrorMessage = KernelSourceResult.TryGet<FText>())
				{
					AddDiagnostic(EOptimusDiagnosticLevel::Error,
						FText::Format(LOCTEXT("CantCreateKernelWithError", "{0}. Compilation aborted."), *ErrorMessage),
						ConnectedNode.Node);
					return {};
				}

				if (BoundKernel.InputDataBindings.IsEmpty() || BoundKernel.OutputDataBindings.IsEmpty())
				{
					AddDiagnostic(EOptimusDiagnosticLevel::Error,
					LOCTEXT("KernelHasNoBindings", "Kernel has either no input or output bindings. Compilation aborted."),
						ConnectedNode.Node);
					return {};
				}

				bool bHasExecution = false;
				for (const TPair<int32, FOptimus_InterfaceBinding>& DataBinding: BoundKernel.InputDataBindings)
				{
					const FOptimus_InterfaceBinding& InterfaceBinding = DataBinding.Value;
					const UComputeDataInterface* DataInterface = InterfaceBinding.DataInterface;
					if (DataInterface->IsExecutionInterface())
					{
						bHasExecution = true;
						break;
					}
				}
				
				if (!bHasExecution)
				{
					AddDiagnostic(EOptimusDiagnosticLevel::Error,
					LOCTEXT("KernelHasNoExecutionDataInterface", "Kernel has no execution data interface connected. Compilation aborted."),
						ConnectedNode.Node);
					return {};
				}
				
				BoundKernel.Kernel->KernelSource = KernelSourceResult.Get<UOptimusKernelSource*>();

				BoundKernels.Add(BoundKernel);
				ComputeGraph->KernelInvocations.Add(BoundKernel.Kernel);
				ComputeGraph->KernelToNode.Add(ConnectedNode.Node);
			}
		}

		
		if (InNodeGraph->GetGraphType() == GraphInfo.GraphType)
		{
			TArray<FKernelWithDataBindings> BoundCopyKernels;
			
			bool bIsUnifiedDispatch = true;
			for (TPair<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> OutputLink : LinksToInsertCopyKernel)
			{
				// Create a copy kernel per source pin, that copies from 1 source pin to multiple target pins
				const FOptimusInstancedPin& SourceInstancedPin = OutputLink.Key;

				
				
				const FShaderValueTypeHandle ValueType = SourceInstancedPin.Pin->GetDataType()->ShaderValueType;
				
				FKernelWithDataBindings BoundCopyKernel;

				BoundCopyKernel.Kernel = NewObject<UComputeKernel>(this);

				FOptimus_InterfaceBindingMap& InputDataBindings = BoundCopyKernel.InputDataBindings;
				FOptimus_InterfaceBindingMap& OutputDataBindings = BoundCopyKernel.OutputDataBindings;

				UOptimusKernelSource* KernelSource = NewObject<UOptimusKernelSource>(BoundCopyKernel.Kernel);
				UComputeDataInterface* CopyKernelDataInterface = CopyKernelDataInterfaceMap[SourceInstancedPin];
				
				FString SourceText;
				SourceText = FString::Printf(
					TEXT("if (Index >= %s::%s().x) return;\n"),
					Optimus::GetKernelInternalNamespaceName(),
					CastChecked<IOptimusComputeKernelDataInterface>(CopyKernelDataInterface)->GetReadNumThreadsFunctionName());
				
				{
					TArray<FShaderFunctionDefinition> Functions;
					CopyKernelDataInterface->GetSupportedInputs(Functions);
					// Simply grab everything the kernel data interface has to offer
					for (int32 FuncIndex = 0; FuncIndex < Functions.Num(); FuncIndex++)
					{
						FShaderFunctionDefinition FuncDef = Functions[FuncIndex];
						for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
						{
							// Making sure parameter has type declaration generated
							ParamType.ResetTypeDeclaration();
						}

						FOptimus_InterfaceBinding InterfaceBinding;
						InterfaceBinding.DataInterface = CopyKernelDataInterface;
						InterfaceBinding.DataInterfaceBindingIndex = FuncIndex;
						InterfaceBinding.BindingFunctionName = FuncDef.Name;
						InterfaceBinding.BindingFunctionNamespace = Optimus::GetKernelInternalNamespaceName();
				
						InputDataBindings.Add(KernelSource->ExternalInputs.Num(), InterfaceBinding);  
				
						KernelSource->ExternalInputs.Emplace(FuncDef);	
					}	
				}
				

				{
					const FDataInterfaceFunctionBinding& CopyFromBinding = CopyFromDataInterfaceMap[SourceInstancedPin];

					if (!CopyFromBinding.DataInterface->CanSupportUnifiedDispatch())
					{
						bIsUnifiedDispatch = false;
					}
					
					TArray<FShaderFunctionDefinition> Functions;
					CopyFromBinding.DataInterface->GetSupportedInputs(Functions);
					FShaderFunctionDefinition FuncDef = Functions[CopyFromBinding.FunctionIndex];

					for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
					{
						ParamType.ResetTypeDeclaration();
					}
				
					FOptimus_InterfaceBinding InterfaceBinding;
					InterfaceBinding.DataInterface = CopyFromBinding.DataInterface;
					InterfaceBinding.DataInterfaceBindingIndex = CopyFromBinding.FunctionIndex;
					InterfaceBinding.BindingFunctionName = FString::Printf(TEXT("Read%s"), *SourceInstancedPin.Pin->GetName());
					InterfaceBinding.BindingFunctionNamespace = FString();
		
					InputDataBindings.Add(KernelSource->ExternalInputs.Num(), InterfaceBinding);  
		
					KernelSource->ExternalInputs.Emplace(FuncDef);

					FString IndexString;
					if (FuncDef.ParamTypes.Num() == 2)
					{
						IndexString = TEXT("Index");
					}
					SourceText += FString::Printf(TEXT("%s Value = %s(%s);\n"), *ValueType->ToString(), *InterfaceBinding.BindingFunctionName, *IndexString);
				}
				
				for (int32 TargetIndex = 0 ; TargetIndex < OutputLink.Value.Num(); TargetIndex++)
				{
					const FOptimusInstancedPin& TargetInstancedPin = OutputLink.Value[TargetIndex];

					// This node is the only data interface provider that has both GPU input and GPU output, and only one of each
					if (Cast<UOptimusNode_Resource>(TargetInstancedPin.Pin->GetOwningNode()))
					{
						check(!ReadableNodeToProducerComputeKernel.Contains(TargetInstancedPin.InstancedNode));
						ReadableNodeToProducerComputeKernel.Add(TargetInstancedPin.InstancedNode, BoundCopyKernel.Kernel);
					}
					
					const FDataInterfaceFunctionBinding& CopyToBinding = CopyToDataInterfaceMap[TargetInstancedPin];

					if (!CopyToBinding.DataInterface->CanSupportUnifiedDispatch())
					{
						bIsUnifiedDispatch = false;
					}
					
					TArray<FShaderFunctionDefinition> Functions;
					CopyToBinding.DataInterface->GetSupportedOutputs(Functions);
					FShaderFunctionDefinition FuncDef = Functions[CopyToBinding.FunctionIndex];

					for (FShaderParamTypeDefinition& ParamType : FuncDef.ParamTypes)
					{
						ParamType.ResetTypeDeclaration();
					}
				
					FOptimus_InterfaceBinding InterfaceBinding;
					InterfaceBinding.DataInterface = CopyToBinding.DataInterface;
					InterfaceBinding.DataInterfaceBindingIndex = CopyToBinding.FunctionIndex;
					InterfaceBinding.BindingFunctionName = FString::Printf(TEXT("Write_%d_%s"), TargetIndex, *TargetInstancedPin.Pin->GetName());
					InterfaceBinding.BindingFunctionNamespace = FString();
		
					OutputDataBindings.Add(KernelSource->ExternalOutputs.Num(), InterfaceBinding);  
		
					KernelSource->ExternalOutputs.Emplace(FuncDef);

					SourceText += FString::Printf(TEXT("%s(Index, Value);\n"), *InterfaceBinding.BindingFunctionName);
				}

				static const FString CopyKernelName = TEXT("CopyKernel");
				static const FIntVector GroupSize = FIntVector(64, 1, 1);
				using UKernelDI = UOptimusCopyKernelDataInterface;
				FString CookedSource = Optimus::GetCookedKernelSource(
					BoundCopyKernel.Kernel->GetPathName(),
					SourceText,
					CopyKernelName,
					GroupSize,
					UKernelDI::StaticClass()->GetDefaultObject<UKernelDI>()->GetReadNumThreadsPerInvocationFunctionName(),
					UKernelDI::StaticClass()->GetDefaultObject<UKernelDI>()->GetReadThreadIndexOffsetFunctionName(),
					bIsUnifiedDispatch
					);
				KernelSource->SetSource(CookedSource);
				KernelSource->EntryPoint = CopyKernelName;
				KernelSource->GroupSize = GroupSize;
				BoundCopyKernel.Kernel->KernelSource = KernelSource;

				BoundCopyKernels.Add(BoundCopyKernel);
				
			}

			TArray<UComputeKernel*> InsertAfterComputeKernelLookUpArray;
			for (TPair<FOptimusInstancedPin, TArray<FOptimusInstancedPin>> OutputLink : LinksToInsertCopyKernel)
			{
				const FOptimusInstancedPin& SourceInstancedPin = OutputLink.Key;
				
				// Indicates that this copy kernel should run immediately after the found compute kernel, which can also be another copy kernel
				// Note: nullptr means that the copy kernel has no kernel node dependency and thus should run before everything else
				UComputeKernel* InsertAfterComputeKernel = nullptr;
				if (UComputeKernel** ComputeKnernel = ReadableNodeToProducerComputeKernel.Find(SourceInstancedPin.InstancedNode))
				{
					InsertAfterComputeKernel = *ComputeKnernel;
				}	
				InsertAfterComputeKernelLookUpArray.Add(InsertAfterComputeKernel);
			}

			check(BoundCopyKernels.Num() == InsertAfterComputeKernelLookUpArray.Num());


			TOptional<int32> NumLastInserted;
			TSet<int32> InsertedCopyKernels;
			while(InsertedCopyKernels.Num() < BoundCopyKernels.Num())
			{
				// Avoid infinite loop, in case of unexpected edge case or bad data
				if (NumLastInserted.IsSet() )
				{
					bool bMadeProgress = (InsertedCopyKernels.Num() - *NumLastInserted) > 0;
					if (!ensure(bMadeProgress))
					{
						return {};	
					}
				}
				NumLastInserted = InsertedCopyKernels.Num();
					
				for (int32 Index = 0 ; Index < BoundCopyKernels.Num() ; Index++)
				{
					if (InsertedCopyKernels.Contains(Index))
					{
						continue;
					}

					const FKernelWithDataBindings& BoundCopyKernel = BoundCopyKernels[Index];
					UComputeKernel* InsertAfterComputeKernel = InsertAfterComputeKernelLookUpArray[Index];

					int32 InsertIndex = INDEX_NONE;
					if (InsertAfterComputeKernel == nullptr)
					{
						InsertIndex = 0;
					}
					else
					{
						int32 BoundKernelIndex = BoundKernels.IndexOfByPredicate([InsertAfterComputeKernel]( const FKernelWithDataBindings& InBoundKernel)
						{
							return InBoundKernel.Kernel == InsertAfterComputeKernel;
						});
						
						if (BoundKernelIndex == INDEX_NONE)
						{
							// Skip since this copy kernel's dependency hasn't been inserted
							continue;
						}

						// By Default insert after the kernel producing the data that this copy kernel is copying from
						InsertIndex = BoundKernelIndex + 1;
					}
					
					
					if (ensure(InsertIndex >= 0) && ensure(InsertIndex <= BoundKernels.Num()))
					{
						BoundKernels.Insert(BoundCopyKernel, InsertIndex);
						ComputeGraph->KernelInvocations.Insert(BoundCopyKernel.Kernel, InsertIndex);
						ComputeGraph->KernelToNode.Insert(nullptr, InsertIndex);
					}

					InsertedCopyKernels.Add(Index);	
				}
			}
		}

		check(ComputeGraph->KernelInvocations.Num() == BoundKernels.Num());

		// Create the graph edges.
		for (int32 KernelIndex = 0; KernelIndex < ComputeGraph->KernelInvocations.Num(); KernelIndex++)
		{
			const FKernelWithDataBindings& BoundKernel = BoundKernels[KernelIndex];
			const TArray<FShaderFunctionDefinition>& KernelInputs = BoundKernel.Kernel->KernelSource->ExternalInputs;

			// FIXME: Hoist these two loops into a helper function/lambda.
			for (const TPair<int32, FOptimus_InterfaceBinding>& DataBinding: BoundKernel.InputDataBindings)
			{
				const int32 KernelBindingIndex = DataBinding.Key;
				const FOptimus_InterfaceBinding& InterfaceBinding = DataBinding.Value;
				const UComputeDataInterface* DataInterface = InterfaceBinding.DataInterface;
				const int32 DataInterfaceBindingIndex = InterfaceBinding.DataInterfaceBindingIndex;
				const FString BindingFunctionName = InterfaceBinding.BindingFunctionName;
				const FString BindingFunctionNamespace = InterfaceBinding.BindingFunctionNamespace;

				// FIXME: Collect this beforehand.
				TArray<FShaderFunctionDefinition> DataInterfaceFunctions;
				DataInterface->GetSupportedInputs(DataInterfaceFunctions);
				
				if (ensure(KernelInputs.IsValidIndex(KernelBindingIndex)) &&
					ensure(DataInterfaceFunctions.IsValidIndex(DataInterfaceBindingIndex)))
				{
					FComputeGraphEdge GraphEdge;
					GraphEdge.bKernelInput = true;
					GraphEdge.KernelIndex = KernelIndex;
					GraphEdge.KernelBindingIndex = KernelBindingIndex;
					GraphEdge.DataInterfaceIndex = ComputeGraph->DataInterfaces.IndexOfByKey(DataInterface);
					GraphEdge.DataInterfaceBindingIndex = DataInterfaceBindingIndex;
					GraphEdge.BindingFunctionNameOverride = BindingFunctionName;
					GraphEdge.BindingFunctionNamespace = BindingFunctionNamespace;
					ComputeGraph->GraphEdges.Add(GraphEdge);
				}
			}

			const TArray<FShaderFunctionDefinition>& KernelOutputs = BoundKernels[KernelIndex].Kernel->KernelSource->ExternalOutputs;
			for (const TPair<int32, FOptimus_InterfaceBinding>& DataBinding: BoundKernel.OutputDataBindings)
			{
				const int32 KernelBindingIndex = DataBinding.Key;
				const FOptimus_InterfaceBinding& InterfaceBinding = DataBinding.Value;
				const UComputeDataInterface* DataInterface = InterfaceBinding.DataInterface;
				const int32 DataInterfaceBindingIndex = InterfaceBinding.DataInterfaceBindingIndex;
				const FString BindingFunctionName = InterfaceBinding.BindingFunctionName;
				const FString BindingFunctionNamespace = InterfaceBinding.BindingFunctionNamespace;

				// FIXME: Collect this beforehand.
				TArray<FShaderFunctionDefinition> DataInterfaceFunctions;
				DataInterface->GetSupportedOutputs(DataInterfaceFunctions);
				
				if (ensure(KernelOutputs.IsValidIndex(KernelBindingIndex)) &&
					ensure(DataInterfaceFunctions.IsValidIndex(DataInterfaceBindingIndex)))
				{
					FComputeGraphEdge GraphEdge;
					GraphEdge.bKernelInput = false;
					GraphEdge.KernelIndex = KernelIndex;
					GraphEdge.KernelBindingIndex = KernelBindingIndex;
					GraphEdge.DataInterfaceIndex = ComputeGraph->DataInterfaces.IndexOfByKey(DataInterface);
					GraphEdge.DataInterfaceBindingIndex = DataInterfaceBindingIndex;
					GraphEdge.BindingFunctionNameOverride = BindingFunctionName;
					GraphEdge.BindingFunctionNamespace = BindingFunctionNamespace;
					ComputeGraph->GraphEdges.Add(GraphEdge);
				}
			}
		}	
	}
	

#if PRINT_COMPILED_OUTPUT
	
#endif

	return Result;
}
#endif // WITH_EDITOR

void UOptimusDeformer::OnDataTypeChanged(FName InTypeName)
{
	for (UOptimusNodeGraph* Graph : Graphs)
	{
		for (UOptimusNode* Node: Graph->Nodes)
		{
			Node->OnDataTypeChanged(InTypeName);
		}
	}

	//TODO: Recreate variables/Resources that uses this type

	// Once we updated the deformer instance, we need to make sure the editor is aware as well
	Notify(EOptimusGlobalNotifyType::DataTypeChanged, nullptr);
}

template<typename Allocator>
static void StringViewSplit(
	TArray<FStringView, Allocator> &OutResult, 
	const FStringView InString,
	const TCHAR* InDelimiter,
	int32 InMaxSplit = INT32_MAX
	)
{
	if (!InDelimiter)
	{
		OutResult.Add(InString);
		return;
	}
	
	const int32 DelimiterLength = FCString::Strlen(InDelimiter); 
	if (DelimiterLength == 0)
	{
		OutResult.Add(InString);
		return;
	}

	InMaxSplit = FMath::Max(0, InMaxSplit);

	int32 StartIndex = 0;
	for (;;)
	{
		const int32 FoundIndex = (InMaxSplit--) ? InString.Find(InDelimiter, StartIndex) : INDEX_NONE;
		if (FoundIndex == INDEX_NONE)
		{
			OutResult.Add(InString.SubStr(StartIndex, INT32_MAX));
			break;
		}

		OutResult.Add(InString.SubStr(StartIndex, FoundIndex - StartIndex));
		StartIndex = FoundIndex + DelimiterLength;
	}
}


UOptimusNodeGraph* UOptimusDeformer::ResolveGraphPath(
	const FStringView InPath,
	FStringView& OutRemainingPath
	) const
{
	TArray<FStringView, TInlineAllocator<4>> Path;
	StringViewSplit<TInlineAllocator<4>>(Path, InPath, TEXT("/"));

	if (Path.Num() == 0)
	{
		return nullptr;
	}
	int32 SubGraphStartIndex = 0;
	
	UOptimusNodeGraph* Graph = nullptr;
	
	for (UOptimusNodeGraph* RootGraph : Graphs)
	{
		if (Path[0].Equals(RootGraph->GetName(), ESearchCase::IgnoreCase))
		{
			SubGraphStartIndex = 1;
			Graph = RootGraph;
			break;
		}
	}

	if (!Graph)
	{
		return nullptr;
	}

	// See if we need to traverse any sub-graphs
	for (; SubGraphStartIndex < Path.Num(); SubGraphStartIndex++)
	{
		bool bFoundSubGraph = false;
		for (UOptimusNodeGraph* SubGraph: Graph->GetGraphs())
		{
			if (Path[SubGraphStartIndex].Equals(SubGraph->GetName(), ESearchCase::IgnoreCase))
			{
				Graph = SubGraph;
				bFoundSubGraph = true;
				break;
			}
		}
		if (!bFoundSubGraph)
		{
			break;
		}
	}

	if (SubGraphStartIndex < Path.Num())
	{
		OutRemainingPath = FStringView(
			Path[SubGraphStartIndex].GetData(),
			static_cast<int32>(Path.Last().GetData() - Path[SubGraphStartIndex].GetData()) + Path.Last().Len());
	}
	else
	{
		OutRemainingPath.Reset();
	}

	return Graph;
}


UOptimusNode* UOptimusDeformer::ResolveNodePath(
	const FStringView InPath,
	FStringView& OutRemainingPath
	) const
{
	FStringView NodePath;

	UOptimusNodeGraph* Graph = ResolveGraphPath(InPath, NodePath);
	if (!Graph || NodePath.IsEmpty())
	{
		return nullptr;
	}

	// We only want at most 2 elements (single split)
	TArray<FStringView, TInlineAllocator<2>> Path;
	StringViewSplit(Path, NodePath, TEXT("."), 1);
	if (Path.IsEmpty())
	{
		return nullptr;
	}

	const FStringView NodeName = Path[0];
	for (UOptimusNode* Node : Graph->GetAllNodes())
	{
		if (Node != nullptr && NodeName.Equals(Node->GetName(), ESearchCase::IgnoreCase))
		{
			OutRemainingPath = Path.Num() == 2 ? Path[1] : FStringView();
			return Node;
		}
	}

	return nullptr;
}


void UOptimusDeformer::Notify(EOptimusGlobalNotifyType InNotifyType, UObject* InObject) const
{
	switch (InNotifyType)
	{
	case EOptimusGlobalNotifyType::GraphAdded: 
	case EOptimusGlobalNotifyType::GraphIndexChanged:
		checkSlow(Cast<UOptimusNodeGraph>(InObject) != nullptr);
		break;
		
	case EOptimusGlobalNotifyType::GraphRemoved:
	case EOptimusGlobalNotifyType::GraphRenamed:
		{
			UOptimusNodeGraph* Graph = CastChecked<UOptimusNodeGraph>(InObject);
			if (ensure(Graph))
			{
				OnGraphRenamedOrRemoved(Graph);
			}

			break;
		}
	case EOptimusGlobalNotifyType::ComponentBindingAdded:
	case EOptimusGlobalNotifyType::ComponentBindingRemoved:
	case EOptimusGlobalNotifyType::ComponentBindingIndexChanged:
	case EOptimusGlobalNotifyType::ComponentBindingRenamed:
	case EOptimusGlobalNotifyType::ComponentBindingSourceChanged:
		checkSlow(Cast<UOptimusComponentSourceBinding>(InObject) != nullptr);
		break;
		
	case EOptimusGlobalNotifyType::ResourceAdded:
	case EOptimusGlobalNotifyType::ResourceRemoved:
	case EOptimusGlobalNotifyType::ResourceIndexChanged:
	case EOptimusGlobalNotifyType::ResourceRenamed:
	case EOptimusGlobalNotifyType::ResourceTypeChanged:
	case EOptimusGlobalNotifyType::ResourceDomainChanged:
		checkSlow(Cast<UOptimusResourceDescription>(InObject) != nullptr);
		break;

	case EOptimusGlobalNotifyType::VariableAdded:
	case EOptimusGlobalNotifyType::VariableRemoved:
	case EOptimusGlobalNotifyType::VariableIndexChanged:
	case EOptimusGlobalNotifyType::VariableRenamed:
	case EOptimusGlobalNotifyType::VariableTypeChanged:
		checkSlow(Cast<UOptimusVariableDescription>(InObject) != nullptr);
		break;

	case EOptimusGlobalNotifyType::ConstantValueChanged:
		if (UOptimusNode_ConstantValue* ConstantValue = Cast<UOptimusNode_ConstantValue>(InObject))
		{
			ConstantValueUpdateDelegate.Broadcast(ConstantValue, ConstantValue->GetValue());
		}
		
		break;
	default:
		checkfSlow(false, TEXT("Unchecked EOptimusGlobalNotifyType!"));
		break;
	}

	const_cast<UOptimusDeformer*>(this)->MarkModified();
	
	GlobalNotifyDelegate.Broadcast(InNotifyType, InObject);
}


void UOptimusDeformer::MarkModified()
{
	if (Status != EOptimusDeformerStatus::HasErrors)
	{
		Status = EOptimusDeformerStatus::Modified;
	}
}

void UOptimusDeformer::SetAllInstancesCanbeActive(bool bInCanBeActive) const
{
	SetAllInstancesCanbeActiveDelegate.Broadcast(bInCanBeActive);
}

void UOptimusDeformer::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Mark with a custom version. This has the nice side-benefit of making the asset indexer
	// skip this object if the plugin is not loaded.
	Ar.UsingCustomVersion(FOptimusObjectVersion::GUID);

	// UComputeGraph stored the number of kernels separately, we need to skip over it or the
	// stream is out of sync.
	if (Ar.CustomVer(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::SwitchToMeshDeformerBase)
	{
		int32 NumKernels = 0;
		Ar << NumKernels;
		for (int32 Index = 0; Index < NumKernels; Index++)
		{
			int32 NumResources = 0;
			Ar << NumResources;

			// If this turns out to be not zero in some asset, we have to add in the entirety
			// of FComputeKernelResource::SerializeShaderMap
			check(NumResources == 0); 
		}
	}
}


void UOptimusDeformer::PostLoad()
{
	Super::PostLoad();

	for (UOptimusVariableDescription* VariableDescription : Variables->Descriptions)
	{
		VariableDescription->ConditionalPostLoad();
	}
	
	// PostLoad everything first before changing anything for back compat
	// Each graph postloads everything it owns
	for (UOptimusNodeGraph* Graph: GetGraphs())
	{
		Graph->ConditionalPostLoad();
	}

	for (FOptimusComputeGraphInfo& Info : ComputeGraphs)
	{
		Info.ComputeGraph->ConditionalPostLoad();
	}
	
	// Fixup any empty array entries.
	Resources->Descriptions.RemoveAllSwap([](const TObjectPtr<UOptimusResourceDescription>& Value) { return Value == nullptr; });
	Variables->Descriptions.RemoveAllSwap([](const TObjectPtr<UOptimusVariableDescription>& Value) { return Value == nullptr; });

	// Fixup any class objects with invalid parents.
	TArray<UObject*> Objects;
	GetObjectsWithOuter(this, Objects, EGetObjectsFlags::None);

	for (UObject* Object : Objects)
	{
		if (UClass* ClassObject = Cast<UClass>(Object))
		{
			Optimus::RenameObject(ClassObject, nullptr, GetPackage());
		}
	}

#if WITH_EDITOR
	FOptimusObjectVersion::Type ObjectVersion = static_cast<FOptimusObjectVersion::Type>(GetLinkerCustomVersion(FOptimusObjectVersion::GUID));
	if (ObjectVersion < FOptimusObjectVersion::ReparentResourcesAndVariables)
	{
		// Move any resource or variable descriptor owned by this deformer to their own container.
		// This is to fix a bug where variables/resources were put in their respective container
		// but directly owned by the deformer. This would cause hidden rename issues when trying to
		// rename a variable/graph/resource to the same name.
		for (UObject* ResourceDescription: Resources->Descriptions)
		{
			if (ResourceDescription->GetOuter() != Resources)
			{
				Optimus::RenameObject(ResourceDescription, nullptr, Resources);
			}
		}
		for (UObject* VariableDescription: Variables->Descriptions)
		{
			if (VariableDescription->GetOuter() != Variables)
			{
				Optimus::RenameObject(VariableDescription, nullptr, Variables);
			}
		}
	}
	if (ObjectVersion < FOptimusObjectVersion::ComponentProviderSupport)
	{
		if (ensure(Bindings->Bindings.IsEmpty()))
		{
			// Create a default skeletal mesh binding. This is always created for skeletal mesh deformers.
			UOptimusComponentSource* ComponentSource = UOptimusSkeletalMeshComponentSource::StaticClass()->GetDefaultObject<UOptimusComponentSource>();
			UOptimusComponentSourceBinding* Binding = CreateComponentBindingDirect(ComponentSource, ComponentSource->GetBindingName());
			Binding->bIsPrimaryBinding = true;
			Bindings->Bindings.Add(Binding);

			(void)MarkPackageDirty();
		}

		// Fix up any data providers to ensure they have a binding.
		PostLoadFixupMissingComponentBindingsCompat();
	}
	if (ObjectVersion < FOptimusObjectVersion::SetPrimaryBindingName)
	{
		FName PrimaryBindingName = UOptimusComponentSourceBinding::GetPrimaryBindingName();
		for (UOptimusComponentSourceBinding* Binding : Bindings->Bindings)
		{
			if (Binding->bIsPrimaryBinding)
			{
				Optimus::RenameObject(Binding, *PrimaryBindingName.ToString(), nullptr);
				Binding->BindingName = PrimaryBindingName;
			}
		}
		TArray<UOptimusNode*> AllComponentSourceNode = GetAllNodesOfClass(UOptimusNode_ComponentSource::StaticClass());
		for (UOptimusNode* Node : AllComponentSourceNode)
		{
			if (UOptimusNode_ComponentSource* ComponentSourceNode = Cast<UOptimusNode_ComponentSource>(Node))
			{
				if (ComponentSourceNode->GetComponentBinding()->IsPrimaryBinding())
				{
					ComponentSourceNode->SetDisplayName(FText::FromName(PrimaryBindingName));
				}
			}
		}
	}
	// Fix any resource data domains if the component binding is valid but the domain is not. This will mostly cut links
	// to kernels with mismatched domain info.
	if (ObjectVersion < FOptimusObjectVersion::DataDomainExpansion)
	{
		PostLoadFixupMismatchedResourceDataDomains();
	}

	if (ObjectVersion < FOptimusObjectVersion::KernelDataInterface)
	{
		PostLoadRemoveDeprecatedExecutionNodes();
	}

	if (ObjectVersion < FOptimusObjectVersion::PropertyBagValueContainer)
	{
		PostLoadRemoveDeprecatedValueContainerGeneratorClass();
	}

	if (ObjectVersion < FOptimusObjectVersion::PropertyPinSupport)
	{
		PostLoadMoveValueFromGraphDataInterfaceToDeformerValueMap();
	}
	
	// If the graph was saved at any previous version, and was clean, recompile it to the latest version.
	if (Status != EOptimusDeformerStatus::HasErrors &&
		ObjectVersion < FOptimusObjectVersion::LatestVersion)
	{
		bool bDelayCompile = false;
		if (ObjectVersion < FOptimusObjectVersion::FunctionGraphUseGuid)
		{
			TArray<UOptimusNode*> FunctionNodes = GetAllNodesOfClass(UOptimusNode_FunctionReference::StaticClass());
			if (FunctionNodes.Num() > 0)
			{
				bDelayCompile = true;
				// Before this version, external function graphs were only soft references 
				// so in some case it is not possible make sure they are fully loaded during PostLoad.
				// Delaying the compilation here such that it is performed after all dependencies are fully loaded (hopefully)
				// It is best to resave the asset to avoid going down this path
				ExecuteOnGameThread(UE_SOURCE_LOCATION,
					[WeakThis = TWeakObjectPtr(this)]()
						{
							if (WeakThis.IsValid())
							{
								WeakThis->Compile();
							}
						});

				UE_LOGF(LogOptimusCore, Warning, "Deformer Graph should be recompiled and resaved to avoid issues in the packaged build, Asset: %ls",  *GetPackage()->GetName());
			}
		}

		if (!bDelayCompile)
		{
			Compile();
		}
	}
#endif // WITH_EDITOR
}


void UOptimusDeformer::PostLoadFixupMissingComponentBindingsCompat()
{
	for (UOptimusNodeGraph* Graph: GetGraphs())
	{
		if (!Optimus::IsExecutionGraphType(Graph->GetGraphType()))
		{
			continue;
		}

		FVector2D::FReal MinimumPosX = std::numeric_limits<FVector2D::FReal>::max(), AccumulatedPosY = 0;

		TMap<UOptimusNode_DataInterface*, UOptimusComponentSourceBinding*> InterfaceBindingMap;  
		
		for (UOptimusNode* Node: Graph->GetAllNodes())
		{
			MinimumPosX = FMath::Min(MinimumPosX, Node->GetGraphPosition().X);
			AccumulatedPosY += Node->GetGraphPosition().Y;

			if (UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(Node))
			{
				// Do we have a compatible binding?
				if (ensure(DataInterfaceNode->DataInterfaceClass))
				{
					const UOptimusComputeDataInterface* DataInterface = DataInterfaceNode->DataInterfaceClass->GetDefaultObject<UOptimusComputeDataInterface>();
					UOptimusComponentSourceBinding* Binding = FindCompatibleBindingWithInterface(DataInterface);
					if (!Binding)
					{
						if (const UOptimusComponentSource* ComponentSource = UOptimusComponentSource::GetSourceFromDataInterface(DataInterface))
						{
							Binding = AddComponentBinding(ComponentSource);
						}
					}

					if (Binding)
					{
						InterfaceBindingMap.Add(DataInterfaceNode, Binding);

						// Make sure the component input pin has been created.
						DataInterfaceNode->ConditionalPostLoad();
					}
				}
			}
		}

		if (!InterfaceBindingMap.IsEmpty())
		{
			static const FVector2D NodeSize(160.0, 40.0); 
			static const FVector2D NodeMargins(40.0, 20.0); 
			
			// Create component source nodes with the requested binding and connect them to the data interface nodes. 
			TMap<UOptimusComponentSourceBinding*, UOptimusNode_ComponentSource*> BindingToNodeMap;
			for (const TPair<UOptimusNode_DataInterface*, UOptimusComponentSourceBinding*>& Item: InterfaceBindingMap)
			{
				BindingToNodeMap.Add(Item.Value, nullptr);
			}

			MinimumPosX -= NodeSize.X + NodeMargins.X;
			AccumulatedPosY /= Graph->GetAllNodes().Num();
			AccumulatedPosY -= NodeSize.Y * 0.5 + (BindingToNodeMap.Num() - 1) * (NodeSize.Y + NodeMargins.Y);
			
			for (const TPair<UOptimusNode_DataInterface*, UOptimusComponentSourceBinding*>& Item: InterfaceBindingMap)
			{
				UOptimusNode_ComponentSource*& ComponentSourceNodePtr = BindingToNodeMap.FindChecked(Item.Value);
				
				if (ComponentSourceNodePtr == nullptr)
				{
					ComponentSourceNodePtr = Cast<UOptimusNode_ComponentSource>(Graph->AddComponentBindingGetNode(Item.Value, FVector2D(MinimumPosX, AccumulatedPosY)));
					AccumulatedPosY += NodeSize.Y + NodeMargins.Y;
				}

				if (!ensure(ComponentSourceNodePtr))
				{
					continue;
				}
				
				UOptimusNodePin* ComponentSourcePin = ComponentSourceNodePtr->GetComponentPin();

				Graph->AddLink(ComponentSourcePin, Item.Key->GetComponentPin());
			}
		}
	}
	(void)MarkPackageDirty();
}


void UOptimusDeformer::PostLoadFixupMismatchedResourceDataDomains()
{
#if 0
	TArray<UOptimusNode*> AllResourceNodes = GetAllNodesOfClass(UOptimusNode_ResourceAccessorBase::StaticClass());
	
	for (UOptimusResourceDescription* ResourceDescription: Resources->Descriptions)
	{
		if (ResourceDescription->IsValidComponentBinding() && ResourceDescription->DataDomain.IsSingleton())
		{
			
			// SetResourceDataDomain(ResourceDescription, ResourceDescription->GetDataDomainFromComponentBinding(), true);
		}
	}
#endif
}

void UOptimusDeformer::PostLoadRemoveDeprecatedExecutionNodes()
{
	for (UOptimusNodeGraph* Graph: GetGraphs())
	{
		// At the time of deprecation, subgraph is not supported
		if (!ensure(Optimus::IsExecutionGraphType(Graph->GetGraphType())))
		{
			continue;
		}
	
		TArray<UOptimusNode*> DeprecatedExecutionDataInterfaceNodes;

		for (UOptimusNode* Node: Graph->GetAllNodes())
		{
			// PostLoad fixup for Kernel Nodes
			if (UOptimusNode_CustomComputeKernel* KernelNode = Cast<UOptimusNode_CustomComputeKernel>(Node))
			{
				// Find the primary ComponentSourceNode for each kernel node
				const UOptimusNodePin* PrimaryGroupPin = KernelNode->GetPrimaryGroupPin();

				UOptimusNode_ComponentSource* ComponentSourceNode = nullptr;
				FOptimusDataTypeHandle IntVector3Type = FOptimusDataTypeRegistry::Get().FindType(TBaseStructure<FIntVector3>::Get());
				
				for (const UOptimusNodePin* Pin : PrimaryGroupPin->GetSubPins())
				{
					if (Pin->GetDirection() == EOptimusNodePinDirection::Input && Pin->GetDataType() == IntVector3Type)
					{
						TArray<FOptimusRoutedNodePin> ConnectedPins = Pin->GetConnectedPinsWithRouting();
						if (ConnectedPins.Num() != 1)
						{
							// Skip if invalid/no connection
							continue;
						}

						const UOptimusNode_DataInterface* DataInterfaceNode =
							Cast<UOptimusNode_DataInterface>(ConnectedPins[0].NodePin->GetOwningNode());
						const IOptimusDeprecatedExecutionDataInterface* ExecDataInterface =
							Cast<IOptimusDeprecatedExecutionDataInterface>(DataInterfaceNode->GetDataInterface(GetTransientPackage()));

						if (!ExecDataInterface)
						{
							// Skip if not connected to exec data interface
							continue;
						}

						UOptimusNodePin* ComponentPin = DataInterfaceNode->GetComponentPin();

						TArray<FOptimusRoutedNodePin> ConnectedComponentPins = ComponentPin->GetConnectedPinsWithRouting();

						if (ConnectedComponentPins.Num() != 1)
						{
							// Skip if the exec data interface does not have a component source
							continue;
						}

						ComponentSourceNode = Cast<UOptimusNode_ComponentSource>(ConnectedComponentPins[0].NodePin->GetOwningNode());
						
						if (ComponentSourceNode)
						{
							// Found a valid component source node, ready to link
							break;
						}
					}
				}

				// Now that we have extract information from every pin, the deprecated ones have no more use and can be removed
				KernelNode->PostLoadRemoveDeprecatedNumThreadsPin();

				// After pin removal, we may have no input data pin to infer component source from,
				// in which case we have to force a direct link between component source node and the kernel primary group pin
				if (PrimaryGroupPin->GetSubPins().Num() > 0)
				{
					continue;
				}

				if (ComponentSourceNode)
				{
					Graph->AddLink(ComponentSourceNode->GetComponentPin(), KernelNode->GetPrimaryGroupPin_Internal());	
				}
			}

			// PostLoad remove execution data interface nodes
			if (const UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(Node))
			{
				if (Cast<IOptimusDeprecatedExecutionDataInterface>(DataInterfaceNode->GetDataInterface(GetTransientPackage())))
				{
					DeprecatedExecutionDataInterfaceNodes.Add(Node);
				}
			}
		}
		
		Graph->RemoveNodes(DeprecatedExecutionDataInterfaceNodes);
	}
	
	(void)MarkPackageDirty();
}

void UOptimusDeformer::PostLoadRemoveDeprecatedValueContainerGeneratorClass()
{
	// Remove deprecated uclass based value container generator class
	TArray<UObject*> ObjectsInPackage;
	GetObjectsWithOuter(GetPackage(), ObjectsInPackage, EGetObjectsFlags::None);

	for (UObject* Object : ObjectsInPackage)
	{
		if (UOptimusValueContainerGeneratorClass* GeneratorClass = Cast<UOptimusValueContainerGeneratorClass>(Object))
		{
			Optimus::RemoveObject(GeneratorClass);
			Optimus::RemoveObject(GeneratorClass->GetDefaultObject());
		}
	}	
}


void UOptimusDeformer::PostLoadMoveValueFromGraphDataInterfaceToDeformerValueMap()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	int ConstantUniqueSuffix = 0;
	for (FOptimusComputeGraphInfo& Info : ComputeGraphs)
	{
		if (UOptimusGraphDataInterface* GraphDataInterface = Info.ComputeGraph->GetGraphDataInterfaceForPostLoadFixUp())
		{
			for (FOptimusGraphVariableDescription& OldDescription : GraphDataInterface->Variables)
			{
				if (OldDescription.ShaderValue_DEPRECATED.IsValid())
				{
					FOptimusValueDescription NewDescription;
					NewDescription.ValueUsage = EOptimusValueUsage::GPU;
					NewDescription.DataType = FOptimusDataTypeRegistry::Get().FindType(OldDescription.ValueType);
					NewDescription.ShaderValue = OldDescription.ShaderValue_DEPRECATED;
					// Make sure the Id is unique across all graphs
					// We could do better here if needed by finding the source constant node of this value, but settling for the simpler solution for now
					// The only thing that would break is scrubbing constant node value without recompiling
					FOptimusValueIdentifier ValueId = {EOptimusValueType::Constant, *(OldDescription.Name + TEXT("_PostLoadFixUp_") + FString::FromInt(ConstantUniqueSuffix))};
					ConstantUniqueSuffix++;
					ValueMap.Add(ValueId, MoveTemp(NewDescription));
					
					OldDescription.ValueId = ValueId;
				}
				else
				{
					FName VariableName = NAME_None;
					// When source object was introduced, we also appended a unique index to the value name provided by each value provider
					// so instead of using the name directly, we need to do this extra step
					if (OldDescription.SourceObject_DEPRECATED.IsNull())
					{
						VariableName = *OldDescription.Name;
					}
					else
					{
						VariableName = *(Optimus::ExtractSourceValueName(OldDescription.Name));
					}

					FOptimusValueIdentifier ValueId = {EOptimusValueType::Variable, VariableName};
					if (!ValueMap.Contains(ValueId))
					{
						for (UOptimusVariableDescription* VariableDescription : GetVariables())
						{
							if (VariableDescription->VariableName == VariableName)
							{
								FOptimusValueDescription NewDescription;
								NewDescription.ValueUsage = EOptimusValueUsage::GPU;
								NewDescription.DataType = FOptimusDataTypeRegistry::Get().FindType(OldDescription.ValueType);
								NewDescription.ShaderValue = VariableDescription->DefaultValueStruct.GetShaderValue(VariableDescription->DataType);
								ValueMap.Add(ValueId, MoveTemp(NewDescription));	
								break;
							}
						}
					}
					
					OldDescription.ValueId = ValueId;
					
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


UOptimusComponentSourceBinding* UOptimusDeformer::FindCompatibleBindingWithInterface(
	const UOptimusComputeDataInterface* InDataInterface) const
{
	if (!ensure(InDataInterface))
	{
		return nullptr;
	}
	
	for (UOptimusComponentSourceBinding* Binding: GetComponentBindings())
	{
		if (!ensure(Binding->GetComponentSource()))
		{
			continue;
		}

		// If the binding comp class is the same or a sub-class of the interface comp class, then
		// they're compatible (e.g. if the interface requires only USceneComponent but the binding class
		// is a USkinnedMeshComponent, then the USkinnedMeshComponent will suffice).
		const UClass* BindingComponentClass = Binding->GetComponentSource()->GetComponentClass();
		const UClass* InterfaceComponentClass = InDataInterface->GetRequiredComponentClass();
		if (BindingComponentClass->IsChildOf(InterfaceComponentClass))
		{
			return Binding;
		}
	}

	return nullptr;
}

void UOptimusDeformer::BeginDestroy()
{
	Super::BeginDestroy();

	FOptimusDataTypeRegistry::Get().GetOnDataTypeChanged().RemoveAll(this);
}

void UOptimusDeformer::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// Whenever the asset is renamed/moved, generated classes parented to the old package
	// are not moved to the new package automatically (see FAssetRenameManager), so we
	// have to manually perform the move/rename, to avoid invalid reference to the old package
	TArray<UClass*> ClassObjects = Optimus::GetClassObjectsInPackage(OldOuter->GetPackage());

	for (UClass* ClassObject : ClassObjects)
	{
		Optimus::RenameObject(ClassObject, nullptr, GetPackage());
	}
}

void UOptimusDeformer::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UOptimusDeformer::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	UClass* BindingClass = nullptr;

	if (Bindings != nullptr)
	{
		for (UOptimusComponentSourceBinding const* Binding : Bindings->Bindings)
		{
			if (Binding != nullptr && Binding->IsPrimaryBinding())
			{
				if (Binding->GetComponentSource() != nullptr)
				{
					BindingClass = Binding->GetComponentSource()->GetComponentClass();
					break;
				}
			}
		}
	}

	if (BindingClass != nullptr)
	{
		FSoftClassPath ClassPath(BindingClass);
		Context.AddTag(FAssetRegistryTag(TEXT("PrimaryBindingClass"), *ClassPath.ToString(), FAssetRegistryTag::TT_Hidden));
	}

	// Expose all the public functions, we used to use FOptimusFunctionNodeGraphHeader, but has since switched to FOptimusFunctionNodeGraphHeaderWithGuid
	{
		FOptimusFunctionNodeGraphHeaderWithGuidArray PublicFunctionHeadersArray;
		for (const UOptimusFunctionNodeGraph* FunctionNodeGraph: GetFunctionGraphs(UOptimusFunctionNodeGraph::AccessSpecifierPublicName))
		{
			PublicFunctionHeadersArray.Headers.Add(FunctionNodeGraph->GetHeaderWithGuid());
		}
	
		FString PublicFunctionString;
	
		FOptimusFunctionNodeGraphHeaderWithGuidArray::StaticStruct()->ExportText(PublicFunctionString, &PublicFunctionHeadersArray, nullptr, nullptr, PPF_None, nullptr);

		Context.AddTag(FAssetRegistryTag(PublicFunctionsWithGuidAssetTagName, *PublicFunctionString, FAssetRegistryTag::TT_Hidden));
	}

	{
		const FName& TagName = SkeletalMeshHalfEdgeBufferAccessor::GetHalfEdgeRequirementAssetTagName(); 
		Context.AddTag(FAssetRegistryTag(TagName, IsSkeletalMeshHalfEdgeBufferRequired() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Hidden));
	}
}

#if WITH_EDITOR

void UOptimusDeformer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	MarkModified();
}
#endif


UMeshDeformerInstanceSettings* UOptimusDeformer::CreateSettingsInstance(UMeshComponent* InMeshComponent)
{
	FName SettingsName(GetName() + TEXT("_Settings"));
	const EObjectFlags CreateObjectFlags = InMeshComponent->HasAnyFlags(RF_ArchetypeObject) ? RF_Public : RF_NoFlags; // Make public when stored in a BP.
	SettingsName = MakeUniqueObjectName(InMeshComponent, UOptimusDeformerInstanceSettings::StaticClass(), SettingsName);
	UOptimusDeformerInstanceSettings *Settings = NewObject<UOptimusDeformerInstanceSettings>(InMeshComponent, SettingsName, CreateObjectFlags);
	Settings->InitializeSettings(this, InMeshComponent);
	return Settings;
}


UMeshDeformerInstance* UOptimusDeformer::CreateInstance(
	UMeshComponent* InMeshComponent,
	UMeshDeformerInstanceSettings* InSettings
	)
{
	if (InMeshComponent == nullptr)
	{
		return nullptr;
	}

	// Return nullptr if deformers are disabled. Clients can then fallback to some other behaviour.
	EShaderPlatform Platform = InMeshComponent->GetScene() != nullptr ? InMeshComponent->GetScene()->GetShaderPlatform() : GMaxRHIShaderPlatform;
	if (!Optimus::IsEnabled() || !Optimus::IsSupported(Platform))
	{
		return nullptr;
	}

	// Return nullptr if running dedicated server
	const UWorld* World = InMeshComponent->GetWorld();
	if (World && World->IsNetMode(NM_DedicatedServer))
	{
		return nullptr;
	}

	UOptimusDeformerDynamicInstanceManager* InstanceManager = NewObject<UOptimusDeformerDynamicInstanceManager>(InMeshComponent);
	
	InstanceManager->DefaultInstance = CreateOptimusInstance(InMeshComponent, InSettings);
	
	return InstanceManager;
}

UOptimusDeformerInstance* UOptimusDeformer::CreateOptimusInstance(UMeshComponent* InMeshComponent, UMeshDeformerInstanceSettings* InSettings)
{
	const FName InstanceName = Optimus::GetUniqueNameForScope(InMeshComponent ,*(GetName() + TEXT("_Instance")));
	
	UOptimusDeformerInstance* Instance = NewObject<UOptimusDeformerInstance>(InMeshComponent, InstanceName);
	Instance->SetMeshComponent(InMeshComponent);
	Instance->SetInstanceSettings(Cast<UOptimusDeformerInstanceSettings>(InSettings));
	Instance->SetupFromDeformer(this);

	CompileEndDelegate.RemoveAll(Instance);
	// Make sure all the instances know when we finish compiling so they can update their local state to match.
	CompileEndDelegate.AddUObject(Instance, &UOptimusDeformerInstance::SetupFromDeformer);
	ConstantValueUpdateDelegate.AddUObject(Instance, &UOptimusDeformerInstance::SetConstantValueDirect);
	SetAllInstancesCanbeActiveDelegate.AddUObject(Instance, &UOptimusDeformerInstance::SetCanBeActive);

	return Instance;
}

bool UOptimusDeformer::IsSkeletalMeshHalfEdgeBufferRequired() const
{
	for (const FOptimusComputeGraphInfo& Info : ComputeGraphs)
	{
		if (Info.ComputeGraph->HasHalfEdgeDataInterface())
		{
			return true;
		}
	}

	return false;
}


void UOptimusDeformer::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty)
{
	if (bMarkAsDirty)
	{
		Modify();
	}
	Mesh = PreviewMesh;
	
	// FIXME: Notify upstream so the viewport can react.
}


USkeletalMesh* UOptimusDeformer::GetPreviewMesh() const
{
	return Mesh;
}


IOptimusNodeGraphCollectionOwner* UOptimusDeformer::ResolveCollectionPath(const FString& InPath)
{
	if (InPath.IsEmpty())
	{
		return this;
	}

	return Cast<IOptimusNodeGraphCollectionOwner>(ResolveGraphPath(InPath));
}


UOptimusNodeGraph* UOptimusDeformer::ResolveGraphPath(const FString& InGraphPath)
{
	FStringView PathRemainder;

	UOptimusNodeGraph* Graph = ResolveGraphPath(InGraphPath, PathRemainder);

	// The graph is only valid if the path was fully consumed.
	return PathRemainder.IsEmpty() ? Graph : nullptr;
}


UOptimusNode* UOptimusDeformer::ResolveNodePath(const FString& InNodePath)
{
	FStringView PathRemainder;

	UOptimusNode* Node = ResolveNodePath(InNodePath, PathRemainder);

	// The graph is only valid if the path was fully consumed.
	return PathRemainder.IsEmpty() ? Node : nullptr;
}


UOptimusNodePin* UOptimusDeformer::ResolvePinPath(const FString& InPinPath)
{
	FStringView PinPath;

	UOptimusNode* Node = ResolveNodePath(InPinPath, PinPath);

	return Node ? Node->FindPin(PinPath) : nullptr;
}


UOptimusNodeGraph* UOptimusDeformer::FindGraphByName(FName InGraphName) const
{
	for (UOptimusNodeGraph* Graph : GetGraphs())
	{
		if (Graph->GetFName() == InGraphName)
		{
			return Graph;
		}
	}

	return nullptr;
}

UOptimusNodeGraph* UOptimusDeformer::CreateGraphDirect(
	EOptimusNodeGraphType InType, 
	FName InName, 
	TOptional<int32> InInsertBefore
	)
{
	// Update graphs is a singleton and is created by default. Transient graphs are only used
	// when duplicating nodes and should never exist as a part of a collection. 
	if (InType == EOptimusNodeGraphType::Update ||
		InType == EOptimusNodeGraphType::Transient)
	{
		return nullptr;
	}

	UClass* GraphClass = UOptimusNodeGraph::StaticClass();
	
	if (InType == EOptimusNodeGraphType::Setup)
	{
		// Do we already have a setup graph?
		if (Graphs.Num() > 1 && Graphs[0]->GetGraphType() == EOptimusNodeGraphType::Setup)
		{
			return nullptr;
		}

		// The name of the setup graph is fixed.
		InName = UOptimusNodeGraph::SetupGraphName;
	}
	else if (InType == EOptimusNodeGraphType::ExternalTrigger)
	{
		if (!UOptimusNodeGraph::IsValidUserGraphName(InName.ToString()))
		{
			return nullptr;
		}

		// If there's already an object with this name, then attempt to make the name unique.
		InName = Optimus::GetUniqueNameForScope(this, InName);
	}
	else if (InType == EOptimusNodeGraphType::Function)
	{
		if (!UOptimusNodeGraph::IsValidUserGraphName(InName.ToString()))
		{
			return nullptr;
		}
		
		GraphClass = UOptimusFunctionNodeGraph::StaticClass();

		// If there's already an object with this name, then attempt to make the name unique.
		InName = Optimus::GetUniqueNameForScope(this, InName);
	}

	UOptimusNodeGraph* Graph = NewObject<UOptimusNodeGraph>(this, GraphClass, InName, RF_Transactional);

	Graph->SetGraphType(InType);

	if (InInsertBefore.IsSet())
	{
		if (!AddGraphDirect(Graph, InInsertBefore.GetValue()))
		{
			Optimus::RemoveObject(Graph);
			return nullptr;
		}
	}
	
	return Graph;
}


bool UOptimusDeformer::AddGraphDirect(
	UOptimusNodeGraph* InGraph,
	int32 InInsertBefore
	)
{
	if (InGraph == nullptr || InGraph->GetOuter() != this)
	{
		return false;
	}

	const bool bHaveSetupGraph = (Graphs.Num() > 1 && Graphs[0]->GetGraphType() == EOptimusNodeGraphType::Setup);

	// If INDEX_NONE, insert at the end.
	if (InInsertBefore == INDEX_NONE)
	{
		InInsertBefore = Graphs.Num();
	}
		
	switch (InGraph->GetGraphType())
	{
	case EOptimusNodeGraphType::Update:
		// We cannot replace the update graph.
		return false;
		
	case EOptimusNodeGraphType::Setup:
		// Do we already have a setup graph?
		if (bHaveSetupGraph)
		{
			return false;
		}
		// The setup graph is always first, if present.
		InInsertBefore = 0;
		break;
		
	case EOptimusNodeGraphType::ExternalTrigger:
		// Trigger graphs are always sandwiched between setup and update.
		InInsertBefore = FMath::Clamp(InInsertBefore, bHaveSetupGraph ? 1 : 0, GetUpdateGraphIndex());
		break;

	case EOptimusNodeGraphType::Function:
		// Function graphs always go last.
		InInsertBefore = Graphs.Num();
		break;

	case EOptimusNodeGraphType::SubGraph:
		// We cannot add subgraphs to the root.
		return false;

	case EOptimusNodeGraphType::Transient:
		checkNoEntry();
		return false;
	}
	
	Graphs.Insert(InGraph, InInsertBefore);

	Notify(EOptimusGlobalNotifyType::GraphAdded, InGraph);

	return true;
}


bool UOptimusDeformer::RemoveGraphDirect(
	UOptimusNodeGraph* InGraph,
	bool bInDeleteGraph
	)
{
	// Not ours?
	const int32 GraphIndex = Graphs.IndexOfByKey(InGraph);
	if (GraphIndex == INDEX_NONE)
	{
		return false;
	}

	if (InGraph->GetGraphType() == EOptimusNodeGraphType::Update)
	{
		return false;
	}

	Graphs.RemoveAt(GraphIndex);

	Notify(EOptimusGlobalNotifyType::GraphRemoved, InGraph);

	if (bInDeleteGraph)
	{
		// Un-parent this graph to a temporary storage and mark it for kill.
		Optimus::RemoveObject(InGraph);
	}

	return true;
}



bool UOptimusDeformer::MoveGraphDirect(
	UOptimusNodeGraph* InGraph, 
	int32 InInsertBefore
	)
{
	const int32 GraphOldIndex = Graphs.IndexOfByKey(InGraph);
	if (GraphOldIndex == INDEX_NONE)
	{
		return false;
	}

	if (InGraph->GetGraphType() != EOptimusNodeGraphType::ExternalTrigger)
	{
		return false;
	}

	// Less than num graphs, because the index is based on the node being moved not being
	// in the list.
	if (InInsertBefore == INDEX_NONE)
	{
		InInsertBefore = GetUpdateGraphIndex();
	}
	else
	{
		const bool bHaveSetupGraph = (Graphs.Num() > 1 && Graphs[0]->GetGraphType() == EOptimusNodeGraphType::Setup);
		InInsertBefore = FMath::Clamp(InInsertBefore, bHaveSetupGraph ? 1 : 0, GetUpdateGraphIndex());
	}

	if (GraphOldIndex == InInsertBefore)
	{
		return true;
	}

	Graphs.RemoveAt(GraphOldIndex);
	Graphs.Insert(InGraph, InInsertBefore);

	Notify(EOptimusGlobalNotifyType::GraphIndexChanged, InGraph);

	return true;
}

bool UOptimusDeformer::RenameGraphDirect(UOptimusNodeGraph* InGraph, const FString& InNewName)
{
	if (Optimus::RenameObject(InGraph, *InNewName, nullptr))
	{
		Notify(EOptimusGlobalNotifyType::GraphRenamed, InGraph);
		return true;
	}

	return false;	
}


bool UOptimusDeformer::RenameGraph(UOptimusNodeGraph* InGraph, const FString& InNewName)
{
	// Not ours?
	int32 GraphIndex = Graphs.IndexOfByKey(InGraph);
	if (GraphIndex == INDEX_NONE)
	{
		return false;
	}

	// Setup and Update graphs cannot be renamed.
	if (InGraph->GetGraphType() == EOptimusNodeGraphType::Setup ||
		InGraph->GetGraphType() == EOptimusNodeGraphType::Update)
	{
		return false;
	}

	if (!UOptimusNodeGraph::IsValidUserGraphName(InNewName))
	{
		return false;
	}

	return GetActionStack()->RunAction<FOptimusNodeGraphAction_RenameGraph>(InGraph, FName(*InNewName));
}


int32 UOptimusDeformer::GetUpdateGraphIndex() const
{
	if (const UOptimusNodeGraph* UpdateGraph = GetUpdateGraph(); ensure(UpdateGraph != nullptr))
	{
		return UpdateGraph->GetGraphIndex();
	}

	return INDEX_NONE;
}


#undef LOCTEXT_NAMESPACE
