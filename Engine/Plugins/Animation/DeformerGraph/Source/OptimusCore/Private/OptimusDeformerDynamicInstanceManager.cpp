// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeformerDynamicInstanceManager.h"

#include "Components/SkeletalMeshComponent.h"
#include "ControlRig.h"
#include "IControlRigObjectBinding.h"
#include "OptimusDeformerInstance.h"
#include "ControlRig/RigUnit_Optimus.h"

#if WITH_EDITORONLY_DATA
#include "Animation/MeshDeformerGeometryReadback.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDeformerDynamicInstanceManager)

void UOptimusDeformerDynamicInstanceManager::AllocateResources()
{
	// Typically called during recreate render state
	
	DefaultInstance->AllocateResources();

	for (TPair<FGuid, TObjectPtr<UOptimusDeformerInstance>>& GuidToInstance : GuidToInstanceMap)
	{
		GuidToInstance.Value->AllocateResources();
	}
}

void UOptimusDeformerDynamicInstanceManager::ReleaseResources()
{
	// Typically called during recreate render state
	
#if WITH_EDITORONLY_DATA
	// Immediately release readback requests that won't be fulfilled, 
	GeometryReadbackRequests.Reset();
#endif // WITH_EDITORONLY_DATA	
	
	DefaultInstance->ReleaseResources();
	
	for (TPair<FGuid, TObjectPtr<UOptimusDeformerInstance>>& GuidToInstance : GuidToInstanceMap)
	{
		GuidToInstance.Value->ReleaseResources();
	}
}

void UOptimusDeformerDynamicInstanceManager::EnqueueWork(FEnqueueWorkDesc const& InDesc)
{
	// Runs during UWorld::SendAllEndOfFrameUpdates

	for (FGuid Guid : GuidsPendingInit)
	{
		if (TObjectPtr<UOptimusDeformerInstance>* InstancePtr = GuidToInstanceMap.Find(Guid))
		{
			
			(*InstancePtr)->AllocateResources();
		}
	}

	GuidsPendingInit.Reset();
	
	// Enqueue work



	TArray<UOptimusDeformerInstance*> SortedInstances;
	SortedInstances.Reserve(GuidToInstanceMap.Num());
	
	static const EOptimusDeformerExecutionPhase Phases[] = {
		EOptimusDeformerExecutionPhase::BeforeDefaultDeformer,
		EOptimusDeformerExecutionPhase::OverrideDefaultDeformer,
		EOptimusDeformerExecutionPhase::AfterDefaultDeformer,
	};

	for (EOptimusDeformerExecutionPhase Phase : Phases)
	{
		if (TMap<int32, TArray<FGuid>>* ExecutionGroupQueueMapPtr = ExecutionQueueMap.Find(Phase))
		{
			TArray<int32> SortedExecutionGroups;
			ExecutionGroupQueueMapPtr->GenerateKeyArray(SortedExecutionGroups);
			SortedExecutionGroups.Sort();

			if (Phase == EOptimusDeformerExecutionPhase::OverrideDefaultDeformer)
			{
				// The last instance in the override queue is the one getting used
				int32 LastOverrideExecutionGroup = SortedExecutionGroups.Last();
				FGuid LastOverrideInstanceGuid = (*ExecutionGroupQueueMapPtr)[LastOverrideExecutionGroup].Last();

				if (TObjectPtr<UOptimusDeformerInstance>* InstancePtr = GuidToInstanceMap.Find(LastOverrideInstanceGuid))
				{
					SortedInstances.Add((*InstancePtr));
				}
			}
			else
			{
				for (int32 ExecutionGroup : SortedExecutionGroups)
				{
					for (FGuid Guid : (*ExecutionGroupQueueMapPtr)[ExecutionGroup])
					{
						if (TObjectPtr<UOptimusDeformerInstance>* InstancePtr = GuidToInstanceMap.Find(Guid))
						{
							SortedInstances.Add((*InstancePtr));
						}
					}
				}
			}
		}
		else if (Phase == EOptimusDeformerExecutionPhase::OverrideDefaultDeformer)
		{
			// Use the default instance if there is no override
			SortedInstances.Add(DefaultInstance);
		}
	}
	
	// Making sure instances in the queue are dispatched sequentially
	uint8 NumComputeGraphsPossiblyEnqueued = 0;
	
	for (int32 Index = 0; Index < SortedInstances.Num(); Index++)
	{
		UOptimusDeformerInstance* Instance = SortedInstances[Index];
		
		Instance->GraphSortPriorityOffset = NumComputeGraphsPossiblyEnqueued;
		NumComputeGraphsPossiblyEnqueued += Instance->ComputeGraphExecInfos.Num();

#if WITH_EDITORONLY_DATA
		// Readback the deformer geometry after the last deformer executes
		if (Index == SortedInstances.Num() - 1)
		{
			for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
			{
				Instance->RequestReadbackDeformerGeometry(MoveTemp(Request));
			}
		}
#endif // WITH_EDITORONLY_DATA
		
		Instance->EnqueueWork(InDesc);
	}
	

	ExecutionQueueMap.Reset();
#if WITH_EDITORONLY_DATA
	// Avoid unbounded accumulation of readback requests
	GeometryReadbackRequests.Reset();
#endif // WITH_EDITORONLY_DATA
}

EMeshDeformerOutputBuffer UOptimusDeformerDynamicInstanceManager::GetOutputBuffers() const
{
	// Since instances can be added dynamically, no way to know in advance if some of these are not written to, so just declare all of them
	EMeshDeformerOutputBuffer Result = EMeshDeformerOutputBuffer::SkinnedMeshPosition | EMeshDeformerOutputBuffer::SkinnedMeshTangents | EMeshDeformerOutputBuffer::SkinnedMeshVertexColor;
	
	return Result;
}

#if WITH_EDITORONLY_DATA
bool UOptimusDeformerDynamicInstanceManager::RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest)
{
	// These requests are forwarded to the last deformer instance that runs, see EnqueueWork
	GeometryReadbackRequests.Add(MoveTemp(InRequest));
	
	return true;
}
#endif // WITH_EDITORONLY_DATA

UMeshDeformerInstance* UOptimusDeformerDynamicInstanceManager::GetInstanceForSourceDeformer()
{
	return DefaultInstance;
}

void UOptimusDeformerDynamicInstanceManager::OnObjectBeginDestroy(IMeshDeformerProducer* InObject)
{
	if (TArray<FGuid>* Instances = ProducerToGuidsMap.Find(Cast<UObject>(InObject)))
	{
		TArray<FGuid> InstancesToRemove = *Instances;
		for (FGuid Guid : InstancesToRemove)
		{
			if (TObjectPtr<UOptimusDeformerInstance>* InstancePtr = GuidToInstanceMap.Find(Guid))
			{
				if (*InstancePtr)
				{
					(*InstancePtr)->ReleaseResources();
				}
			}

			GuidToInstanceMap.Remove(Guid);
		}
	}
	
	ProducerToGuidsMap.Remove(Cast<UObject>(InObject));
	
	InObject->OnBeginDestroy().RemoveAll(this);
	
}

void UOptimusDeformerDynamicInstanceManager::BeginDestroy()
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	ProducerToGuidsMap.GenerateKeyArray(Objects);

	// Release resources should have been called, so just unregister callbacks for good measure
	
	for (TWeakObjectPtr<UObject>& Object : Objects)
	{
		if (Object.IsValid())
		{
			if(IMeshDeformerProducer* ManagedObject = Cast<IMeshDeformerProducer>(Object))
			{
				ManagedObject->OnBeginDestroy().RemoveAll(this);
			}
		}
	}
	
	Super::BeginDestroy();
}

void UOptimusDeformerDynamicInstanceManager::AddProducerDeformer(IMeshDeformerProducer* InProducer, FGuid InInstanceGuid, UOptimusDeformer* InDeformer)
{
	check(IsInGameThread());
	
	if (ensure(!GuidToInstanceMap.Contains(InInstanceGuid)))
	{
		UOptimusDeformerInstance* DeformerInstance = InDeformer->CreateOptimusInstance(CastChecked<UMeshComponent>(GetOuter()), nullptr);
		GuidToInstanceMap.Add(InInstanceGuid, DeformerInstance);
		GuidsPendingInit.Add(InInstanceGuid);

		if (TArray<FGuid>* GuidArray = ProducerToGuidsMap.Find(Cast<UObject>(InProducer)))
		{
			GuidArray->Add(InInstanceGuid);
		}
		else
		{
			ProducerToGuidsMap.Add(Cast<UObject>(InProducer), {InInstanceGuid});
			
			// First time for this control rig, register some callbacks as well
			check(!InProducer->OnBeginDestroy().IsBoundToObject(this));
			
			// Assuming owning component of the rig cannot change
			UOptimusDeformerDynamicInstanceManager* DeformerInstanceManager = this;
			InProducer->OnBeginDestroy().AddUObject(this, &UOptimusDeformerDynamicInstanceManager::OnObjectBeginDestroy);

		}
	}
}

UOptimusDeformerInstance* UOptimusDeformerDynamicInstanceManager::GetDeformerInstance(FGuid InInstanceGuid)
{
	TObjectPtr<UOptimusDeformerInstance>* InstancePtr = GuidToInstanceMap.Find(InInstanceGuid);
	
	if (InstancePtr)
	{
		return *InstancePtr;
	}

	return nullptr;
}

void UOptimusDeformerDynamicInstanceManager::EnqueueProducerDeformer(FGuid InInstanceGuid, EOptimusDeformerExecutionPhase InExecutionPhase, int32 InExecutionGroup)
{
	// Typically called from anim thread, but there shouldn't be concurrent access to this queue. All rigs running on the current mesh should run sequentially.
	
	FScopeLock ScopeLock(&EnqueueCriticalSection);
	TArray<FGuid>& InstanceQueueRef = ExecutionQueueMap.FindOrAdd(InExecutionPhase).FindOrAdd(InExecutionGroup);
	
	// If we ever get duplicates, it means extra unnecessary instances were added via extra control rig evaluations triggered by user actions like moving a control.
	// So let's invalidate those instances.
	if (FGuid* BadInstance = InstanceQueueRef.FindByKey(InInstanceGuid))
	{
		*BadInstance = FGuid();
	}
	InstanceQueueRef.Add(InInstanceGuid);
}

