// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeformerInstance.h"

#include "Components/MeshComponent.h"
#include "ComputeFramework/ComputeFramework.h"
#include "ComputeWorkerInterface.h"
#include "IOptimusDeformerAssetPathAccessor.h"
#include "IOptimusDeformerGeometryReadbackProvider.h"
#include "IOptimusPersistentBufferProvider.h"
#include "IOptimusValueProvider.h"
#include "DataInterfaces/OptimusDataInterfaceGraph.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"
#include "OptimusComputeGraph.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusVariableDescription.h"
#include "RenderGraphBuilder.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDeformerInstance)


/** Container for a pooled buffer. */
struct FOptimusPersistentStructuredBuffer
{
	TRefCountPtr<FRDGPooledBuffer> PooledBuffer;
	int32 ElementStride = 0;
	int32 ElementCount = 0;
};


void FOptimusPersistentBufferPool::GetResourceBuffers(
	FRDGBuilder& GraphBuilder,
	FName InResourceName,
	int32 InLODIndex,
	int32 InElementStride,
	int32 InRawStride,
	TArray<int32> const& InElementCounts,
	TArray<FRDGBufferRef>& OutBuffers,
	bool& bOutJustAllocated)
{
	OutBuffers.Reset();
	bOutJustAllocated = false;

	TMap<int32, TArray<FOptimusPersistentStructuredBuffer>>& LODResources = ResourceBuffersMap.FindOrAdd(InResourceName);  
	TArray<FOptimusPersistentStructuredBuffer>* ResourceBuffersPtr = LODResources.Find(InLODIndex);
	if (ResourceBuffersPtr == nullptr)
	{
		// Create pooled buffers and store.
		TArray<FOptimusPersistentStructuredBuffer> ResourceBuffers;
		AllocateBuffers(GraphBuilder, InElementStride, InRawStride, InElementCounts, ResourceBuffers, OutBuffers);
		LODResources.Add(InLODIndex, MoveTemp(ResourceBuffers));
		bOutJustAllocated = true;
	}
	else
	{
		ValidateAndGetBuffers(GraphBuilder,InElementStride, InElementCounts, *ResourceBuffersPtr, OutBuffers);
	}
}

void FOptimusPersistentBufferPool::GetImplicitPersistentBuffers(
	FRDGBuilder& GraphBuilder,
	FName DataInterfaceName,
	int32 InLODIndex,
	int32 InElementStride, 
	int32 InRawStride,
	TArray<int32> const& InElementCounts,
	TArray<FRDGBuffer*>& OutBuffers,
	bool& bOutJustAllocated)
{
	OutBuffers.Reset();
	bOutJustAllocated = false;

	TMap<int32, TArray<FOptimusPersistentStructuredBuffer>>& LODResources = ImplicitBuffersMap.FindOrAdd(DataInterfaceName);  
	TArray<FOptimusPersistentStructuredBuffer>* ResourceBuffersPtr = LODResources.Find(InLODIndex);
	if (ResourceBuffersPtr == nullptr)
	{
		// Create pooled buffers and store.
		TArray<FOptimusPersistentStructuredBuffer> ResourceBuffers;
		AllocateBuffers(GraphBuilder, InElementStride, InRawStride, InElementCounts, ResourceBuffers, OutBuffers);
		LODResources.Add(InLODIndex, MoveTemp(ResourceBuffers));
		bOutJustAllocated = true;
	}
	else
	{
		ValidateAndGetBuffers(GraphBuilder,InElementStride, InElementCounts, *ResourceBuffersPtr, OutBuffers);
	}
}



void FOptimusPersistentBufferPool::AllocateBuffers(
	FRDGBuilder& GraphBuilder,
	int32 InElementStride,
	int32 InRawStride,
	TArray<int32> const& InElementCounts,
	TArray<FOptimusPersistentStructuredBuffer>& OutResourceBuffers,
	TArray<FRDGBuffer*>& OutBuffers
	)
{
	OutResourceBuffers.Reserve(InElementCounts.Num());

	// If we are using a raw type alias for the buffer then we need to adjust stride and count.
	check(InRawStride == 0 || InElementStride % InRawStride == 0);
	const int32 Stride = InRawStride ? InRawStride : InElementStride;
	const int32 ElementStrideMultiplier = InRawStride ? InElementStride / InRawStride : 1;

	for (int32 Index = 0; Index < InElementCounts.Num(); Index++)
	{
		FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(Stride, InElementCounts[Index] * ElementStrideMultiplier);
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("FOptimusPersistentBuffer"), ERDGBufferFlags::None);
		OutBuffers.Add(Buffer);

		FOptimusPersistentStructuredBuffer& PersistentBuffer = OutResourceBuffers.AddDefaulted_GetRef();
		PersistentBuffer.ElementStride = InElementStride;
		PersistentBuffer.ElementCount = InElementCounts[Index];
		PersistentBuffer.PooledBuffer = GraphBuilder.ConvertToExternalBuffer(Buffer);
	}
}

void FOptimusPersistentBufferPool::ValidateAndGetBuffers(
	FRDGBuilder& GraphBuilder,
	int32 InElementStride,
	TArray<int32> const& InElementCounts,
	const TArray<FOptimusPersistentStructuredBuffer>& InResourceBuffers,
	TArray<FRDGBuffer*>& OutBuffers
	) const
{
	// Verify that the buffers are correct based on the incoming information. 
	// If there's a mismatch, then something has gone wrong upstream.
	// Maybe either duplicated names, missing resource clearing on recompile, or something else.
	if (!ensure(InResourceBuffers.Num() == InElementCounts.Num()))
	{
		return;
	}

	for (int32 Index = 0; Index < InResourceBuffers.Num(); Index++)
	{
		const FOptimusPersistentStructuredBuffer& PersistentBuffer = InResourceBuffers[Index];
		if (!ensure(PersistentBuffer.PooledBuffer.IsValid()) ||
			!ensure(PersistentBuffer.ElementStride == InElementStride) ||
			!ensure(PersistentBuffer.ElementCount == InElementCounts[Index]))
		{
			OutBuffers.Reset();
			return;
		}	

		// Register buffer back into the graph and return it.
		FRDGBufferRef Buffer = GraphBuilder.RegisterExternalBuffer(PersistentBuffer.PooledBuffer);
		OutBuffers.Add(Buffer);
	}
}

void FOptimusPersistentBufferPool::ReleaseResources()
{
	check(IsInRenderingThread());
	ResourceBuffersMap.Reset();
	ImplicitBuffersMap.Reset();
}

FOptimusDeformerInstanceExecInfo::FOptimusDeformerInstanceExecInfo()
{
	GraphType = EOptimusNodeGraphType::Update;
}


bool FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(FString& InOutName)
{
	// Remove suffix for blueprint spawned components.
	return InOutName.RemoveFromEnd(TEXT("_GEN_VARIABLE"));
}

FName FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(FName InName)
{
	FString Name = InName.ToString();
	if (GetSanitizedComponentName(Name))
	{
		return FName(Name);
	}
	// No change.
	return InName;
}

FName FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(UActorComponent const* InComponent)
{
	return InComponent ? GetSanitizedComponentName(InComponent->GetFName()) : FName();
}

TSoftObjectPtr<UActorComponent> FOptimusDeformerInstanceComponentBinding::GetActorComponent(AActor const* InActor, FString const& InName)
{
	if (InActor != nullptr && !InName.IsEmpty())
	{
		FString Path = InActor->GetPathName() + TEXT(".") + InName;
		return TSoftObjectPtr<UActorComponent>(FSoftObjectPath(Path));
	}
	return {};
}

TSoftObjectPtr<UActorComponent> FOptimusDeformerInstanceComponentBinding::GetActorComponent(AActor const* InActor) const
{
	return GetActorComponent(InActor, ComponentName.ToString());
}


void UOptimusDeformerInstanceSettings::InitializeSettings(UOptimusDeformer* InDeformer, UMeshComponent* InPrimaryComponent)
{
	Deformer = InDeformer;

	const TArray<UOptimusComponentSourceBinding*>& ComponentBindings = InDeformer->GetComponentBindings();  
	Bindings.SetNum(ComponentBindings.Num());
	for (int32 BindingIndex = 0; BindingIndex < Bindings.Num(); ++BindingIndex)
	{
		Bindings[BindingIndex].ProviderName = ComponentBindings[BindingIndex]->BindingName;
		if (BindingIndex == 0)
		{
			Bindings[BindingIndex].ComponentName = FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(InPrimaryComponent);
		}
	}
}

void UOptimusDeformerInstanceSettings::GetComponentBindings(
	UOptimusDeformer* InDeformer, 
	UMeshComponent* InPrimaryComponent, 
	TArray<UActorComponent*>& OutComponents) const
{
	AActor const* Actor = InPrimaryComponent != nullptr ? InPrimaryComponent->GetOwner() : nullptr;

	// Try to map onto the configured component bindings as much as possible.
	TMap<FName, UActorComponent*> ExistingBindings;

	for (FOptimusDeformerInstanceComponentBinding const& Binding : Bindings)
	{
		TSoftObjectPtr<UActorComponent> ActorComponent = Binding.GetActorComponent(Actor);
		UActorComponent* Component = ActorComponent.Get();
		ExistingBindings.Add(Binding.ProviderName, Component);
	}

	auto GetMatchingComponent = [](const AActor* Actor, const UOptimusComponentSourceBinding* Binding) -> UActorComponent*
	{
		if (Actor && Binding)
		{
			const TSubclassOf<UActorComponent>& BindingSourceClass = Binding->GetComponentSource()->GetComponentClass();
			if (!Binding->ComponentTags.IsEmpty())
			{
				TSet<UActorComponent*> TaggedComponents;
				for (FName Tag : Binding->ComponentTags)
				{
					TArray<UActorComponent*> Components = Actor->GetComponentsByTag(BindingSourceClass, Tag);
					for (UActorComponent* Component : Components)
					{
						TaggedComponents.Add(Component);
					}
				}

				if (!TaggedComponents.IsEmpty())
				{
					TArray<UActorComponent*> RankedTaggedComponents = TaggedComponents.Array();

					// Rank the components by the number of tags they match.
					RankedTaggedComponents.Sort(
						[Tags=TSet<FName>(Binding->ComponentTags)](const UActorComponent& InCompA, const UActorComponent& InCompB)
					{
						TSet<FName> TagsA(InCompA.ComponentTags);
						TSet<FName> TagsB(InCompB.ComponentTags);

						return Tags.Intersect(TagsA).Num() < Tags.Intersect(TagsB).Num();
					});

					if (!RankedTaggedComponents.IsEmpty())
					{
						return RankedTaggedComponents[0];
					}
				}
			}

			// Otherwise just use class matching on components owned by the actor.
			TArray<UActorComponent*> Components;
			Actor->GetComponents(BindingSourceClass, Components);
			if (!Components.IsEmpty())
			{
				return Components[0];
			}
		}
		return nullptr;
	};

	// Iterate component bindings and try to find a match.
	TSet<UActorComponent*> ComponentsUsed;
	const TArray<UOptimusComponentSourceBinding*>& ComponentBindings = InDeformer->GetComponentBindings();
	OutComponents.Reset(ComponentBindings.Num());
	for (const UOptimusComponentSourceBinding* Binding : ComponentBindings)
	{
		FName BindingName = Binding->BindingName;
		UActorComponent* BoundComponent = nullptr;

		// Primary binding always binds to the mesh component we're applied to.
		if (Binding->IsPrimaryBinding())
		{
			BoundComponent = InPrimaryComponent;
		}
		else
		{
			// Try an existing binding first and see if they still match by class. We ignore tags for this match
			// because we want to respect the will of the user, unless absolutely not possible (i.e. class mismatch).
			if (ExistingBindings.Contains(BindingName))
			{
				if (UActorComponent* Component = ExistingBindings[BindingName])
				{
					if (Component->IsA(Binding->GetComponentSource()->GetComponentClass()))
					{
						BoundComponent = Component;
					}
				}
			}
			
			// If not, try the programmatic resolver delegate.
			if (!BoundComponent && ComponentResolver.IsBound())
			{
				BoundComponent = ComponentResolver.Execute(BindingName);
			}
			
			// If not, try to find a component owned by this actor that matches the tag and class.
			if (!BoundComponent && Actor != nullptr)
			{
				BoundComponent = GetMatchingComponent(Actor, Binding);
			}
		}

		OutComponents.Add(BoundComponent);
		ComponentsUsed.Add(BoundComponent);
	}
}

UOptimusComponentSourceBinding const* UOptimusDeformerInstanceSettings::GetComponentBindingByName(FName InBindingName) const
{
	if (const UOptimusDeformer* DeformerResolved = Deformer.Get())
	{
		for (UOptimusComponentSourceBinding* Binding: DeformerResolved->GetComponentBindings())
		{
			if (Binding->BindingName == InBindingName)
			{
				return Binding;
			}
		}
	}
	return nullptr;
}


void UOptimusDeformerInstance::SetMeshComponent(UMeshComponent* InMeshComponent)
{ 
	check(InMeshComponent);
	MeshComponent = InMeshComponent;
	Scene = MeshComponent->GetScene();
}

void UOptimusDeformerInstance::SetInstanceSettings(UOptimusDeformerInstanceSettings* InInstanceSettings)
{
	InstanceSettings = InInstanceSettings; 
}


void UOptimusDeformerInstance::SetupFromDeformer(UOptimusDeformer* InDeformer)
{
	const EMeshDeformerOutputBuffer PreviousOutputBuffer = GetOutputBuffers();
	
	// If we're doing a recompile, ditch all stored render resources.
	ReleaseResources();

	// Update the component bindings before creating data providers. 
	// The bindings are in the same order as the component bindings in the deformer.
	TArray<UActorComponent*> BoundComponents;
	UOptimusDeformerInstanceSettings* InstanceSettingsPtr = InstanceSettings.Get(); 
	if (InstanceSettingsPtr == nullptr)
	{
		// If we don't have any settings, then create a temporary object to get bindings.
		InstanceSettingsPtr = NewObject<UOptimusDeformerInstanceSettings>();
		InstanceSettingsPtr->InitializeSettings(InDeformer, MeshComponent.Get());
	}
	InstanceSettingsPtr->GetComponentBindings(InDeformer, MeshComponent.Get(), BoundComponents);
	
	WeakBoundComponents.Reset();
	for (UActorComponent* Component : BoundComponents)
	{
		WeakBoundComponents.Add(Component);
	}

	WeakComponentSources.Reset();
	const TArray<UOptimusComponentSourceBinding*>& ComponentBindings = InDeformer->GetComponentBindings();
	for (const UOptimusComponentSourceBinding* ComponentBinding : ComponentBindings)
	{
		WeakComponentSources.Add(ComponentBinding->GetComponentSource());
	}
	
	// Create the persistent buffer pool
	BufferPool = MakeShared<FOptimusPersistentBufferPool>();
	
	// Create local storage for deformer graph constants/variables
	ValueMap = InDeformer->ValueMap;
	DataInterfacePropertyOverrideMap = InDeformer->DataInterfacePropertyOverrideMap;
	
	// (Re)Create and bind data providers.
	ComputeGraphExecInfos.Reset();
	GraphsToRunOnNextTick.Reset();
	
	for (int32 GraphIndex = 0; GraphIndex < InDeformer->ComputeGraphs.Num(); ++GraphIndex)
	{
		FOptimusComputeGraphInfo const& ComputeGraphInfo = InDeformer->ComputeGraphs[GraphIndex];
		FOptimusDeformerInstanceExecInfo& Info = ComputeGraphExecInfos.AddDefaulted_GetRef();
		Info.GraphName = ComputeGraphInfo.GraphName;
		Info.GraphType = ComputeGraphInfo.GraphType;
		Info.ComputeGraph = ComputeGraphInfo.ComputeGraph;

		// ComputeGraphs are sorted by the order we want to run them in. 
		// Using the graph index as our sort priority prevents kernels from the different (but related) graphs running simultaineously.
		Info.ComputeGraphInstance.SetGraphSortPriority(GraphIndex);

		if (BoundComponents.Num())
		{
			UMeshComponent* ActorComponent = MeshComponent.Get();
			AActor const* Actor = ActorComponent ? ActorComponent->GetOwner() : nullptr;
			for (int32 Index = 0; Index < BoundComponents.Num(); Index++)
			{
				Info.ComputeGraphInstance.CreateDataProviders(Info.ComputeGraph, Index, BoundComponents[Index]);
			}
		}
		else
		{
			// Fall back on everything being the given component.
			for (int32 Index = 0; Index < InDeformer->GetComponentBindings().Num(); Index++)
			{
				Info.ComputeGraphInstance.CreateDataProviders(Info.ComputeGraph, Index, MeshComponent.Get());
			}
		}

		for(TObjectPtr<UComputeDataProvider> DataProvider: Info.ComputeGraphInstance.GetDataProviders())
		{
			// Make the persistent buffer data provider aware of the buffer pool and current LOD index.
			if (IOptimusPersistentBufferProvider* PersistentBufferProvider = Cast<IOptimusPersistentBufferProvider>(DataProvider))
			{
				PersistentBufferProvider->SetBufferPool(BufferPool);
			}

			// Set this instance on the graph data provider so that it can query variables.
			if (IOptimusDeformerInstanceAccessor* InstanceAccessor = Cast<IOptimusDeformerInstanceAccessor>(DataProvider))
			{
				InstanceAccessor->SetDeformerInstance(this);
			}

			if (IOptimusDeformerAssetPathAccessor* AssetPathAccessor = Cast<IOptimusDeformerAssetPathAccessor>(DataProvider))
			{
				AssetPathAccessor->SetOptimusDeformerAssetPath(FTopLevelAssetPath(InDeformer));
			}
			
#if WITH_EDITORONLY_DATA
			if (Info.GraphType == EOptimusNodeGraphType::Update)
			{
				if (IOptimusDeformerGeometryReadbackProvider* GeometryReadbackProvider = Cast<IOptimusDeformerGeometryReadbackProvider>(DataProvider))
				{
					WeakGeometryReadbackProvider = GeometryReadbackProvider;
				}
			}
#endif
		}

		// Schedule the setup graph to run.
		if (Info.GraphType == EOptimusNodeGraphType::Setup)
		{
			GraphsToRunOnNextTick.Add(Info.GraphName);
		}
	}
	

	if (UMeshComponent* Ptr = MeshComponent.Get())
	{
		// In case we are writing to different buffers, notify the mesh component such that it can recreate render state and allocate necessary
		// passthrough vertex factories
		const EMeshDeformerOutputBuffer CurrentOutputBuffer = GetOutputBuffers();
		if (CurrentOutputBuffer != PreviousOutputBuffer)
		{
			Ptr->MarkRenderStateDirty();
		}
		Ptr->MarkRenderDynamicDataDirty();
	}
}


void UOptimusDeformerInstance::SetCanBeActive(bool bInCanBeActive)
{
	bCanBeActive = bInCanBeActive;
}

FOptimusValueContainerStruct UOptimusDeformerInstance::GetDataInterfacePropertyOverride(const UComputeDataInterface* DataInterface, FName PinName)
{
	if (const FOptimusDataInterfacePropertyOverrideInfo* OverrideInfoPtr = DataInterfacePropertyOverrideMap.Find(DataInterface))
	{
		if (const FOptimusValueIdentifier* Overrider = OverrideInfoPtr->PinNameToValueIdMap.Find(PinName))
		{
			return ValueMap[*Overrider].Value;
		}
	}

	// No override
	return {};
}

const FShaderValueContainer& UOptimusDeformerInstance::GetShaderValue(
	const FOptimusValueIdentifier& InValueId
	) const
{
	return ValueMap[InValueId].ShaderValue;
}

void UOptimusDeformerInstance::AllocateResources()
{
}

void UOptimusDeformerInstance::ReleaseResources()
{
	if (Scene || BufferPool)
	{
		ENQUEUE_RENDER_COMMAND(OptimusReleaseResources)([BufferPool=MoveTemp(BufferPool), Scene = Scene, OwnerPointer = this] (FRHICommandListImmediate& InCmdList)
		{
			if (Scene)
			{
				ComputeFramework::AbortWork(Scene, OwnerPointer);
			}

			if (BufferPool)
			{
				BufferPool->ReleaseResources();
			}
		});
	}
}

void UOptimusDeformerInstance::EnqueueWork(UMeshDeformerInstance::FEnqueueWorkDesc const& InDesc)
{
	// Convert execution group enum to ComputeTaskExecutionGroup name.
	FName ExecutionGroupName;
	switch (InDesc.ExecutionGroup)
	{
	case UMeshDeformerInstance::ExecutionGroup_Immediate:
		ExecutionGroupName = ComputeTaskExecutionGroup::Immediate;
		break;
	case UMeshDeformerInstance::ExecutionGroup_Default:
	case UMeshDeformerInstance::ExecutionGroup_EndOfFrameUpdate:
		ExecutionGroupName = ComputeTaskExecutionGroup::EndOfFrameUpdate;
		break;
	case UMeshDeformerInstance::ExecutionGroup_BeginInitViews:
		ExecutionGroupName = ComputeTaskExecutionGroup::BeginInitViews;
		break;
	default:
		ensure(0);
		return;
	}


	
	// Store the execution group name for data providers to use
	CurrentExecutionGroupName = ExecutionGroupName;

	// Enqueue work.
	bool bIsWorkEnqueued = false;
	if (bCanBeActive)
	{
		bool AreAllGraphsReady = true;
		for (FOptimusDeformerInstanceExecInfo& Info: ComputeGraphExecInfos)
		{
			if (Info.ComputeGraph->HasKernelResourcesPendingShaderCompilation())
			{
				AreAllGraphsReady = false;
				break;
			}
		}

		if (AreAllGraphsReady)
		{
			// Get the current queued graphs.
			TSet<FName> GraphsToRun;
			{
				UE::TScopeLock<FCriticalSection> Lock(GraphsToRunOnNextTickLock);
				Swap(GraphsToRunOnNextTick, GraphsToRun);
			}
			
			for (FOptimusDeformerInstanceExecInfo& Info: ComputeGraphExecInfos)
			{
				if (Info.GraphType == EOptimusNodeGraphType::Update || GraphsToRun.Contains(Info.GraphName))
				{
					bIsWorkEnqueued |= Info.ComputeGraphInstance.EnqueueWork(Info.ComputeGraph, InDesc.Scene, ExecutionGroupName, InDesc.OwnerName, InDesc.FallbackDelegate, this, GraphSortPriorityOffset);
				}
			}
		}
	}

	if (!bIsWorkEnqueued)
	{
		// If we failed to enqueue work then enqueue the fallback.
		// todo: This might need enqueuing for EndOfFrame instead of immediate execution?
		ENQUEUE_RENDER_COMMAND(ComputeFrameworkEnqueueFallback)([FallbackDelegate = InDesc.FallbackDelegate](FRHICommandListImmediate& RHICmdList)
		{ 
			FallbackDelegate.ExecuteIfBound();
		});
	}
	else if (InDesc.ExecutionGroup == UMeshDeformerInstance::ExecutionGroup_Immediate)
	{
		// If we succesfully enqueued to the Immediate group then flush all work on that group now.
		ComputeFramework::FlushWork(InDesc.Scene, ExecutionGroupName);
	}
}

EMeshDeformerOutputBuffer UOptimusDeformerInstance::GetOutputBuffers() const
{
	EMeshDeformerOutputBuffer Result = EMeshDeformerOutputBuffer::None;

	for (const FOptimusDeformerInstanceExecInfo& ExecInfo : ComputeGraphExecInfos)
	{
		if (const UOptimusComputeGraph* ComputeGraph = Cast<UOptimusComputeGraph>(ExecInfo.ComputeGraph))
		{
			Result |= ComputeGraph->GetOutputBuffers();
		}
	}

	return Result;
}

#if WITH_EDITORONLY_DATA
bool UOptimusDeformerInstance::RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest)
{
	if (WeakGeometryReadbackProvider.IsValid())
	{
		WeakGeometryReadbackProvider->RequestReadbackDeformerGeometry(MoveTemp(InRequest));
		return true;
	}

	return false;
}
#endif // WITH_EDITORONLY_DATA

namespace
{
	template <typename T>
	bool SetValue(TMap<FOptimusValueIdentifier, FOptimusValueDescription>& InValueMap, const FOptimusValueIdentifier& InValueId, FName InTypeName, T const& InValue)
	{
		FOptimusDataTypeHandle WantedType = FOptimusDataTypeRegistry::Get().FindType(InTypeName);

		if (FOptimusValueDescription* Description = InValueMap.Find(InValueId))
		{
			if (Description->DataType == WantedType)
			{
				TUniquePtr<FProperty> Property(WantedType->CreateProperty(nullptr, NAME_None));
				if (ensure(Property->GetSize() == sizeof(T)))
				{
					const uint8* ValueBytes = reinterpret_cast<const uint8*>(&InValue);
					TArrayView<const uint8> ValueView(ValueBytes, sizeof(T));
					if (Description->ValueUsage == EOptimusValueUsage::GPU)
					{
						WantedType->ConvertPropertyValueToShader(ValueView, Description->ShaderValue);
					}
					
					if (Description->ValueUsage == EOptimusValueUsage::CPU)
					{
						Description->Value.SetValue(Description->DataType, ValueView);
					}
				}
				
				return true;	
			}
		}

		return false;
	}
	
	template <typename T>
	bool SetVariableValue(TMap<FOptimusValueIdentifier, FOptimusValueDescription>& InValueMap, FName InVariableName, FName InTypeName, T const& InValue)
	{
		return SetValue(InValueMap, {EOptimusValueType::Variable, InVariableName}, InTypeName, InValue);
	}
}


bool UOptimusDeformerInstance::SetBoolVariable(FName InVariableName, bool InValue)
{
	return SetVariableValue(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(*FBoolProperty::StaticClass()), InValue);
}

bool UOptimusDeformerInstance::SetBoolArrayVariable(FName InVariableName, const TArray<bool>& InValue)
{
	return SetVariableValue(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(*FBoolProperty::StaticClass()), InValue);
}

bool UOptimusDeformerInstance::SetIntVariable(FName InVariableName, int32 InValue)
{
	return SetVariableValue<int32>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(*FIntProperty::StaticClass()), InValue);
}

bool UOptimusDeformerInstance::SetIntArrayVariable(FName InVariableName, const TArray<int32>& InValue)
{
	return SetVariableValue<TArray<int32>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(*FIntProperty::StaticClass()), InValue);
}

bool UOptimusDeformerInstance::SetInt2Variable(FName InVariableName, const FIntPoint& InValue)
{
	return SetVariableValue<FIntPoint>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FIntPoint>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetInt2ArrayVariable(FName InVariableName, const TArray<FIntPoint>& InValue)
{
	return SetVariableValue<TArray<FIntPoint>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FIntPoint>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetInt3Variable(FName InVariableName, const FIntVector& InValue)
{
	return SetVariableValue<FIntVector>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FIntVector>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetInt3ArrayVariable(FName InVariableName, const TArray<FIntVector>& InValue)
{
	return SetVariableValue<TArray<FIntVector>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FIntVector>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetInt4Variable(FName InVariableName, const FIntVector4& InValue)
{
	return SetVariableValue<FIntVector4>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FIntVector4>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetInt4ArrayVariable(FName InVariableName, const TArray<FIntVector4>& InValue)
{
	return SetVariableValue<TArray<FIntVector4>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FIntVector4>::Get()), InValue);
}


bool UOptimusDeformerInstance::SetFloatVariable(FName InVariableName, double InValue)
{
	if (SetVariableValue<double>(ValueMap, InVariableName, FDoubleProperty::StaticClass()->GetFName(), InValue))
	{
		return true;
	}
	
	// Fall back on float
	return SetVariableValue<float>(ValueMap, InVariableName, FFloatProperty::StaticClass()->GetFName(), static_cast<float>(InValue));
}

bool UOptimusDeformerInstance::SetFloatArrayVariable(FName InVariableName, const TArray<double>& InValue)
{
	return SetVariableValue<TArray<double>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(FDoubleProperty::StaticClass()->GetFName()), InValue);
}

bool UOptimusDeformerInstance::SetVector2Variable(FName InVariableName, const FVector2D& InValue)
{
	return SetVariableValue<FVector2D>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FVector2D>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetVector2ArrayVariable(FName InVariableName, const TArray<FVector2D>& InValue)
{
	return SetVariableValue<TArray<FVector2D>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FVector2D>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetVectorVariable(FName InVariableName, const FVector& InValue)
{
	return SetVariableValue<FVector>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FVector>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetVectorArrayVariable(FName InVariableName, const TArray<FVector>& InValue)
{
	return SetVariableValue<TArray<FVector>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FVector>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetVector4Variable(FName InVariableName, const FVector4& InValue)
{
	return SetVariableValue<FVector4>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FVector4>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetVector4ArrayVariable(FName InVariableName, const TArray<FVector4>& InValue)
{
	return SetVariableValue<TArray<FVector4>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FVector4>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetLinearColorVariable(FName InVariableName, const FLinearColor& InValue)
{
	return SetVariableValue<FLinearColor>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FLinearColor>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetLinearColorArrayVariable(FName InVariableName, const TArray<FLinearColor>& InValue)
{
	return SetVariableValue<TArray<FLinearColor>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FLinearColor>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetQuatVariable(FName InVariableName, const FQuat& InValue)
{
	return SetVariableValue<FQuat>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FQuat>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetQuatArrayVariable(FName InVariableName, const TArray<FQuat>& InValue)
{
	return SetVariableValue<TArray<FQuat>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FQuat>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetRotatorVariable(FName InVariableName, const FRotator& InValue)
{
	return SetVariableValue<FRotator>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FRotator>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetRotatorArrayVariable(FName InVariableName, const TArray<FRotator>& InValue)
{
	return SetVariableValue<TArray<FRotator>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FRotator>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetTransformVariable(FName InVariableName, const FTransform& InValue)
{
	return SetVariableValue<FTransform>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(TBaseStructure<FTransform>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetTransformArrayVariable(FName InVariableName, const TArray<FTransform>& InValue)
{
	return SetVariableValue<TArray<FTransform>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(TBaseStructure<FTransform>::Get()), InValue);
}

bool UOptimusDeformerInstance::SetNameVariable(FName InVariableName, const FName& InValue)
{
	return SetVariableValue<FName>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetTypeName(*FNameProperty::StaticClass()), InValue);
}

bool UOptimusDeformerInstance::SetNameArrayVariable(FName InVariableName, const TArray<FName>& InValue)
{
	return SetVariableValue<TArray<FName>>(ValueMap, InVariableName, FOptimusDataTypeRegistry::GetArrayTypeName(*FNameProperty::StaticClass()), InValue);
}


bool UOptimusDeformerInstance::EnqueueTriggerGraph(FName InTriggerGraphName)
{
	for(FOptimusDeformerInstanceExecInfo& ExecInfo: ComputeGraphExecInfos)
	{
		if (ExecInfo.GraphType == EOptimusNodeGraphType::ExternalTrigger && ExecInfo.GraphName == InTriggerGraphName)
		{
			UE::TScopeLock<FCriticalSection> Lock(GraphsToRunOnNextTickLock);
			GraphsToRunOnNextTick.Add(ExecInfo.GraphName);
			return true;
		}
	}
	
	return false;
}


void UOptimusDeformerInstance::SetConstantValueDirect(TSoftObjectPtr<UObject> InSourceObject, FOptimusValueContainerStruct const& InValue)
{
	// This is an editor only operation when constant nodes are edited in the graph and we want to see the result without a full compile step.
	if (IOptimusValueProvider* ValueProvider = Cast<IOptimusValueProvider>(InSourceObject.LoadSynchronous()))
	{
		if (FOptimusValueDescription* Description = ValueMap.Find(ValueProvider->GetValueIdentifier()))
		{
			check(Description->DataType == ValueProvider->GetValueDataType());
			
			if (Description->ValueUsage == EOptimusValueUsage::CPU)
			{
				Description->Value = InValue;
			}
			
			if (Description->ValueUsage == EOptimusValueUsage::GPU)
			{
				Description->ShaderValue = InValue.GetShaderValue(ValueProvider->GetValueDataType());
			}
		}	
	}
}
