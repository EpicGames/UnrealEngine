// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusExecutionDomain.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusExecutionDomain)


FString FOptimusExecutionDomain::AsExpression() const
{
	if (Type==EOptimusExecutionDomainType::DomainName)
	{
		return Name.ToString();
	}

	return Expression;
}

bool FOptimusExecutionDomain::IsDefined() const
{
	if (Type == EOptimusExecutionDomainType::DomainName)
	{
		return !Name.IsNone();
	}

	return !Expression.TrimStartAndEnd().IsEmpty();
}
