// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusFunctionNodeGraph.h"
#include "OptimusFunctionNodeGraphHeaderWithGuid.h"
#include "OptimusDeformer.h"
#include "OptimusObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusFunctionNodeGraph)

FName UOptimusFunctionNodeGraph::AccessSpecifierPublicName = TEXT("Public");
FName UOptimusFunctionNodeGraph::AccessSpecifierPrivateName = TEXT("Private");


UOptimusFunctionNodeGraph::UOptimusFunctionNodeGraph()
{
}

void UOptimusFunctionNodeGraph::PostLoad()
{
	Super::PostLoad();

	FOptimusObjectVersion::Type ObjectVersion = static_cast<FOptimusObjectVersion::Type>(GetLinkerCustomVersion(FOptimusObjectVersion::GUID));
	if (ObjectVersion < FOptimusObjectVersion::FunctionGraphUseGuid)
	{
		check(!Guid.IsValid());
		Guid = GetGuidForGraphWithoutGuid(TSoftObjectPtr<UOptimusFunctionNodeGraph>(this));
	}

	check(Guid.IsValid());
}

FString UOptimusFunctionNodeGraph::GetNodeName() const
{
	return GetName();
}

void UOptimusFunctionNodeGraph::Init()
{
	Guid = FGuid::NewGuid();
}

FOptimusFunctionGraphIdentifier UOptimusFunctionNodeGraph::GetGraphIdentifier() const
{
	FOptimusFunctionGraphIdentifier GraphIdentifier;
	GraphIdentifier.Asset = GetTypedOuter<UOptimusDeformer>();
	
	check(Guid.IsValid());
	
	GraphIdentifier.Guid = Guid;

	return GraphIdentifier;
}

TArray<FName> UOptimusFunctionNodeGraph::GetAccessSpecifierOptions() const
{
	return {AccessSpecifierPublicName, AccessSpecifierPrivateName};
}

FOptimusFunctionNodeGraphHeaderWithGuid UOptimusFunctionNodeGraph::GetHeaderWithGuid() const
{
	FOptimusFunctionNodeGraphHeaderWithGuid Header;

	Header.FunctionGraphGuid = Guid;
	Header.FunctionName = GetFName();
	Header.Category = Category;

	return Header;	
}

FGuid UOptimusFunctionNodeGraph::GetGuid() const
{
	return Guid;
}

FGuid UOptimusFunctionNodeGraph::GetGuidForGraphWithoutGuid(TSoftObjectPtr<UOptimusFunctionNodeGraph> InGraph)
{
	return FGuid::NewDeterministicGuid(InGraph.ToString());
}


#if WITH_EDITOR


bool UOptimusFunctionNodeGraph::CanEditChange(const FProperty* InProperty) const
{
	if (InProperty == StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UOptimusNodeSubGraph, InputBindings)))
	{
		return false;
	}
	else if (InProperty == StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UOptimusNodeSubGraph, OutputBindings)))
	{
		return false;
	}

	return true;
}

#endif
