// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusGeometryReadbackProcessor.h"

#include "MeshAttributes.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "StaticMeshAttributes.h"
#include "Animation/MeshDeformerGeometryReadback.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Async/ParallelFor.h"

#if WITH_EDITORONLY_DATA

// At the moment there are meshes with MeshToImportVertexMap not matching their mesh description for some reason
// Once the root cause is determined we can remove this
#define USE_MESH_TO_IMPORT_VERTEX_MAP 0

FOptimusGeometryReadbackProcessor& FOptimusGeometryReadbackProcessor::Get()
{
	static FOptimusGeometryReadbackProcessor Instance;
	return Instance;
}

void FOptimusGeometryReadbackProcessor::Add(TSharedPtr<FGeometryReadback> InReadback)
{
	Get().GeometryReadbacks.Enqueue(InReadback);
}

void FOptimusGeometryReadbackProcessor::ProcessCompletedGeometryReadback_RenderThread()
{
	// Try to process readbacks in the order they are added,
	// later readbacks won't be processed until earlier readbacks have been popped and processed
	while (!GeometryReadbacks.IsEmpty())
	{
		TSharedPtr<FGeometryReadback> Readback = *GeometryReadbacks.Peek();

		// Stop processing if a buffer is not ready
		if ((Readback->Position.bShouldReadback && Readback->Position.ReadbackData.Num() == 0) ||
			(Readback->Tangent.bShouldReadback && Readback->Tangent.ReadbackData.Num() == 0) ||
			(Readback->Color.bShouldReadback && Readback->Color.ReadbackData.Num() == 0))
		{
			break;
		}

		// Ready for processing
		GeometryReadbacks.Dequeue();
		
		auto ReadbackGeometryToMeshDescription = [Readback]()
		{
			const int32 LODIndex = Readback->LodIndex;
			const TArray<uint8>& Positions = Readback->Position.ReadbackData; 
			const TArray<uint8>& NormalsTangents = Readback->Tangent.ReadbackData; 
			const TArray<uint8>& Colors = Readback->Color.ReadbackData; 

			const int32 NumVertPositions = Positions.Num() / SizeOfVertPosition;
			const int32 NumVertNormalsTangents= NormalsTangents.Num() / SizeOfVertTangents;
			const int32 NumVertColors = Colors.Num() / SizeOfVertColor;

			int32 NumVertices = 0;
			// Make sure sizes of all buffers make sense
			if (ensure(NumVertPositions == NumVertNormalsTangents || NumVertPositions == 0 || NumVertNormalsTangents == 0) && 
				ensure(NumVertNormalsTangents == NumVertColors || NumVertNormalsTangents == 0 || NumVertColors == 0) &&
				ensure(NumVertPositions == NumVertColors || NumVertPositions == 0 || NumVertColors == 0))
			{
				NumVertices =
					NumVertPositions != 0 ? NumVertPositions :
					NumVertNormalsTangents != 0 ? NumVertNormalsTangents :
					NumVertColors != 0 ? NumVertColors : 0;
			}

			// Readback has no valid data,
			if (!ensure(NumVertices != 0))
			{
				return;
			}
			
			ProcessReadbackRequestingMeshDescription(Readback,NumVertices);
			ProcessReadbackRequestingVertexDataArray(Readback,NumVertices);
		};

		// Readback data should be processed sequentially on a background thread, so using last task as prerequisite
		if (LastReadbackProcessingTask_RenderThread)
		{
			LastReadbackProcessingTask_RenderThread = FFunctionGraphTask::CreateAndDispatchWhenReady(ReadbackGeometryToMeshDescription, TStatId(), LastReadbackProcessingTask_RenderThread, ENamedThreads::AnyBackgroundThreadNormalTask);	
		}
		else
		{
			LastReadbackProcessingTask_RenderThread = FFunctionGraphTask::CreateAndDispatchWhenReady(ReadbackGeometryToMeshDescription, TStatId(), NULL, ENamedThreads::AnyBackgroundThreadNormalTask);	
		}
	}	
}

void FOptimusGeometryReadbackProcessor::ProcessReadbackRequestingMeshDescription(const TSharedPtr<FGeometryReadback>& Readback, int32 NumVertices)
{
	bool bNeedMeshDescription = false;
	for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : Readback->GeometryReadbackRequests)
	{
		if (Request->MeshDescriptionCallback_AnyThread.IsSet())
		{
			bNeedMeshDescription = true;	
		}
	}

	if (!bNeedMeshDescription)
	{
		return;
	}

	const int32 LODIndex = Readback->LodIndex;
	const TArray<uint8>& Positions = Readback->Position.ReadbackData; 
	const TArray<uint8>& NormalsTangents = Readback->Tangent.ReadbackData; 
	const TArray<uint8>& Colors = Readback->Color.ReadbackData; 

	const USkeletalMesh* SkeletalMesh = Readback->SkeletalMesh.Get();

	if (!SkeletalMesh)
	{
		return;
	}
	
	FMeshDescription MeshDescription;
	SkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription);
	
	// Avoid conditional build during parallel for loop below
	MeshDescription.BuildVertexIndexers();

	TMeshAttributesRef<FVertexID, FVector3f> PositionAttribute =
		MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	TMeshAttributesRef<FVertexInstanceID, FVector3f> NormalAttribute =
		MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	TMeshAttributesRef<FVertexInstanceID, FVector3f> TangentAttribute = 
		MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	TMeshAttributesRef<FVertexInstanceID, float> BinormalSignAttribute = 
		MeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);
	TMeshAttributesRef<FVertexID, FVector4f> ColorAttribute =
		MeshDescription.VertexAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);
	
	FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];

#if USE_MESH_TO_IMPORT_VERTEX_MAP
	const TArray<int32>& RenderToImportedMap = LODModel.MeshToImportVertexMap;
	uint32 MaxImportVertex = LODModel.MaxImportVertex;
#else
	const TArray<uint32>& RenderToImportedMap = LODModel.GetRawPointIndices();
	uint32 MaxImportVertex = 0;
	for (uint32 Raw : RenderToImportedMap)
	{
		MaxImportVertex = FMath::Max(MaxImportVertex, Raw);
	}
#endif
	
	TArray<std::atomic<int32>> ImportedVertWriteCount;
	ImportedVertWriteCount.SetNum(MaxImportVertex+1);
	for(std::atomic<int32>& Counter : ImportedVertWriteCount)
	{
		Counter = 0;
	}
	
	// Parallel for each render vert
	ParallelFor(NumVertices,
		[
		&Readback,
		&MeshDescription,
		&PositionAttribute, 
		&NormalAttribute, 
		&TangentAttribute, 
		&BinormalSignAttribute,
		&ColorAttribute,
		NumVertices,
		&RenderToImportedMap, 
		&Positions, 
		&NormalsTangents,
		&Colors,
		&ImportedVertWriteCount
		](int32 RenderVertIndex)
	{
		// int32 ImportedVertexIndex = RenderToImportedMap[RenderVertIndex];
		uint32 ImportedVertexIndex = RenderToImportedMap[RenderVertIndex];
		const FVertexID ImportedVertexID(ImportedVertexIndex);

		// Only write to each imported vert once
		if (++ImportedVertWriteCount[ImportedVertexIndex] > 1)
		{
			return;
		}
		
		// Positions
		if (Positions.Num() > 0 && PositionAttribute.IsValid() && PositionAttribute.GetNumElements() == Positions.Num())
		{
			const float* Ptr = reinterpret_cast<const float*>(Positions.GetData());
		
			const float PositionX = Ptr[RenderVertIndex*3 + 0];
			const float PositionY = Ptr[RenderVertIndex*3 + 1];
			const float PositionZ = Ptr[RenderVertIndex*3 + 2];

			PositionAttribute[ImportedVertexID] = FVector3f(PositionX,PositionY,PositionZ);
		}

		// Normals Tangents
		const TArrayView<const FVertexInstanceID> VertexInstances = MeshDescription.GetVertexVertexInstanceIDs(ImportedVertexID);
		if (NormalsTangents.Num() > 0)
		{
			const FPackedRGBA16N* Ptr = reinterpret_cast<const FPackedRGBA16N*>(NormalsTangents.GetData());
			const FVector3f Tangent = Ptr[RenderVertIndex*2 + 0].ToFVector3f();
			const FVector4f Normal = Ptr[RenderVertIndex*2 + 1].ToFVector4f();
		
			for (const FVertexInstanceID VertexInstanceID : VertexInstances)
			{
				if (ensure(NormalAttribute.IsValid()))
				{
					NormalAttribute[VertexInstanceID] = FVector3f(Normal);
				}

				if (ensure(TangentAttribute.IsValid()))
				{
					TangentAttribute[VertexInstanceID] = Tangent;
				}
				
				if (BinormalSignAttribute.IsValid())
				{
					BinormalSignAttribute[VertexInstanceID] = Normal.W;
				}
			}
		}

		// Colors
		if (Colors.Num() > 0 && ColorAttribute.IsValid())
		{
			check(sizeof(FColor) == 4);
		
			const FColor* Ptr = reinterpret_cast<const FColor*>(Colors.GetData());

			for (const FVertexInstanceID VertexInstanceID : VertexInstances)
			{
				ColorAttribute[VertexInstanceID] = FVector4f(Ptr[RenderVertIndex]);
			}
		}
	});

	for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : Readback->GeometryReadbackRequests)
	{
		if (Request->MeshDescriptionCallback_AnyThread.IsSet())
		{
			Request->MeshDescriptionCallback_AnyThread(MeshDescription);
			Request->bMeshDescriptionHandled = true;
		}
	}
}

void FOptimusGeometryReadbackProcessor::ProcessReadbackRequestingVertexDataArray(const TSharedPtr<FGeometryReadback>& Readback, int32 NumVertices)
{
	bool bNeedVertDataArrays= false;
	for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : Readback->GeometryReadbackRequests)
	{
		if (Request->VertexDataArraysCallback_AnyThread.IsSet())
		{
			bNeedVertDataArrays = true;
		}
	}

	if (!bNeedVertDataArrays)
	{
		return;
	}
	
	const TArray<uint8>& Positions = Readback->Position.ReadbackData; 
	const TArray<uint8>& NormalsTangents = Readback->Tangent.ReadbackData; 
	const TArray<uint8>& Colors = Readback->Color.ReadbackData; 

	const USkeletalMesh* SkeletalMesh = Readback->SkeletalMesh.Get();

	if (!SkeletalMesh)
	{
		return;
	}

	FMeshDeformerGeometryReadbackVertexDataArrays VertexDataArrays;

	VertexDataArrays.LODIndex = Readback->LodIndex;
	VertexDataArrays.Positions.SetNum(NumVertices);
	VertexDataArrays.Normals.SetNum(NumVertices);
	VertexDataArrays.Tangents.SetNum(NumVertices);
	VertexDataArrays.Colors.SetNum(NumVertices);
	
	// Parallel for each render vert
	ParallelFor(NumVertices,
		[
		&Readback,
		NumVertices,
		&Positions, 
		&NormalsTangents,
		&Colors,
		&VertexDataArrays
		](int32 RenderVertIndex)
	{
		
		// Positions
		if (Positions.Num() > 0)
		{
			const float* Ptr = reinterpret_cast<const float*>(Positions.GetData());
		
			const float PositionX = Ptr[RenderVertIndex*3 + 0];
			const float PositionY = Ptr[RenderVertIndex*3 + 1];
			const float PositionZ = Ptr[RenderVertIndex*3 + 2];

			VertexDataArrays.Positions[RenderVertIndex] = FVector3f(PositionX,PositionY,PositionZ);
		}

		// Normals Tangents
		if (NormalsTangents.Num() > 0)
		{
			const FPackedRGBA16N* Ptr = reinterpret_cast<const FPackedRGBA16N*>(NormalsTangents.GetData());
			const FVector3f Tangent = Ptr[RenderVertIndex*2 + 0].ToFVector3f();
			const FVector4f Normal = Ptr[RenderVertIndex*2 + 1].ToFVector4f();
		
			VertexDataArrays.Normals[RenderVertIndex] = Normal;	
			VertexDataArrays.Tangents[RenderVertIndex] = Tangent;	
		}

		// Colors
		if (Colors.Num() > 0)
		{
			check(sizeof(FColor) == 4);
		
			const FColor* Ptr = reinterpret_cast<const FColor*>(Colors.GetData());
			VertexDataArrays.Colors[RenderVertIndex] = FVector4f(Ptr[RenderVertIndex]);
		}
	});

	for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : Readback->GeometryReadbackRequests)
	{
		if (Request->VertexDataArraysCallback_AnyThread.IsSet())
		{
			Request->VertexDataArraysCallback_AnyThread(VertexDataArrays);
			Request->bVertexDataArraysHandled= true;
		}
	}

}

#endif // WITH_EDITORONLY_DATA
