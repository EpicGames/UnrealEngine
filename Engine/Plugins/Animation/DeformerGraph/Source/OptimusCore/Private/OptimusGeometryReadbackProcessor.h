// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SkeletalMeshDeformerHelpers.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "Engine/SkeletalMesh.h"
#include "Containers/MpscQueue.h"
#include "Animation/MeshDeformerGeometryReadback.h"

#if WITH_EDITORONLY_DATA
class FOptimusGeometryReadbackProcessor
{
public:

	struct FBufferReadback
	{
		bool bShouldReadback = false;
		TArray<uint8> ReadbackData;
		FComputeDataProviderRenderProxy::FReadbackCallback OnReadbackCompleted_RenderThread;
	};
	
	struct FGeometryReadback
	{
		uint64 FrameNumber = 0;
		TArray<TUniquePtr<FMeshDeformerGeometryReadbackRequest>> GeometryReadbackRequests;

		TWeakObjectPtr<USkeletalMesh> SkeletalMesh;
		int32 LodIndex = 0;
		
		FBufferReadback Position;
		FBufferReadback Tangent;
		FBufferReadback Color;
	};

	static FOptimusGeometryReadbackProcessor& Get();
	void Add(TSharedPtr<FGeometryReadback>);
	void ProcessCompletedGeometryReadback_RenderThread();

private:

	static void ProcessReadbackRequestingMeshDescription(const TSharedPtr<FGeometryReadback>& Readback, int32 NumVertices);
	static void ProcessReadbackRequestingVertexDataArray(const TSharedPtr<FGeometryReadback>& Readback, int32 NumVertices);
	
	// All data provider shares the same processor singleton
	TMpscQueue<TSharedPtr<FGeometryReadback>> GeometryReadbacks;
	FGraphEventRef LastReadbackProcessingTask_RenderThread;
	
	static constexpr int32 SizeOfVertPosition = FSkeletalMeshDeformerHelpers::PosBufferElementMultiplier * FSkeletalMeshDeformerHelpers::PosBufferBytesPerElement; // float3, 3 * float, 12 Bytes per vert
	static constexpr int32 SizeOfVertTangents = FSkeletalMeshDeformerHelpers::TangentBufferElementMultiplier * FSkeletalMeshDeformerHelpers::TangentBufferBytesPerElement; // 2 * half4, which is 4 * 2 bytes, 16 bytes per vert
	static constexpr int32 SizeOfVertColor = FSkeletalMeshDeformerHelpers::ColorBufferBytesPerElement; // 1 * 4 * 8bit, 4 bytes per vert


};
#endif
