// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusHelpers.h"

#include "Matrix3x4.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusExpressionEvaluator.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "ShaderParameterMetadataBuilder.h"
#include "ShaderParameterMetadata.h"
#include "SkeletalRenderPublic.h"
#include "Animation/SkinWeightProfileManager.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Components/SkeletalMeshComponent.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "OptimusHelpers"

FName Optimus::GetUniqueNameForScope(UObject* InScopeObj, FName InName)
{
	// If there's already an object with this name, then attempt to make the name unique.
	// For some reason, MakeUniqueObjectName does not already do this check, hence this function.
	if (StaticFindObjectFast(UObject::StaticClass(), InScopeObj, InName) != nullptr)
	{
		InName = MakeUniqueObjectName(InScopeObj, UObject::StaticClass(), InName);
	}

	return InName;
}

Optimus::FUniqueNameGenerator::FUniqueNameGenerator(UObject* InScopeObject)
{
	ScopeObject = InScopeObject;
}

FName Optimus::FUniqueNameGenerator::GetUniqueName(FName InName)
{
	FName Result = Optimus::GetUniqueNameForScope(ScopeObject, InName);
	Result = GenerateUniqueNameFromExistingNames(Result, GeneratedName);

	// Result be usable at this point since the name number strictly increases.
	// Only take the slow route if there is still a name collision for mysterious reasons
	if (!ensure(StaticFindObjectFast(UObject::StaticClass(), ScopeObject, Result) == nullptr))
	{
		do
		{
			Result.SetNumber( Result.GetNumber() + 1);
		} while (StaticFindObjectFast(UObject::StaticClass(), ScopeObject, Result) != nullptr || GeneratedName.Contains(Result));
	}

	GeneratedName.Add(Result);
	return Result;
}

FName Optimus::GetSanitizedNameForHlsl(FName InName)
{
	FString Name = InName.ToString();

	// Remove spaces
	Name.ReplaceInline(TEXT(" "), TEXT(""));

	// Replace other bad characters with "_" 
	for (int32 i = 0; i < Name.Len(); ++i)
	{
		TCHAR& C = Name[i];

		const bool bGoodChar =
			FChar::IsAlpha(C) ||											// Any letter (upper and lowercase) anytime
			(C == '_') ||  													// _  
			((i > 0) && FChar::IsDigit(C));									// 0-9 after the first character

		if (!bGoodChar)
		{
			C = '_';
		}
	}

	return *Name;
}

const void Optimus::ConvertFTransformToFMatrix3x4(const FTransform& InTransform, FShaderValueContainerView OutShaderValue)
{
	// Code taken from FGPUBaseSkinVertexFactory::FShaderDataType::UpdateBoneData
	if (ensure(OutShaderValue.ShaderValue.Num() == FShaderValueType::Get(EShaderFundamentalType::Float, 3, 4)->GetResourceElementSize()))
	{
		uint8* ShaderValuePtr = OutShaderValue.ShaderValue.GetData();

		FMatrix3x4& ShaderMat = *((FMatrix3x4*)(ShaderValuePtr));
		const FMatrix44f& Matrix = ConvertFTransformToFMatrix44f(InTransform);
		// Explicit SIMD implementation seems to be faster than standard implementation
#if PLATFORM_ENABLE_VECTORINTRINSICS
		VectorRegister4Float InRow0 = VectorLoadAligned(&(Matrix.M[0][0]));
		VectorRegister4Float InRow1 = VectorLoadAligned(&(Matrix.M[1][0]));
		VectorRegister4Float InRow2 = VectorLoadAligned(&(Matrix.M[2][0]));
		VectorRegister4Float InRow3 = VectorLoadAligned(&(Matrix.M[3][0]));

		VectorRegister4Float Temp0 = VectorShuffle(InRow0, InRow1, 0, 1, 0, 1);
		VectorRegister4Float Temp1 = VectorShuffle(InRow2, InRow3, 0, 1, 0, 1);
		VectorRegister4Float Temp2 = VectorShuffle(InRow0, InRow1, 2, 3, 2, 3);
		VectorRegister4Float Temp3 = VectorShuffle(InRow2, InRow3, 2, 3, 2, 3);

		VectorStoreAligned(VectorShuffle(Temp0, Temp1, 0, 2, 0, 2), &(ShaderMat.M[0][0]));
		VectorStoreAligned(VectorShuffle(Temp0, Temp1, 1, 3, 1, 3), &(ShaderMat.M[1][0]));
		VectorStoreAligned(VectorShuffle(Temp2, Temp3, 0, 2, 0, 2), &(ShaderMat.M[2][0]));
#else
		Matrix.To3x4MatrixTranspose((float*)ShaderMat.M);
#endif
	}
}

bool Optimus::RenameObject(UObject* InObjectToRename, const TCHAR* InNewName, UObject* InNewOuter)
{
	return InObjectToRename->Rename(InNewName, InNewOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
}

void Optimus::RemoveObject(UObject* InObjectToRemove)
{
	InObjectToRemove->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	InObjectToRemove->MarkAsGarbage();
}

TArray<UClass*> Optimus::GetClassObjectsInPackage(UPackage* InPackage)
{
	TArray<UObject*> Objects;
	GetObjectsWithOuter(InPackage, Objects, EGetObjectsFlags::None);

	TArray<UClass*> ClassObjects;
	for (UObject* Object : Objects)
	{
		if (UClass* Class = Cast<UClass>(Object))
		{
			ClassObjects.Add(Class);
		}
	}

	return ClassObjects;
}

FText Optimus::GetTypeDisplayName(UScriptStruct* InStructType)
{
#if WITH_EDITOR
	FText DisplayName = InStructType->GetDisplayNameText();
#else
	FText DisplayName = FText::FromName(InStructType->GetFName());
#endif

	return DisplayName;
}

FName Optimus::GetMemberPropertyShaderName(UScriptStruct* InStruct, const FProperty* InMemberProperty)
{
	if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStruct))
	{
		// Remove Spaces
		FString ShaderMemberName = InStruct->GetAuthoredNameForField(InMemberProperty).Replace(TEXT(" "), TEXT(""));

		if (ensure(!ShaderMemberName.IsEmpty()))
		{
			// Sanitize the name, user defined struct can have members with names that start with numbers
			if (!FChar::IsAlpha(ShaderMemberName[0]) && !FChar::IsUnderscore(ShaderMemberName[0]))
			{
				ShaderMemberName = FString(TEXT("_")) + ShaderMemberName;
			}
		}

		return *ShaderMemberName;
	}

	return InMemberProperty->GetFName();
}

namespace Optimus::Private
{
	FName GetTypeNameForGuid(const FGuid& Guid)
	{
		return FName(*FString::Printf(TEXT("FUserDefinedStruct_%s"), *Guid.ToString()));
	}
}

FName Optimus::GetTypeName(UScriptStruct* InStructType, bool bInShouldGetUniqueNameForUserDefinedStruct)
{
	if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStructType))
	{
		if (bInShouldGetUniqueNameForUserDefinedStruct)
		{
			return Optimus::Private::GetTypeNameForGuid(UserDefinedStruct->GetCustomGuid());
		}
	}

	return FName(*InStructType->GetStructCPPName());
}

FName Optimus::GetTypeName(const FAssetData& InStructAsset)
{
	check(InStructAsset.AssetClassPath == UUserDefinedStruct::StaticClass()->GetClassPathName());

	FGuid Guid;

	// UUserDefinedStruct::Guid is asset registry searchable, we can find it without loading the actual asset
	static const FName NAME_Guid = GET_MEMBER_NAME_CHECKED(UUserDefinedStruct, Guid);

	ensure(InStructAsset.GetTagValue(NAME_Guid, Guid));

	return Optimus::Private::GetTypeNameForGuid(Guid);
}

void Optimus::ConvertObjectPathToShaderFilePath(FString& InOutPath)
{
	// Shader compiler recognizes "/Engine/Generated/..." path as special. 
	// It doesn't validate file suffix etc.
	InOutPath = FString::Printf(TEXT("/Engine/Generated/UObject%s.ush"), *InOutPath);
	// Shader compilation result parsing will break if it finds ':' where it doesn't expect.
	InOutPath.ReplaceCharInline(TEXT(':'), TEXT('@'));
}

bool Optimus::ConvertShaderFilePathToObjectPath(FString& InOutPath)
{
	if (!InOutPath.RemoveFromStart(TEXT("/Engine/Generated/UObject")))
	{
		return false;
	}

	InOutPath.ReplaceCharInline(TEXT('@'), TEXT(':'));
	InOutPath.RemoveFromEnd(TEXT(".ush"));
	return true;
}

FString Optimus::GetCookedKernelSource(
	const FString& InObjectPathName,
	const FString& InShaderSource,
	const FString& InKernelName,
	FIntVector InGroupSize,
	const TCHAR* InReadNumThreadsPerInvocationFunctionName,
	const TCHAR* InReadThreadIndexOffsetFunctionName,
	bool bInIsUnifiedDispatch
	)
{
	// FIXME: Create source range mappings so that we can go from error location to
	// our source.
	FString Source = InShaderSource;

#if PLATFORM_WINDOWS
	// Remove old-school stuff.
	Source.ReplaceInline(TEXT("\r"), TEXT(""));
#endif

	FString ShaderPathName = InObjectPathName;
	Optimus::ConvertObjectPathToShaderFilePath(ShaderPathName);

	const bool bHasKernelKeyword = Source.Contains(TEXT("KERNEL"), ESearchCase::CaseSensitive);

	const FString ComputeShaderUtilsInclude = TEXT("#include \"/Engine/Private/ComputeShaderUtils.ush\"");
	
	const FString KernelFunc = FString::Printf(
		TEXT("[numthreads(%d,%d,%d)]\nvoid %s(uint3 GroupId : SV_GroupID, uint GroupIndex : SV_GroupIndex)"), 
		InGroupSize.X, InGroupSize.Y, InGroupSize.Z, *InKernelName);
	
	const FString TheadIndexForInvocation = FString::Printf(
	TEXT("GetUnWrappedDispatchThreadId(GroupId, GroupIndex, %d)"),
		InGroupSize.X * InGroupSize.Y * InGroupSize.Z
	);

	const FString ThreadIndexOffsetForKernel = FString::Printf(TEXT(""));

	// Avoid thread index check for unified dispatch to allow for the usage of Group Sync primitives
	// Shader compiler does not like early return when those primitives are used
	const FString IndexCheckAndApplyOffset= bInIsUnifiedDispatch ?
		FString::Printf(
			TEXT(
			"uint Index = %s;\n" ),
			*TheadIndexForInvocation
			) :
		FString::Printf(
			TEXT(
			"uint IndexForInvocation = %s;\n"
			"if (IndexForInvocation >= %s::%s()) return;\n"
			"uint Index = IndexForInvocation + %s::%s();\n"),
			*TheadIndexForInvocation,
			GetKernelInternalNamespaceName(),
			InReadNumThreadsPerInvocationFunctionName,
			GetKernelInternalNamespaceName(),
			InReadThreadIndexOffsetFunctionName
			);
	
	if (bHasKernelKeyword)
	{
		Source.ReplaceInline(TEXT("KERNEL"), TEXT("void __kernel_func(uint Index)"), ESearchCase::CaseSensitive);

		return FString::Printf(
			TEXT(
				"#line 1 \"%s\"\n"
				"%s\n\n"
				"%s\n\n"
				"%s\n"
				"{\n"
				"%s\n"
				"__kernel_func(Index);\n"
				"}\n"
				), *ShaderPathName, *Source, *ComputeShaderUtilsInclude,*KernelFunc, *IndexCheckAndApplyOffset);
	}
	else
	{
		return FString::Printf(
		TEXT(
			"%s\n"
			"%s\n"
			"{\n"
			"%s\n"
			"#line 1 \"%s\"\n"
			"%s\n"
			"}\n"
			), *ComputeShaderUtilsInclude,*KernelFunc, *IndexCheckAndApplyOffset, *ShaderPathName, *Source);
	}
}

const TCHAR* Optimus::GetKernelInternalNamespaceName()
{
	static const TCHAR* KernelInternalNamespaceName = TEXT("KernelInternal");
	return KernelInternalNamespaceName;
}


Optimus::Expression::FParseResult Optimus::ParseExecutionDomainExpression(
	const FString& InExpression,
	TWeakObjectPtr<const UOptimusComponentSource> InComponentSource
	)
{
	TMap<FName, float> EngineConstants;

	for(FName ExecutionDomain: InComponentSource->GetExecutionDomains())
	{
		EngineConstants.Add(ExecutionDomain, 0);
	}

	Expression::FEngine Engine;
	return Engine.Parse(InExpression, [EngineConstants](FName InName)->TOptional<float>
	{
		if (const float* Value = EngineConstants.Find(InName))
		{
			return *Value;
		};

		return {};
	});
}

TVariant<bool, FText> Optimus::IsExecutionDomainUnifiedDispatchOnly(const FString& InExpression,
	TWeakObjectPtr<const UOptimusComponentSource> InComponentSource)
{
	bool bUnifiedDispatchOnly = true;
	{
		using namespace Optimus::Expression;
		FParseResult ParseResult = ParseExecutionDomainExpression(InExpression, InComponentSource);
		if (ParseResult.IsType<FParseError>())
		{
			return TVariant<bool, FText>(TInPlaceType<FText>(), FText::Format(
					LOCTEXT("ExecutionDomainParsingError", "Error while parsing Execution Domain: {0}"),
					FText::FromString(ParseResult.Get<FParseError>().Message)));
		}

		FExpressionObject ExpressionObject = ParseResult.Get<FExpressionObject>();

		// Only fixed domain supports both unified and non-unified dispatch
		if (TOptional<FName> FixedDomain = ExpressionObject.GetAsSingleConstant())
		{
			bUnifiedDispatchOnly = false;
		}
	}

	return TVariant<bool, FText>(TInPlaceType<bool>(), bUnifiedDispatchOnly);
}


bool Optimus::EvaluateExecutionDomainExpressionParseResult(
	const TVariant<Expression::FExpressionObject, Expression::FParseError>& InParseResult,
	TWeakObjectPtr<const UOptimusComponentSource> InComponentSource,
	TWeakObjectPtr<const UActorComponent> InWeakComponent,
	TArray<int32>& OutInvocationThreadCount
	)
{
	if (!InWeakComponent.IsValid() || !InComponentSource.IsValid())
	{
		return false;
	}
	
	using namespace Expression;
	if (InParseResult.IsType<FParseError>())
	{
		return false;
	}

	const UActorComponent* Component = InWeakComponent.Get();
	int32 LodIndex = InComponentSource->GetLodIndex(Component);
	int32 NumInvocations = InComponentSource->GetDefaultNumInvocations(Component, LodIndex);

	// In some cases the component can be set to an unusable state intentionally
	// such as when the OptimusEditor shuts down,
	// in which case we don't have data to work with so simply do nothing
	if (NumInvocations < 1)
	{
		return false;
	}

	// In the case of Fixed Domain, multiple invocations are supported when the data interface does not support unified dispatch
	// in the case of unified dispatch, it run a single invocation with thread count equal to the sum of all invocations
	if (TOptional<FName> FixedDomain = InParseResult.Get<FExpressionObject>().GetAsSingleConstant())
	{
		InComponentSource->GetComponentElementCountsForExecutionDomain(
				*FixedDomain,
				Component,
				LodIndex,
				OutInvocationThreadCount);
		
		return OutInvocationThreadCount.Num() > 0;
	}
	
	// Other custom domains only support single invocation / unified dispatch
	TMap<FName, float> EngineConstants;

	for(FName ExecutionDomain: InComponentSource->GetExecutionDomains())
	{
		TArray<int32> ElementCounts;
		if (!InComponentSource->GetComponentElementCountsForExecutionDomain(
				ExecutionDomain,
				Component,
				LodIndex,
				ElementCounts))
		{
			return false;
		}

		if (!ensure(NumInvocations == ElementCounts.Num()))
		{
			// Component source needs to provide as many values for each of its execution domains as there are invocations
			return false;
		}

		int32 TotalCount = 0;
		for (int32 Count : ElementCounts)
		{
			TotalCount += Count;
		}

		EngineConstants.Add(ExecutionDomain, TotalCount);	
	}

	FEngine Engine;
	const int32 TotalElementCountForUnifiedDispatch = static_cast<int32>(Engine.Execute(InParseResult.Get<FExpressionObject>(), 
		[&EngineConstants](FName InName)->TOptional<float>
		{
			if (const float* Value = EngineConstants.Find(InName))
			{
				return *Value;
			};

			return {};
		}));
	
	OutInvocationThreadCount.Add(TotalElementCountForUnifiedDispatch);
	
	return true;	
}

bool Optimus::FindMovedItemInNameArray(const TArray<FName>& Old, const TArray<FName>& New, FName& OutSubjectName, FName& OutNextName)
{
	FName NameToMove = NAME_None;
	int32 DivergeIndex = INDEX_NONE;

	check(New.Num() == Old.Num())
	
	for (int32 Index = 0; Index < New.Num(); Index++)
	{
		if (New[Index]!= Old[Index] && DivergeIndex == INDEX_NONE)
		{
			DivergeIndex = Index;
			continue;
		}

		if (DivergeIndex != INDEX_NONE)
		{
			if (New[DivergeIndex] == Old[Index])
			{
				NameToMove = Old[DivergeIndex];
			}
			else if (ensure(New[Index] == Old[DivergeIndex]))
			{
				NameToMove = New[DivergeIndex];
			}
			break;
		}
	}

	if (DivergeIndex != INDEX_NONE)
	{
		OutSubjectName = NameToMove;
		
		const int32 NameIndex = New.IndexOfByPredicate([NameToMove](const FName& InBinding)
		{
			return InBinding == NameToMove;
		}); 

		const int32 NextNameIndex = NameIndex + 1;
		OutNextName = New.IsValidIndex(NextNameIndex) ?  FName(New[NextNameIndex]) : NAME_None;

		return true;
	}

	OutSubjectName = NAME_None;
	OutNextName = NAME_None;
	return false;
}

FName Optimus::GenerateUniqueNameFromExistingNames(FName InBaseName, const TArray<FName>& InExistingNames)
{
	TMap<FName, int32> BaseNameToMaxNumber;

	for(const FName ExistingName : InExistingNames)
	{
		const int32 Number = ExistingName.GetNumber();
		FName BaseName = ExistingName;
		BaseName.SetNumber(0);

		if (const int32* ExistingNumber = BaseNameToMaxNumber.Find(BaseName))
		{
			BaseNameToMaxNumber[BaseName] = FMath::Max(Number, *ExistingNumber);
		}
		else
		{
			BaseNameToMaxNumber.Add(BaseName) = Number;
		}
	}

	FName NewName = InBaseName;
	const int32 InputNumber = InBaseName.GetNumber();
	FName InputBaseName = InBaseName;
	InputBaseName.SetNumber(0);
	if (const int32* ExistingNumber = BaseNameToMaxNumber.Find(InputBaseName))
	{
		if (InputNumber <= *ExistingNumber)
		{
			NewName.SetNumber(*ExistingNumber+1);
		}
	}

	return NewName;
}

FString Optimus::MakeUniqueValueName(const FString& InValueName, int32 InUniqueIndex)
{
	return InValueName + TEXT("_") + FString::FromInt(InUniqueIndex);
}

FString Optimus::ExtractSourceValueName(const FString& InUniqueValueName)
{
	int32 LastUnderscoreIndex;
	InUniqueValueName.FindLastChar(TEXT('_'), LastUnderscoreIndex);
	if (ensure(LastUnderscoreIndex != INDEX_NONE))
	{
		return InUniqueValueName.Left(LastUnderscoreIndex);
	}

	return InUniqueValueName;
}

bool Optimus::IsSkinWeightProfileAvailable(const FSkeletalMeshLODRenderData& InLodRenderData, FName InSkinWeightProfile)
{
	if (InSkinWeightProfile.IsNone())
	{
		return true;
	}

	const FSkinWeightProfileStack ProfileStack{InSkinWeightProfile};
	FSkinWeightVertexBuffer* Buffer = InLodRenderData.SkinWeightProfilesData.GetOverrideBuffer(ProfileStack);
	return Buffer != nullptr;
}

void Optimus::RequestSkinWeightProfileForDeformer(USkeletalMeshComponent* InSkeletalMeshComponent, FName InSkinWeightProfile, TOptional<int32> InLOD)
{
	if (!IsValid(InSkeletalMeshComponent))
	{
		return;
	}
	FSkeletalMeshObject* SkeletalMeshObject = InSkeletalMeshComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	
	for (int32 LODIndex = 0; LODIndex < SkeletalMeshRenderData.NumInlinedLODs; LODIndex++)
	{
		if (!InSkeletalMeshComponent->HasMeshDeformer(LODIndex))
		{
			continue;
		}

		FSkeletalMeshLODRenderData const& LodRenderData = SkeletalMeshRenderData.LODRenderData[LODIndex];
		if (!LodRenderData.SkinWeightProfilesData.ContainsProfile(InSkinWeightProfile))
		{
			continue;
		}
		
		bool bShouldRequest = false;
		if (InLOD)
		{
			if (*InLOD == LODIndex)
			{
				bShouldRequest = true;
			}
		}
		else
		{
			bShouldRequest = true;
		}

		if (!bShouldRequest)
		{
			continue;
		}
		
		if (!IsSkinWeightProfileAvailable(LodRenderData, InSkinWeightProfile))
		{
			const FSkinWeightProfileStack ProfileStack{InSkinWeightProfile};
			// Put in a skin weight profile request
			if (FSkinWeightProfileManager* Manager = FSkinWeightProfileManager::Get(InSkeletalMeshComponent->GetWorld()))
			{
				FRequestFinished DummyCallback = [](TWeakObjectPtr<USkeletalMesh> WeakMesh, FSkinWeightProfileStack ProfileStack){};
				Manager->RequestSkinWeightProfileStack(ProfileStack, InSkeletalMeshComponent->GetSkinnedAsset(), InSkeletalMeshComponent, DummyCallback, LODIndex);
			}
		}	
	}
}


#undef LOCTEXT_NAMESPACE
