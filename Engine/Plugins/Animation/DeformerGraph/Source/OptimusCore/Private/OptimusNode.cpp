// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode.h"

#include "Actions/OptimusNodeActions.h"
#include "OptimusActionStack.h"
#include "OptimusDataDomain.h"
#include "OptimusCoreModule.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusBindingTypes.h"
#include "OptimusDeformer.h"
#include "OptimusDiagnostic.h"
#include "OptimusHelpers.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "OptimusObjectVersion.h"
#include "Actions/OptimusNodeGraphActions.h"

#include "Algo/Reverse.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode)


const FName UOptimusNode::CategoryName::DataInterfaces("Data Interfaces");
const FName UOptimusNode::CategoryName::Deformers("Deformers");
const FName UOptimusNode::CategoryName::Resources("Resources");
const FName UOptimusNode::CategoryName::Variables("Variables");
const FName UOptimusNode::CategoryName::Values("Values");

// NOTE: There really should be a central place for these. Magic strings are _bad_.
const FName UOptimusNode::PropertyMeta::Category("Category");
const FName UOptimusNode::PropertyMeta::Input("Input");
const FName UOptimusNode::PropertyMeta::Output("Output");
const FName UOptimusNode::PropertyMeta::Resource("Resource");
const FName UOptimusNode::PropertyMeta::AllowParameters("AllowParameters");

UOptimusNode::UOptimusNode()
{
	// TODO: Clean up properties (i.e. remove EditAnywhere, VisibleAnywhere for outputs).
}

UOptimusNode::~UOptimusNode()
{
	
}


FName UOptimusNode::GetNodeName() const
{
	return GetClass()->GetFName();
}


FText UOptimusNode::GetDisplayName() const
{
	if (DisplayName.IsEmpty())
	{
		FString Name = GetNodeName().ToString();
		FString PackageName, NodeName;

		if (!Name.Split("_", &PackageName, &NodeName))
		{
			NodeName = Name;
		}

		// Try to make the name a bit prettier.
		return FText::FromString(FName::NameToDisplayString(NodeName, false));
	}

	return DisplayName;
}


bool UOptimusNode::SetDisplayName(FText InDisplayName)
{
	if (DisplayName.EqualTo(InDisplayName))
	{
		return false;
	}
	
	DisplayName = InDisplayName;

	Notify(EOptimusGraphNotifyType::NodeDisplayNameChanged);

	return true;
}

FText UOptimusNode::GetTooltipText() const
{
	return {};
}


bool UOptimusNode::SetGraphPosition(const FVector2D& InPosition)
{
	return GetActionStack()->RunAction<FOptimusNodeAction_MoveNode>(this, InPosition);
}


bool UOptimusNode::SetGraphPositionDirect(
	const FVector2D& InPosition
	)
{
	if (InPosition.ContainsNaN() || InPosition.Equals(GraphPosition))
	{
		return false;
	}

	GraphPosition = InPosition;


	Notify(EOptimusGraphNotifyType::NodePositionChanged);

	return true;
}


FString UOptimusNode::GetNodePath() const
{
	UOptimusNodeGraph* Graph = GetOwningGraph();
	FString GraphPath(TEXT("<Unknown>"));
	if (Graph)
	{
		GraphPath = Graph->GetCollectionPath();
	}

	return UOptimusNodeGraph::ConstructPath(GraphPath, GetName(), {});
}


UOptimusNodeGraph* UOptimusNode::GetOwningGraph() const
{
	return Cast<UOptimusNodeGraph>(GetOuter());
}

UOptimusNodePin* UOptimusNode::GetPinByName(const FName& PinName) const
{
	for (UOptimusNodePin* Pin : Pins)
	{
		if(Pin->GetName() == PinName)
		{
			return Pin;
		}
	}
	return nullptr;
}

TArray<UOptimusNodePin*> UOptimusNode::GetPinsByDirection(EOptimusNodePinDirection InDirection,
	bool bInRecursive) const
{
	TArray<UOptimusNodePin*> Results;

	for (UOptimusNodePin* Pin : Pins)
	{
		if (Pin->GetDirection() == InDirection)
		{
			if (bInRecursive)
			{
				Results.Append(Pin->GetSubPinsRecursively(true));
			}
			else
			{
				Results.Add(Pin);
			}
		}
	}

	return Results;
}

bool UOptimusNode::CanConnectPinToPin(
	const UOptimusNodePin& InThisNodesPin,
	const UOptimusNodePin& InOtherNodesPin,
	FString* OutReason
	) const
{
	if (!CanConnectPinToNode(&InOtherNodesPin, InThisNodesPin.GetDirection(), OutReason))
	{
		return false;
	}
	
	// Check with overridden implementations on the node themselves.
	if (!ValidateConnection(InThisNodesPin, InOtherNodesPin, OutReason))
	{
		return false;
	}

	// Check with overridden implementations on the node themselves.
	if (!InOtherNodesPin.GetOwningNode()->ValidateConnection(InOtherNodesPin, InThisNodesPin, OutReason))
	{
		return false;
	}

	return true;
}


bool UOptimusNode::CanConnectPinToNode(
	const UOptimusNodePin* InOtherPin,
	EOptimusNodePinDirection InConnectionDirection,
	FString* OutReason) const
{
	if (!ensure(InOtherPin))
	{
		if (OutReason)
		{
			*OutReason = TEXT("No pin given.");
		}
		return false;
	}

	if (InConnectionDirection == InOtherPin->GetDirection())
	{
		if (OutReason)
		{
			const FString DirectionText =
				InConnectionDirection == EOptimusNodePinDirection::Input ? TEXT("input") : TEXT("output");
			
			*OutReason = FString::Printf(TEXT("Can't connect an %s pin to a %s")
				, *DirectionText , *DirectionText);
				
		}
		return false;
	}

	// Check for self-connect.
	if (this == InOtherPin->GetOwningNode())
	{
		if (OutReason)
		{
			*OutReason = TEXT("Can't connect pins on the same node.");
		}
		return false;
	}

	if (this->GetOwningGraph() != InOtherPin->GetOwningNode()->GetOwningGraph())
	{
		if (OutReason)
		{
			*OutReason = TEXT("Pins belong to nodes from two different graphs.");
		}
		return false;
	}

	// Will this connection cause a cycle?
	const UOptimusNode* OutputNode = InConnectionDirection == EOptimusNodePinDirection::Output ? this : InOtherPin->GetOwningNode();
	const UOptimusNode* InputNode = InConnectionDirection == EOptimusNodePinDirection::Input ? this : InOtherPin->GetOwningNode();

	if (GetOwningGraph()->DoesLinkFormCycle(OutputNode, InputNode))
	{
		if (OutReason)
		{
			*OutReason = TEXT("Connection results in a cycle.");
		}
		return false;
	}

	return true;
}


void UOptimusNode::SetDiagnosticLevel(EOptimusDiagnosticLevel InDiagnosticLevel)
{
	if (DiagnosticLevel != InDiagnosticLevel)
	{
		DiagnosticLevel = InDiagnosticLevel;
		Notify(EOptimusGraphNotifyType::NodeDiagnosticLevelChanged);
	}
}


UOptimusNodePin* UOptimusNode::FindPin(const FStringView InPinPath) const
{
	TArray<FName> PinPath = UOptimusNodePin::GetPinNamePathFromString(InPinPath);
	if (PinPath.IsEmpty())
	{
		return nullptr;
	}

	return FindPinFromPath(PinPath);
}


UOptimusNodePin* UOptimusNode::FindPinFromPath(const TArray<FName>& InPinPath) const
{
	UOptimusNodePin* const* PinPtrPtr = CachedPinLookup.Find(InPinPath);
	if (PinPtrPtr)
	{
		return *PinPtrPtr;
	}

	TArrayView<const TObjectPtr<UOptimusNodePin>> CurrentPins = MakeArrayView(Pins);
	UOptimusNodePin* FoundPin = nullptr;

	for (FName PinName : InPinPath)
	{
		if (CurrentPins.IsEmpty())
		{
			FoundPin = nullptr;
			break;
		}

		TObjectPtr<UOptimusNodePin> const* FoundPinPtr = CurrentPins.FindByPredicate(
		    [&PinName](const UOptimusNodePin* Pin) {
			    return Pin->GetFName() == PinName;
		    });

		if (FoundPinPtr == nullptr)
		{
			FoundPin = nullptr;
			break;
		}

		FoundPin = *FoundPinPtr;
		CurrentPins = FoundPin->GetSubPins();
	}

	CachedPinLookup.Add(InPinPath, FoundPin);

	return FoundPin;
}


UOptimusNodePin* UOptimusNode::FindPinFromProperty(
	const FProperty* InRootProperty,
	const FProperty* InSubProperty
	) const
{
	TArray<FName> PinPath;

	// This feels quite icky.
	if (InRootProperty == InSubProperty || InSubProperty == nullptr)
	{
		PinPath.Add(InRootProperty->GetFName());
	}
	else if (const FStructProperty* StructProp = CastField<const FStructProperty>(InRootProperty))
	{
		const UStruct *Struct = StructProp->Struct;

		// Crawl up the property hierarchy until we hit the root prop UStruct.
		while (ensure(InSubProperty))
		{
			PinPath.Add(InSubProperty->GetFName());

			if (const UStruct *OwnerStruct = InSubProperty->GetOwnerStruct())
			{
				if (ensure(OwnerStruct == Struct))
				{
					PinPath.Add(InRootProperty->GetFName());
					break;
				}
				else
				{
					return nullptr;
				}
			}
			else
			{
				InSubProperty = InSubProperty->GetOwner<const FProperty>();
			}
		}

		Algo::Reverse(PinPath);
	}

	return FindPinFromPath(PinPath);
}


TArray<UClass*> UOptimusNode::GetAllNodeClasses()
{
	TArray<UClass*> NodeClasses;
	
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		
		if (!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden) &&
			Class->IsChildOf(StaticClass()) &&
			Class->GetPackage() != GetTransientPackage())
		{
			NodeClasses.Add(Class);
		}
	}
	return NodeClasses;
}

FName UOptimusNode::GetAvailablePinNameStable(const UObject* InNodeOrPin, FName InName)
{
	TArray<const UOptimusNodePin*> PinsToCompare;
	if (const UOptimusNode* Node = Cast<const UOptimusNode>(InNodeOrPin))
	{
		PinsToCompare.Append(Node->GetPins());
	}
	else if (const UOptimusNodePin* Pin = Cast<const UOptimusNodePin>(InNodeOrPin))
	{
		PinsToCompare.Append(Pin->GetSubPins());
	}

	TArray<FName> PinNames;

	for(const UOptimusNodePin* Pin : PinsToCompare)
	{
		PinNames.Add(Pin->GetFName());
	}

	return Optimus::GenerateUniqueNameFromExistingNames(InName, PinNames);
}


void UOptimusNode::PostCreateNode()
{
	TGuardValue<bool> NodeConstructionGuard(bConstructingNode, true);
	InitializeTransientData();
	
	CachedPinLookup.Empty();

	Pins.Empty();
	
	ConstructNode();

	// Update the pins display names
	InitializePinsDisplayName();
}

void UOptimusNode::PostDeserializeNode()
{
	TGuardValue<bool> NodeConstructionGuard(bConstructingNode, true);
	InitializeTransientData();
	InitializePinsDisplayName();	
}


void UOptimusNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FOptimusObjectVersion::GUID);
}


void UOptimusNode::PostLoad()
{
	Super::PostLoad();

	// PostLoad fix up pass
	PostLoadNodeSpecificData();

	// Initialize transient data for nodes and pins
	PostDeserializeNode();
}

void UOptimusNode::PostLoadNodeSpecificData()
{
	// Earlier iterations didn't set this flag. 
	SetFlags(RF_Transactional);

	for (UOptimusNodePin* Pin : Pins)
	{
		Pin->ConditionalPostLoad();
	}
}

#if WITH_EDITOR
void UOptimusNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (const UOptimusNodeGraph *Graph = GetOwningGraph())
	{
		if (UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(Graph->GetCollectionRoot()))
		{
			Deformer->MarkModified();
		}
	}
	
}
#endif

void UOptimusNode::Notify(EOptimusGraphNotifyType InNotifyType)
{
	if (CanNotify())
	{
		UOptimusNodeGraph *Graph = Cast<UOptimusNodeGraph>(GetOuter());

		if (Graph)
		{
			Graph->Notify(InNotifyType, this);
		}
	}
}



void UOptimusNode::ConstructNode()
{
	CreatePinsFromStructLayout(GetClass(), nullptr);
}

void UOptimusNode::InitializeTransientData()
{
	// Called from three places, which are all possible way a node object can be created
	// 1. PostCreateNode
	// 2. PostLoad
	// 3. FOptimusEditorClipboard::ProcessPostCreateObject
}


void UOptimusNode::ExportState(FArchive& Ar) const
{
	// Take a copy of the node's core data that would allow the node to reconstruct itself during restore state
	
	// We have to use the proxy archive because FMemoryWriter simply asserts when the object has object references.
	// And even though we are serializing object references as strings. For things like pins we will recreate them anyway
	FObjectAndNameAsStringProxyArchive NodeProxyArchive(
			Ar, /* bInLoadIfFindFails=*/false);

	SerializeScriptProperties(NodeProxyArchive);
}

void UOptimusNode::ImportState(FArchive& Ar)
{
	// Don't try to load any object references, the referenced object is likely no longer valid for this node and have to be recreated (such as pins)
	// For objects we can find, it is likely that the object is owned by some top level object and is still alive and safe to reference.
	
	FObjectAndNameAsStringProxyArchive NodeProxyArchive(
		Ar, /* bInLoadIfFindFails=*/false);
	
	SerializeScriptProperties(NodeProxyArchive);
}

void UOptimusNode::EnableDynamicPins()
{
	bDynamicPins = true;
}


UOptimusNodePin* UOptimusNode::AddPin(
	FName InName,
	EOptimusNodePinDirection InDirection,
	const FOptimusDataDomain& InDataDomain,
	FOptimusDataTypeRef InDataType,
	UOptimusNodePin* InBeforePin,
	UOptimusNodePin* InGroupingPin
	)
{
	if (!bDynamicPins)
	{
		UE_LOGF(LogOptimusCore, Error, "Attempting to add a pin to a non-dynamic node: %ls", *GetNodePath());
		return nullptr;
	}

	if (InBeforePin)
	{
		if (InBeforePin->GetOwningNode() != this)
		{
			UE_LOGF(LogOptimusCore, Error, "Attempting to place a pin before one that does not belong to this node: %ls", *InBeforePin->GetPinPath());
			return nullptr;
		}
		if (InBeforePin->GetParentPin() != nullptr || (InBeforePin->GetParentPin() && !InBeforePin->GetParentPin()->IsGroupingPin()))
		{
			UE_LOGF(LogOptimusCore, Error, "Attempting to place a pin before one that is not a top-level pin or inside a grouping pin: %ls", *InBeforePin->GetPinPath());
			return nullptr;
		}
	}
	if (InGroupingPin)
	{
		if (InGroupingPin->GetOwningNode() != this)
		{
			UE_LOGF(LogOptimusCore, Error, "Attempting to place a pin under a group pin that does not belong to this node: %ls", *InGroupingPin->GetPinPath());
			return nullptr;
		}

		if (InBeforePin && !InGroupingPin->GetSubPins().Contains(InBeforePin))
		{
			UE_LOGF(LogOptimusCore, Error, "Before pin is not a part of the given group: %ls", *InBeforePin->GetPinPath());
			return nullptr;
		}
	}

	FOptimusNodeAction_AddPin *AddPinAction = new FOptimusNodeAction_AddPin(
		this, InName, InDirection, InDataDomain, InDataType, InBeforePin, InGroupingPin); 
	if (!GetActionStack()->RunAction(AddPinAction))
	{
		return nullptr;
	}

	return AddPinAction->GetPin(GetActionStack()->GetGraphCollectionRoot());
}

UOptimusNodePin* UOptimusNode::AddPin(const FOptimusParameterBinding& InBinding, EOptimusNodePinDirection InDirection, UOptimusNodePin* InBeforePin)
{
	return AddPin(InBinding.Name, InDirection, InBinding.DataDomain, InBinding.DataType, InBeforePin);
}

UOptimusNodePin* UOptimusNode::AddPinDirect(
    FName InName,
    EOptimusNodePinDirection InDirection,
    const FOptimusDataDomain& InDataDomain,
    FOptimusDataTypeRef InDataType,
    UOptimusNodePin* InBeforePin,
    UOptimusNodePin* InParentPin
	)
{
	UObject* PinParent = InParentPin ? Cast<UObject>(InParentPin) : this;
	UOptimusNodePin* Pin = NewObject<UOptimusNodePin>(PinParent, InName);

	Pin->InitializeWithData(InDirection, InDataDomain, InDataType);

	// Add sub-pins, if the registered type is set to show them but only for value types.
	if (InDataDomain.IsSingleton() &&
		EnumHasAnyFlags(InDataType->TypeFlags, EOptimusDataTypeFlags::ShowElements))
	{
		if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InDataType->TypeObject))
		{
			CreatePinsFromStructLayout(Struct, Pin);
		}
	}
	
	InsertPinIntoHierarchy(Pin, InParentPin, InBeforePin);
	
	return Pin;
}


UOptimusNodePin* UOptimusNode::AddPinDirect(
	const FOptimusParameterBinding& InBinding,
	EOptimusNodePinDirection InDirection,
	UOptimusNodePin* InBeforePin
	)
{
	return AddPinDirect(InBinding.Name, InDirection, InBinding.DataDomain, InBinding.DataType, InBeforePin);
}


UOptimusNodePin* UOptimusNode::AddGroupingPin(
	FName InName,
	EOptimusNodePinDirection InDirection,
	UOptimusNodePin* InBeforePin
	)
{
	if (!bDynamicPins)
	{
		UE_LOGF(LogOptimusCore, Error, "Attempting to add a pin to a non-dynamic node: %ls", *GetNodePath());
		return nullptr;
	}

	if (InBeforePin)
	{
		if (InBeforePin->GetOwningNode() != this)
		{
			UE_LOGF(LogOptimusCore, Error, "Attempting to place a pin before one that does not belong to this node: %ls", *InBeforePin->GetPinPath());
			return nullptr;
		}
		if (InBeforePin->GetParentPin() != nullptr)
		{
			UE_LOGF(LogOptimusCore, Error, "Attempting to place a pin before one that is not a top-level pin: %ls", *InBeforePin->GetPinPath());
			return nullptr;
		}
	}
	
	FOptimusNodeAction_AddGroupingPin *AddPinAction = new FOptimusNodeAction_AddGroupingPin(this, InName, InDirection, InBeforePin); 
	if (!GetActionStack()->RunAction(AddPinAction))
	{
		return nullptr;
	}

	return AddPinAction->GetPin(GetActionStack()->GetGraphCollectionRoot());
}


UOptimusNodePin* UOptimusNode::AddGroupingPinDirect(
	FName InName,
	EOptimusNodePinDirection InDirection,
	UOptimusNodePin* InBeforePin
	)
{
	UOptimusNodePin* Pin = NewObject<UOptimusNodePin>(this, InName);

	Pin->InitializeWithGrouping(InDirection);

	InsertPinIntoHierarchy(Pin, nullptr, InBeforePin);
	
	return Pin;
}

void UOptimusNode::InsertPinIntoHierarchy(
	UOptimusNodePin* InNewPin, 
	UOptimusNodePin* InParentPin,
	UOptimusNodePin* InInsertBeforePin
	)
{
	if (InParentPin)
	{
		InParentPin->AddSubPin(InNewPin, InInsertBeforePin);
	}
	else
	{
		int32 Index = Pins.Num();
		if (InInsertBeforePin && ensure(Pins.IndexOfByKey(InInsertBeforePin) != INDEX_NONE))
		{
			Index = Pins.IndexOfByKey(InInsertBeforePin); 
		}
		Pins.Insert(InNewPin, Index);
	}

	if (CanNotify())
	{
		InNewPin->Notify(EOptimusGraphNotifyType::PinAdded);
	}
}


bool UOptimusNode::RemovePin(UOptimusNodePin* InPin)
{
	if (!bDynamicPins)
	{
		UE_LOGF(LogOptimusCore, Error, "Attempting to remove a pin from a non-dynamic node: %ls", *GetNodePath());
		return false;
	}

	if (InPin->GetParentPin() != nullptr && !InPin->GetParentPin()->IsGroupingPin())
	{
		UE_LOGF(LogOptimusCore, Error, "Attempting to remove a non-root or non-group pin: %ls", *InPin->GetPinPath());
		return false;
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction;
	Action->SetTitlef(TEXT("Remove Pin"));

	TArray<UOptimusNodePin*> PinsToRemove = InPin->GetSubPinsRecursively();
	PinsToRemove.Add(InPin);

	const UOptimusNodeGraph* Graph = GetOwningGraph();

	// Validate that there are no links to the pins we want to remove.
	for (const UOptimusNodePin* Pin: PinsToRemove)
	{
		for (const UOptimusNodeLink *Link: Graph->GetPinLinks(Pin))
		{
			Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Link);
		}
	}

	// We need to explicitly remove sub-pins of grouping pins, since the action relies
	// on sub-pin recreation to be done only for nested types.
	if (InPin->IsGroupingPin())
	{
		for (UOptimusNodePin* Pin: InPin->GetSubPins())
		{
			Action->AddSubAction<FOptimusNodeAction_RemovePin>(Pin);
		}
	}

	Action->AddSubAction<FOptimusNodeAction_RemovePin>(InPin);
	
	return GetActionStack()->RunAction(Action);
}


bool UOptimusNode::RemovePinDirect(UOptimusNodePin* InPin)
{
	TArray<UOptimusNodePin*> PinsToRemove = InPin->GetSubPinsRecursively();
	PinsToRemove.Add(InPin);

	// Reverse the list so that we start by deleting the leaf-most pins first.
	Algo::Reverse(PinsToRemove);

	const UOptimusNodeGraph* Graph = GetOwningGraph();

	// Validate that there are no links to the pins we want to remove.
	for (const UOptimusNodePin* Pin: PinsToRemove)
	{
		if (!Graph->GetConnectedPins(Pin).IsEmpty())
		{
			UE_LOGF(LogOptimusCore, Warning, "Attempting to remove a connected pin: %ls", *Pin->GetPinPath());
			return false;
		}
	}

	// We only notify on the root pin once we're no longer reachable.
	if (InPin->GetParentPin() == nullptr)
	{
		Pins.Remove(InPin); 
	}
	else
	{
		InPin->GetParentPin()->SubPins.Remove(InPin);
	}
	InPin->Notify(EOptimusGraphNotifyType::PinRemoved);
	
	for (UOptimusNodePin* Pin: PinsToRemove)
	{
		ExpandedPins.Remove(Pin->GetUniqueName());

		Optimus::RemoveObject(Pin);
	}

	CachedPinLookup.Reset();

	return true;
}


bool UOptimusNode::MovePin(
	UOptimusNodePin* InPinToMove,
	const UOptimusNodePin* InPinBefore
	)
{
	if (!InPinToMove || InPinToMove == InPinBefore)
	{
		return false;
	}
	if (!InPinBefore)
	{
		// Is no before target given and we're already the last pin?
		if (InPinToMove->GetParentPin() == nullptr)
		{
			if (GetPins().Last() == InPinToMove)
			{
				return false;
			}
		}
		else if (InPinToMove->GetParentPin())
		{
			if (InPinToMove->GetParentPin()->GetSubPins().Last() == InPinToMove)
			{
				return false;
			}
		}
	}
	if (InPinBefore && InPinBefore->GetParentPin() != InPinToMove->GetParentPin())
	{
		UE_LOGF(LogOptimusCore, Warning, "Attempting to move a pin before a non-sibling pin: %ls", *InPinToMove->GetPinPath());
		return false;
	}

	return GetActionStack()->RunAction<FOptimusNodeAction_MovePin>(InPinToMove, InPinBefore);
}


bool UOptimusNode::MovePinDirect(
	UOptimusNodePin* InPinToMove,
	const UOptimusNodePin* InPinBefore
	)
{
	decltype(Pins)* MovablePins;
	if (UOptimusNodePin* ParentPin = InPinToMove->GetParentPin())
	{
		MovablePins = &ParentPin->SubPins;
	}
	else
	{
		MovablePins = &Pins;
	}
	
	const int32 PinIndex = MovablePins->IndexOfByKey(InPinToMove);
	if (InPinBefore)
	{
		MovablePins->RemoveAt(PinIndex);
		
		const int32 BeforeIndex = MovablePins->IndexOfByKey(InPinBefore);
		MovablePins->Insert(InPinToMove, BeforeIndex);
	}
	else
	{
		MovablePins->RemoveAt(PinIndex);
		MovablePins->Add(InPinToMove);
	}

	InPinToMove->Notify(EOptimusGraphNotifyType::PinMoved);
	return true;
}

bool UOptimusNode::MovePinToGroupPinDirect(UOptimusNodePin* InPinToMove, UOptimusNodePin* InGroupPin)
{
	UOptimusNodePin* CurrentGroupPin = InPinToMove->GetParentPin();
	TArray<TObjectPtr<UOptimusNodePin>>& PinGroup = CurrentGroupPin ? CurrentGroupPin->SubPins : Pins;
	
	PinGroup.Remove(InPinToMove);

	Optimus::RenameObject(InPinToMove, nullptr, InGroupPin);
	
	InsertPinIntoHierarchy(InPinToMove, InGroupPin, nullptr);

	return true;
}


bool UOptimusNode::SetPinDataType
(
	UOptimusNodePin* InPin,
	FOptimusDataTypeRef InDataType
	)
{
	if (!InPin || InPin->GetDataType() == InDataType.Resolve())
	{
		return false;
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction;
	Action->SetTitlef(TEXT("Set Pin Type"));

	// Disconnect all the links because they _will_ become incompatible.
	for (UOptimusNodePin* ConnectedPin: InPin->GetConnectedPins())
	{
		if (InPin->GetDirection() == EOptimusNodePinDirection::Input)
		{
			Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(ConnectedPin, InPin);
		}
		else
		{
			Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(InPin, ConnectedPin);
		}
	}
	
	Action->AddSubAction<FOptimusNodeAction_SetPinType>(InPin, InDataType);
	
	return GetActionStack()->RunAction(Action);
}


bool UOptimusNode::SetPinDataTypeDirect(
	UOptimusNodePin* InPin, 
	FOptimusDataTypeRef InDataType
	)
{
	// We can currently only change pin types if they have no underlying property.
	if (ensure(InPin) && ensure(InDataType.IsValid()) && 
	    ensure(InPin->GetPropertyFromPin() == nullptr))
	{
		if (!InPin->SetDataType(InDataType))
		{
			return false;
		}

		// For value types, we want to show sub-pins.
		if (InPin->GetDataDomain().IsSingleton())
		{
			// Remove all sub-pins, if there were any.		
			TGuardValue<bool> SuppressNotifications(bSendNotifications, false);
			
			// If the type was already a sub-element type, remove the existing pins.
			InPin->ClearSubPins();
			
			// Add sub-pins, if the registered type is set to show them but only for value types.
			if (EnumHasAllFlags(InDataType->TypeFlags, EOptimusDataTypeFlags::ShowElements))
			{
				if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InDataType->TypeObject))
				{
					CreatePinsFromStructLayout(Struct, InPin);
				}
			}
		}
		
		if (CanNotify())
		{
			InPin->Notify(EOptimusGraphNotifyType::PinTypeChanged);
		}

		return true;
	}
	else
	{
		return false;
	}
}


bool UOptimusNode::SetPinName(
	UOptimusNodePin* InPin, 
	FName InNewName
	)
{
	if (!InPin || InPin->GetFName() == InNewName)
	{
		return false;
	}

	// FIXME: Namespace check?
	return GetActionStack()->RunAction<FOptimusNodeAction_SetPinName>(InPin, InNewName);
}


bool UOptimusNode::SetPinNameDirect(
	UOptimusNodePin* InPin, 
	FName InNewName
	)
{
	if (ensure(InPin) && InNewName != NAME_None)
	{
		const FName OldName = InPin->GetFName();
		const bool bIsExpanded = ExpandedPins.Contains(OldName);

		if (InPin->SetName(InNewName))
		{
			// Flush the lookup table
			CachedPinLookup.Reset();

			if (bIsExpanded)
			{
				ExpandedPins.Remove(OldName);
				ExpandedPins.Add(InNewName);
			}
			return true;
		}
	}

	// No success.
	return false;
}


bool UOptimusNode::SetPinDataDomain(
	UOptimusNodePin* InPin,
	const FOptimusDataDomain& InDataDomain
	)
{
	if (!InPin || InPin->GetDataDomain() == InDataDomain)
	{
		return false;
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction;
	Action->SetTitlef(TEXT("Set Pin Data Domain"));

	// If we're not an output pin, or if the new domain levels are non-empty, then
	// we need to disconnect all links, because they _will_ become incompatible.
	if (InPin->GetDirection() == EOptimusNodePinDirection::Input || !InDataDomain.IsSingleton())
	{
		// Make sure to disconnect all possible sub-pins.
		constexpr bool bIncludeThisPin = true;
		for (UOptimusNodePin* Pin: InPin->GetSubPinsRecursively(bIncludeThisPin))
		{
			for (UOptimusNodePin* ConnectedPin: Pin->GetConnectedPins())
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
				{
					if (!FOptimusDataDomain::AreCompatible(ConnectedPin->GetDataDomain(), InDataDomain, nullptr))
					{
						Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(ConnectedPin, Pin);
					}
				}
				else
				{
					if (!FOptimusDataDomain::AreCompatible(InDataDomain, ConnectedPin->GetDataDomain(), nullptr))
					{
						Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Pin, ConnectedPin);
					}
				}
			}
		}
	}
	
	Action->AddSubAction<FOptimusNodeAction_SetPinDataDomain>(InPin, InDataDomain);
	
	return GetActionStack()->RunAction(Action);
}


bool UOptimusNode::SetPinDataDomainDirect(
	UOptimusNodePin* InPin,
	const FOptimusDataDomain& InDataDomain
	)
{
	const bool bWasSingleton = InPin->GetDataDomain().IsSingleton();
	const bool bIsSingleton = InDataDomain.IsSingleton();
	InPin->DataDomain = InDataDomain;

	// If we switched to/from singleton values, then we need to clear sub-pins or create sub-pins for the singleton value.
	// TODO: Allow sub-pin controls on resources or just do away with sub-pins altogether?
	if (bWasSingleton != bIsSingleton)
	{
		// Remove all sub-pins, if there were any.		
		TGuardValue<bool> SuppressNotifications(bSendNotifications, false);

		// Remove all existing pins, because we may be going from a non-empty domain to an empty one. 
		InPin->ClearSubPins();

		if (bIsSingleton)
		{
			// Add sub-pins, if the registered type is set to show them but only for value types.
			if (EnumHasAllFlags(InPin->DataType->TypeFlags, EOptimusDataTypeFlags::ShowElements))
			{
				if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InPin->DataType->TypeObject))
				{
					CreatePinsFromStructLayout(Struct, InPin);
				}
			}
		}
	}

	if (CanNotify())
	{
		InPin->Notify(EOptimusGraphNotifyType::PinDataDomainChanged);
	}
	
	return true;
}


void UOptimusNode::SetPinExpanded(const UOptimusNodePin* InPin, bool bInExpanded)
{
	FName Name = InPin->GetUniqueName();
	if (bInExpanded)
	{
		if (!ExpandedPins.Contains(Name))
		{
			ExpandedPins.Add(Name);
			InPin->Notify(EOptimusGraphNotifyType::PinExpansionChanged);
		}
	}
	else
	{
		if (ExpandedPins.Contains(Name))
		{
			ExpandedPins.Remove(Name);
			InPin->Notify(EOptimusGraphNotifyType::PinExpansionChanged);
		}
	}
}


bool UOptimusNode::GetPinExpanded(const UOptimusNodePin* InPin) const
{
	return ExpandedPins.Contains(InPin->GetUniqueName());
}


void UOptimusNode::CreatePinsFromStructLayout(
	const UStruct* InStruct, 
	UOptimusNodePin* InParentPin
	)
{
	for (const FProperty* Property : TFieldRange<FProperty>(InStruct))
	{
		if (InParentPin)
		{
			// Sub-pins keep the same direction as the parent.
			CreatePinFromProperty(InParentPin->GetDirection(), Property, InParentPin);
		}
#if WITH_EDITOR
		else if (Property->HasMetaData(PropertyMeta::Input))
		{
			if (Property->HasMetaData(PropertyMeta::Output))
			{
				UE_LOGF(LogOptimusCore, Error, "Pin on %ls.%ls marked both input and output. Ignoring it as output.",
					*GetName(), *Property->GetName());
			}

			CreatePinFromProperty(EOptimusNodePinDirection::Input, Property, InParentPin);
		}
		else if (Property->HasMetaData(PropertyMeta::Output))
		{
			CreatePinFromProperty(EOptimusNodePinDirection::Output, Property, InParentPin);
		}
#endif
	}
}


UOptimusNodePin* UOptimusNode::CreatePinFromProperty(
    EOptimusNodePinDirection InDirection,
	const FProperty* InProperty,
	UOptimusNodePin* InParentPin
	)
{
	if (!ensure(InProperty))
	{
		return nullptr;
	}

	// Is this a legitimate type for pins?
	const FOptimusDataTypeRegistry& Registry = FOptimusDataTypeRegistry::Get();

	FOptimusDataTypeHandle DataType = Registry.FindType(*InProperty);

	if (!DataType.IsValid())
	{
		UE_LOGF(LogOptimusCore, Error, "No registered type found for pin '%ls'.", *InProperty->GetName());
		return nullptr;
	}

	return AddPinDirect(InProperty->GetFName(), InDirection, {}, DataType, nullptr, InParentPin);
}


UOptimusActionStack* UOptimusNode::GetActionStack() const
{
	UOptimusNodeGraph *Graph = GetOwningGraph();
	if (Graph == nullptr)
	{
		return nullptr;
	}
	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(Graph->GetCollectionRoot());
	if (!Deformer)
	{
		return nullptr;
	}

	return Deformer->GetActionStack();
}
