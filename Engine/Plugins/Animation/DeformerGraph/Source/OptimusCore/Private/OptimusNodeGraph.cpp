// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodeGraph.h"

#include "OptimusDeformer.h"
#include "OptimusNode.h"
#include "OptimusNodePair.h"
#include "OptimusNodeLink.h"
#include "OptimusNodePin.h"
#include "OptimusActionStack.h"
#include "OptimusCoreModule.h"
#include "OptimusHelpers.h"
#include "Actions/OptimusNodeGraphActions.h"
#include "Actions/OptimusNodeActions.h"
#include "Nodes/OptimusNode_GetResource.h"
#include "Nodes/OptimusNode_GetVariable.h"
#include "Nodes/OptimusNode_Resource.h"
#include "Nodes/OptimusNode_SetResource.h"

#include "Containers/Queue.h"
#include "Nodes/OptimusNode_ConstantValue.h"
#include "Nodes/OptimusNode_DataInterface.h"
#include "Nodes/OptimusNode_AnimAttributeDataInterface.h"

#include "Nodes/OptimusNode_ComputeKernelFunction.h"
#include "Templates/Function.h"
#include "UObject/Package.h"

#include <limits>

#include "IOptimusComponentBindingReceiver.h"
#include "IOptimusNodePairProvider.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusFunctionNodeGraphHeader.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusNode_Comment.h"
#include "Nodes/OptimusNode_CustomComputeKernel.h"
#include "Nodes/OptimusNode_FunctionReference.h"
#include "Nodes/OptimusNode_GraphTerminal.h"
#include "Nodes/OptimusNode_SubGraphReference.h"
#include "Nodes/OptimusNode_ComponentSource.h"

#include "DataInterfaces/OptimusDataInterfaceAnimAttribute.h"
#include "Nodes/OptimusNode_ComponentSource.h"
#include "Nodes/OptimusNode_LoopTerminal.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodeGraph)

#define LOCTEXT_NAMESPACE "OptimusNodeGraph"


const FName UOptimusNodeGraph::SetupGraphName("SetupGraph");
const FName UOptimusNodeGraph::UpdateGraphName("UpdateGraph");
const FName UOptimusNodeGraph::DefaultSubGraphName("SubGraph");
const FName UOptimusNodeGraph::DefaultSubGraphRefNodeName("SubGraphNode");

FString UOptimusNodeGraph::GetFunctionGraphCollectionPath(const FString& InFunctionName)
{
	// Function Graph is always at the top level
	return InFunctionName;
}

void UOptimusNodeGraph::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	UObject::PostDuplicate(DuplicateMode);

	// Check if the duplication took place at the asset level
	// if so, we have to recreate all constant nodes such that their class pointers
	// don't point to classes in the source asset. This can happen because generated class
	// in the source package/asset are not duplicated automatically to the new package/asset
	{
		TArray<UOptimusNode_ConstantValue*> ConstantNodesToRecreate;

		for (UOptimusNode* Node : Nodes)
		{
			if (UOptimusNode_ConstantValue* ConstantNode = Cast<UOptimusNode_ConstantValue>(Node))
			{
				if (ConstantNode->GetClass()->GetPackage() != GetPackage())
				{
					ConstantNodesToRecreate.Add(ConstantNode);
				}
			}
		}

		for (UOptimusNode_ConstantValue* ConstantNode : ConstantNodesToRecreate)
		{
			UOptimusNode_ConstantValueGeneratorClass* CurrentNodeClass =
				Cast<UOptimusNode_ConstantValueGeneratorClass>(ConstantNode->GetClass());

			// Create the equivalent node class in current package
			UClass* NewNodeClass =
				UOptimusNode_ConstantValueGeneratorClass::GetClassForType(GetPackage(), CurrentNodeClass->DataType);
			
			// Save node data
			TArray<uint8> NodeData;
			{
				FMemoryWriter NodeArchive(NodeData);
				FObjectAndNameAsStringProxyArchive NodeProxyArchive(
						NodeArchive, /* bInLoadIfFindFails=*/ false);
				ConstantNode->SerializeScriptProperties(NodeProxyArchive);
			}

			// Save pin connections
			TArrayView<UOptimusNodePin* const> Pins = ConstantNode->GetPins();

			TMap<FName, TArray<UOptimusNodePin*>> ConnectedPinsMap;
			
			for (UOptimusNodePin* Pin : Pins)
			{
				ConnectedPinsMap.Add(Pin->GetFName(), Pin->GetConnectedPins());
			}

			FName NodeName = ConstantNode->GetFName();
			
			RemoveNodeDirect(ConstantNode, false /* remove all links as well */);

			auto BootstrapNodeFunc = [NodeData](UOptimusNode* InNode) -> bool
			{
				{
					FMemoryReader NodeArchive(NodeData);
					FObjectAndNameAsStringProxyArchive NodeProxyArchive(
							NodeArchive, /* bInLoadIfFindFails=*/true);
					InNode->SerializeScriptProperties(NodeProxyArchive);
				}
				
				return true;
			};
			
			UOptimusNode* NewNode = CreateNodeDirect(NewNodeClass, NodeName, BootstrapNodeFunc);

			// Recover links
			for (UOptimusNodePin* Pin : NewNode->GetPins())
			{
				if (TArray<UOptimusNodePin*>* ConnectedPins = ConnectedPinsMap.Find(Pin->GetFName()))
				{
					for (UOptimusNodePin* ConnectedPin : *ConnectedPins)
					{
						if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
						{
							AddLinkDirect(ConnectedPin, Pin);
						}
						else if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
						{
							AddLinkDirect(Pin, ConnectedPin);
						}
					}
				}
			}
		}	
	}
}

void UOptimusNodeGraph::PostLoad()
{
	Super::PostLoad();

	for (UOptimusNodeGraph* SubGraph: SubGraphs)
	{
		SubGraph->ConditionalPostLoad();
	}
	
	for (UOptimusNode* Node : Nodes)
	{
		Node->ConditionalPostLoad();
	}

	for (UOptimusNodeLink* Link : Links)
	{
		Link->ConditionalPostLoad();
	}

	PostLoadReplaceAnimAttributeDataInterfaceNodeWithGenericDataInterfaceNode();
}

UOptimusNodeGraph* UOptimusNodeGraph::GetParentGraph() const
{
	return Cast<UOptimusNodeGraph>(GetOuter());
}


bool UOptimusNodeGraph::IsValidUserGraphName(
	const FString& InGraphName,
	FText* OutFailureReason
	)
{
	// Reserved names are reserved.
	if (InGraphName.Equals(SetupGraphName.ToString(), ESearchCase::IgnoreCase) ||
		InGraphName.Equals(UpdateGraphName.ToString(), ESearchCase::IgnoreCase))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FText::Format(LOCTEXT("ReservedName", "'{}' is a reserved name."), FText::FromString(InGraphName));
		}
		return false;
	}

	// '.' and '/' are path separateors. '$' is used for special graphs (e.g. libraries).
	return FName::IsValidXName(InGraphName, TEXT("./$"), OutFailureReason);
}

FString UOptimusNodeGraph::ConstructPath(const FString& GraphPath, const FString& NodeName, const FString& PinPath)
{
	if (!ensure(!NodeName.IsEmpty()))
	{
		return {};
	}
	
	FString Result = NodeName;
	if (!GraphPath.IsEmpty())
	{
		Result = GraphPath + TEXT("/") + Result;
	}

	if (!PinPath.IsEmpty())
	{
		Result = Result + TEXT(".") + PinPath;
	}

	return Result;
}

bool UOptimusNodeGraph::IsReadOnly() const
{
	if (IsFunctionGraph())
	{
		return true;
	}

	if (UOptimusNodeGraph* GraphOwner = Cast<UOptimusNodeGraph>(GetCollectionOwner()))
	{
		return GraphOwner->IsReadOnly();
	}

	return false;
}

int32 UOptimusNodeGraph::GetGraphIndex() const
{
	return GetCollectionOwner()->GetGraphs().IndexOfByKey(this);
}


FOptimusGraphNotifyDelegate& UOptimusNodeGraph::GetNotifyDelegate()
{
	return GraphNotifyDelegate;
}


IOptimusPathResolver* UOptimusNodeGraph::GetPathResolver() const
{
	return Cast<IOptimusPathResolver>(GetCollectionRoot());
}


UOptimusNode* UOptimusNodeGraph::AddNodeInternal(
	const TSubclassOf<UOptimusNode> InNodeClass,
	const FVector2D& InPosition,
	TFunction<void(UOptimusNode*)> InNodeConfigFunc
	)
{
	// FIXME: Need better naming.
	FName NodeName = Optimus::GetUniqueNameForScope(this, InNodeClass->GetFName());
	FOptimusNodeGraphAction_AddNode *AddNodeAction = new FOptimusNodeGraphAction_AddNode(
		GetCollectionPath(), InNodeClass, NodeName,
		[InNodeConfigFunc, InPosition](UOptimusNode *InNode) {
			if (InNodeConfigFunc)
			{
				InNodeConfigFunc(InNode);
			}
			return InNode->SetGraphPositionDirect(InPosition); 
		});
	if (!GetActionStack()->RunAction(AddNodeAction))
	{
		return nullptr;
	}

	return AddNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot());
}

TArray<UOptimusNode*> UOptimusNodeGraph::AddNodePairInternal(const TSubclassOf<UOptimusNode> InNodeClass,
	const FVector2D& InPosition, TFunction<void(UOptimusNode*)> InFirstNodeConfigFunc, TFunction<void(UOptimusNode*)> InSecondNodeConfigFunc)
{
	FName NodeName = Optimus::GetUniqueNameForScope(this, *(InNodeClass->GetName() + TEXT("_Entry")));

	FOptimusCompoundAction *Action = new FOptimusCompoundAction(TEXT("Add Loop Nodes"));
	
	FOptimusNodeGraphAction_AddNode *AddFirstNodeAction = new FOptimusNodeGraphAction_AddNode(
		GetCollectionPath(), InNodeClass, NodeName,
		[InFirstNodeConfigFunc, InPosition](UOptimusNode *InNode) {
			if (InFirstNodeConfigFunc)
			{
				InFirstNodeConfigFunc(InNode);
			}
			return InNode->SetGraphPositionDirect(InPosition); 
		});

	Action->AddSubAction(AddFirstNodeAction);
	
	NodeName = Optimus::GetUniqueNameForScope(this, *(InNodeClass->GetName() + TEXT("_Exit")));
	
	FOptimusNodeGraphAction_AddNode *AddSecondNodeAction = new FOptimusNodeGraphAction_AddNode(
			GetCollectionPath(), InNodeClass, NodeName,
			[InSecondNodeConfigFunc, InPosition](UOptimusNode *InNode) {
				if (InSecondNodeConfigFunc)
				{
					InSecondNodeConfigFunc(InNode);
				}
				return InNode->SetGraphPositionDirect(InPosition + FVector2D(100, 0)); 
			});

	Action->AddSubAction(AddSecondNodeAction);
	
	if (!GetActionStack()->RunAction(Action))
	{
		return {};
	}

	TArray<UOptimusNode*> AddedNodes;

	AddedNodes.Add(AddFirstNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot()));
	AddedNodes.Add(AddSecondNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot()));
	
	return AddedNodes;
}

void UOptimusNodeGraph::GatherRelatedObjects(
	const TArray<UOptimusNode*>& InNodes,
	TArray<UOptimusNode*>& OutNodes,
	TArray<UOptimusNodePair*>& OutNodePairs,
	TArray<UOptimusNodeSubGraph*>& OutSubGraphs
	)
{
	OutNodes = InNodes;
	for (UOptimusNode* Node : InNodes)
	{
		if (Cast<IOptimusNodePairProvider>(Node))
		{
			if (UOptimusNodePair* NodePair = Node->GetOwningGraph()->GetNodePair(Node); ensure(NodePair))
			{
				OutNodes.AddUnique(NodePair->GetNodeCounterpart(Node));
				OutNodePairs.AddUnique(NodePair);
			}
		}

		if (UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(Node))
		{
			OutSubGraphs.AddUnique(SubGraphNode->GetReferencedSubGraph());
		}
	}	
}

bool UOptimusNodeGraph::DuplicateSubGraph(
	UOptimusActionStack* InActionStack,
	const FString& InGraphOwnerPath,
	UOptimusNodeSubGraph* InSourceSubGraph,
	FName InNewGraphName
	)
{
	
	// Ensure the name is unique here instead of relying on AddGraph action
	// because subsequent actions needs to use the final graph path
	// Note: we could change the actions to accept a lambda instead of specific value in the future
	FName SubGraphName = InNewGraphName;
	
	const FString SubGraphPath = ConstructSubGraphPath(InGraphOwnerPath, SubGraphName.ToString());
	UOptimusNode_GraphTerminal* SourceSubGraphEntryNode = InSourceSubGraph->GetTerminalNode(EOptimusTerminalType::Entry);
	UOptimusNode_GraphTerminal* SourceSubGraphReturnNode = InSourceSubGraph->GetTerminalNode(EOptimusTerminalType::Return);
	const FVector2D EntryNodePosition = SourceSubGraphEntryNode->GetGraphPosition();
	const FVector2D ReturnNodePosition = SourceSubGraphReturnNode->GetGraphPosition();

	TMap<FString, TArray<FString>> InternalLinksToAdd;
	
	for (UOptimusNodeLink* Link : InSourceSubGraph->GetAllLinks())
	{
		if (Link->GetNodeInputPin()->GetOwningNode() == SourceSubGraphReturnNode)
		{
			FString ReturnNodeInputPinPath = ConstructPath(
				SubGraphPath,
				UOptimusNode_GraphTerminal::ReturnNodeName.ToString(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeOutputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			InternalLinksToAdd.FindOrAdd(InternalNodeOutputPinPath).Add(ReturnNodeInputPinPath);
		}
		else if (Link->GetNodeOutputPin()->GetOwningNode() == SourceSubGraphEntryNode)
		{
			FString EntryNodeOutputPinPath = ConstructPath(
				SubGraphPath,
				UOptimusNode_GraphTerminal::EntryNodeName.ToString(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeInputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			InternalLinksToAdd.FindOrAdd(EntryNodeOutputPinPath).Add(InternalNodeInputPinPath);
		}
		else
		{
			FString OutputPinPath = ConstructPath(
					SubGraphPath,
					Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
					Link->GetNodeOutputPin()->GetUniqueName().ToString()
					);

			FString InputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			
			InternalLinksToAdd.FindOrAdd(OutputPinPath).Add(InputPinPath);
		}
	}	
	
	
	TArray<UOptimusNode*> SubGraphNodes = InSourceSubGraph->GetAllNodes();
	
	SubGraphNodes.RemoveAll([](UOptimusNode* InNode)
	{
		return Cast<UOptimusNode_GraphTerminal>(InNode) != nullptr;
	});

	TArray<UOptimusNode*> NodesToDuplicate;
	TArray<UOptimusNodePair*> NodePairsToDuplicate;
	TArray<UOptimusNodeSubGraph*> SubGraphsToDuplicate;
	GatherRelatedObjects(SubGraphNodes, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);

	TArray<TPair<FString, FString>> NewNodePairs;
	for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
	{
		// Generate new pair
		const UOptimusNode *FirstNode = NodePair->GetFirst();
		const UOptimusNode *SecondNode = NodePair->GetSecond();

		FString NewFirstNodePath = ConstructPath(SubGraphPath, FirstNode->GetName(), {});
		FString NewSecondNodePath = ConstructPath(SubGraphPath, SecondNode->GetName(), {});

		NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
	}

	{
		FOptimusActionScope(*InActionStack, TEXT("Duplicate SubGraph"));
		InActionStack->RunAction<FOptimusNodeGraphAction_AddGraph>(InGraphOwnerPath, EOptimusNodeGraphType::SubGraph, SubGraphName, INDEX_NONE,
			[InputBindings = InSourceSubGraph->InputBindings, OutputBindings = InSourceSubGraph->OutputBindings](UOptimusNodeGraph* InGraph)
			{
				UOptimusNodeSubGraph* SubGraph = CastChecked<UOptimusNodeSubGraph>(InGraph);
				SubGraph->InputBindings = InputBindings;
				SubGraph->OutputBindings = OutputBindings;
				return true;
			});
	
		InActionStack->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::EntryNodeName,
			[EntryNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* EntryNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				EntryNode->TerminalType = EOptimusTerminalType::Entry;
				return EntryNode->SetGraphPositionDirect(EntryNodePosition);
			});

		InActionStack->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::ReturnNodeName,
			[ReturnNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* ReturnNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				ReturnNode->TerminalType = EOptimusTerminalType::Return;
				return ReturnNode->SetGraphPositionDirect(ReturnNodePosition);
			});

		for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
		{
			DuplicateSubGraph(InActionStack, SubGraphPath, SubGraphToDuplicate, SubGraphToDuplicate->GetFName());
		}
		
		// Copy the nodes to the sub graph
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			InActionStack->RunAction<FOptimusNodeGraphAction_DuplicateNode>(
				SubGraphPath, Node, Node->GetFName(), [](UOptimusNode*)
				{
					return true;
				});
		}

		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			InActionStack->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}	
		
		// Restore links
		for (const TPair<FString, TArray<FString>>& InternalLinks : InternalLinksToAdd )
		{
			const FString& OutputPinPath = InternalLinks.Key;
			for (const FString& InputPinPath : InternalLinks.Value)
			{
				InActionStack->RunAction<FOptimusNodeGraphAction_AddLink>(OutputPinPath, InputPinPath);
			}
		}
	}
	
	return true;
}


UOptimusNode* UOptimusNodeGraph::AddNode(
	const TSubclassOf<UOptimusNode> InNodeClass, 
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(InNodeClass, InPosition, /*InNodeConfigFunc*/{});
}


UOptimusNode* UOptimusNodeGraph::AddValueNode(
	FOptimusDataTypeRef InDataTypeRef,
	const FVector2D& InPosition
	)
{
	UClass *ValueNodeClass = UOptimusNode_ConstantValueGeneratorClass::GetClassForType(GetPackage(), InDataTypeRef);
	return AddNodeInternal(ValueNodeClass, InPosition, /*InNodeConfigFunc*/{});
}


UOptimusNode* UOptimusNodeGraph::AddDataInterfaceNode(
	const TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass,
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(UOptimusNode_DataInterface::StaticClass(), InPosition,
		[InDataInterfaceClass](UOptimusNode *InNode)
		{
			Cast<UOptimusNode_DataInterface>(InNode)->SetDataInterfaceClass(InDataInterfaceClass);			
		});
}

TArray<UOptimusNode*> UOptimusNodeGraph::AddLoopTerminalNodes(const FVector2D& InPosition)
{
	FOptimusCompoundAction *Action = new FOptimusCompoundAction(TEXT("Add Loop Nodes"));

	FName NodeName = Optimus::GetUniqueNameForScope(this, *(UOptimusNode_LoopTerminal::StaticClass()->GetName() + TEXT("_Entry")));
	FOptimusNodeGraphAction_AddNode *AddEntryNodeAction = new FOptimusNodeGraphAction_AddNode(
		GetCollectionPath(), UOptimusNode_LoopTerminal::StaticClass(), NodeName,
		[InPosition](UOptimusNode *InNode) {
			UOptimusNode_LoopTerminal* Node = CastChecked<UOptimusNode_LoopTerminal>(InNode);
			Node->TerminalType = EOptimusTerminalType::Entry;
			return InNode->SetGraphPositionDirect(InPosition); 
		});

	TWeakPtr<FOptimusNodeGraphAction_AddNode> WeakAddEntryNodeAction =
		StaticCastWeakPtr<FOptimusNodeGraphAction_AddNode>(Action->AddSubAction(AddEntryNodeAction));

	NodeName = Optimus::GetUniqueNameForScope(this, *(UOptimusNode_LoopTerminal::StaticClass()->GetName() + TEXT("_Return"))); 

	FOptimusNodeGraphAction_AddNode *AddReturnNodeAction = new FOptimusNodeGraphAction_AddNode(
		GetCollectionPath(), UOptimusNode_LoopTerminal::StaticClass(), NodeName,
		[InPosition](UOptimusNode *InNode) {
			UOptimusNode_LoopTerminal* Return = CastChecked<UOptimusNode_LoopTerminal>(InNode);
			Return->TerminalType = EOptimusTerminalType::Return;
			return InNode->SetGraphPositionDirect(InPosition + FVector2d(300, 0)); 
		});

	TWeakPtr<FOptimusNodeGraphAction_AddNode> WeakAddReturnNodeAction =
	StaticCastWeakPtr<FOptimusNodeGraphAction_AddNode>(Action->AddSubAction(AddReturnNodeAction));

	Action->AddSubAction<FOptimusNodeGraphAction_AddNodePair>(WeakAddEntryNodeAction, WeakAddReturnNodeAction);

	if (!GetActionStack()->RunAction(Action))
	{
		return {};
	}

	TArray<UOptimusNode*> AddedNodes;

	AddedNodes.Add(AddEntryNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot()));
	AddedNodes.Add(AddReturnNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot()));
	
	return AddedNodes;
}

UOptimusNode* UOptimusNodeGraph::AddCommentNode(const FVector2D& InPosition, const FVector2D& InSize,const FLinearColor& InColor)
{
	return AddCommentNode(InPosition, InSize, InColor, false);
	
}

UOptimusNode* UOptimusNodeGraph::AddCommentNode(const FVector2D& InPosition, const FVector2D& InSize, const FLinearColor& InColor,
	bool bCreatedFromUI)
{
	return AddNodeInternal(UOptimusNode_Comment::StaticClass(), InPosition,
		[InColor, InSize, bCreatedFromUI](UOptimusNode *InNode)
		{
			UOptimusNode_Comment* CommentNode = CastChecked<UOptimusNode_Comment>(InNode);
			CommentNode->SetSizeDirect(UE::Slate::FDeprecateVector2DParameter(InSize));
			CommentNode->CommentColor = InColor;
			CommentNode->FontSize = 18;
			CommentNode->Comment = TEXT("Comment");
#if WITH_EDITOR
			CommentNode->bCreatedFromUI = bCreatedFromUI;
#endif
		});	
}

UOptimusNode* UOptimusNodeGraph::AddFunctionReferenceNode(UOptimusFunctionNodeGraph* InFunctionGraph, const FVector2D& InPosition)
{
	return AddNodeInternal(UOptimusNode_FunctionReference::StaticClass(), InPosition,
	[InFunctionGraph](UOptimusNode *InNode)
	{
		UOptimusNode_FunctionReference* FunctionNode = CastChecked<UOptimusNode_FunctionReference>(InNode);
		FunctionNode->SetReferencedFunctionGraph(InFunctionGraph->GetGraphIdentifier());
	});
}

UOptimusNode* UOptimusNodeGraph::AddResourceNode(
	UOptimusResourceDescription* InResourceDesc,
	const FVector2D& InPosition
)
{
	return AddNodeInternal(UOptimusNode_Resource::StaticClass(), InPosition,
		[InResourceDesc](UOptimusNode* InNode)
		{
			Cast<UOptimusNode_Resource>(InNode)->SetResourceDescription(InResourceDesc);
		});
}


UOptimusNode* UOptimusNodeGraph::AddResourceGetNode(
	UOptimusResourceDescription* InResourceDesc, 
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(UOptimusNode_GetResource::StaticClass(), InPosition,
		[InResourceDesc](UOptimusNode *InNode)
		{
			Cast<UOptimusNode_GetResource>(InNode)->SetResourceDescription(InResourceDesc);			
		});
}


UOptimusNode* UOptimusNodeGraph::AddResourceSetNode(
	UOptimusResourceDescription* InResourceDesc, 
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(UOptimusNode_SetResource::StaticClass(), InPosition,
		[InResourceDesc](UOptimusNode *InNode)
		{
			Cast<UOptimusNode_SetResource>(InNode)->SetResourceDescription(InResourceDesc);			
		});
}


UOptimusNode* UOptimusNodeGraph::AddVariableGetNode(
	UOptimusVariableDescription* InVariableDesc, 
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(UOptimusNode_GetVariable::StaticClass(), InPosition,
		[InVariableDesc](UOptimusNode *InNode)
		{
			Cast<UOptimusNode_GetVariable>(InNode)->SetVariableDescription(InVariableDesc);			
		});
}


UOptimusNode* UOptimusNodeGraph::AddComponentBindingGetNode(
	UOptimusComponentSourceBinding* InComponentBinding,
	const FVector2D& InPosition
	)
{
	return AddNodeInternal(UOptimusNode_ComponentSource::StaticClass(), InPosition,
		[InComponentBinding](UOptimusNode *InNode)
		{
			Cast<UOptimusNode_ComponentSource>(InNode)->SetComponentSourceBinding(InComponentBinding);
		});
}


bool UOptimusNodeGraph::RemoveNode(UOptimusNode* InNode)
{
	if (!InNode)
	{
		return false;
	}

	return RemoveNodes({InNode});
}


bool UOptimusNodeGraph::RemoveNodes(const TArray<UOptimusNode*> &InNodes)
{
	return RemoveNodesAndCount(InNodes) > 0;
}


int32 UOptimusNodeGraph::RemoveNodesAndCount(const TArray<UOptimusNode*> &InNodes)
{
	// Validate the input set.
	if (InNodes.Num() == 0)
	{
		return 0;
	}

	TArray<UOptimusNode*> NodesToRemove;
	TArray<UOptimusNodePair*> NodePairsToRemove;
	TArray<UOptimusNodeSubGraph*> SubGraphsToRemove;
	GatherRelatedObjects(InNodes, NodesToRemove, NodePairsToRemove, SubGraphsToRemove);
	
	for (UOptimusNode* Node : NodesToRemove)
	{
		if (Node == nullptr || Node->GetOwningGraph() != this)
		{
			return 0;
		}
	}

	TSet<int32> AllLinkIndexes;

	// Get all unique links for all the given nodes and remove them *before* we remove the nodes.
	for (const UOptimusNode* Node : NodesToRemove)
	{
		AllLinkIndexes.Append(GetAllLinkIndexesToNode(Node));
	}

	const FString ActionTitle = NodesToRemove.Num() == 1 ? TEXT("Remove Node") : FString::Printf(TEXT("Remove %d Nodes"), NodesToRemove.Num()) ;
	
	FOptimusActionScope ActionScope(*GetActionStack(), ActionTitle);	
	
	for (const int32 LinkIndex : AllLinkIndexes)
	{
		GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveLink>(Links[LinkIndex]);
	}

	for (const UOptimusNodePair* NodePair: NodePairsToRemove)
	{
		GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveNodePair>(NodePair);
	}	
	
	for (UOptimusNode* Node : NodesToRemove)
	{
		GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveNode>(Node);
	}

	for (UOptimusNodeSubGraph* SubGraph : SubGraphsToRemove)
	{
		RemoveGraph(SubGraph);
	}

	return NodesToRemove.Num();
}

void UOptimusNodeGraph::RemoveGraph(UOptimusNodeGraph* InNodeGraph)
{
	if (!InNodeGraph)
	{
		return;
	}

	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Remove Graph"));

	InNodeGraph->RemoveNodes(InNodeGraph->GetAllNodes());

	GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveGraph>(InNodeGraph);
}

UOptimusNode* UOptimusNodeGraph::DuplicateNode(
	UOptimusNode* InNode,
	const FVector2D& InPosition
	)
{
	if (!InNode)
	{
		return nullptr;
	}

	// This API does not support duplicating loop terminals, or subgraph nodes currently
	// Use DuplicateNodes() below
	if (!ensure(!Cast<UOptimusNode_LoopTerminal>(InNode)))
	{
		return nullptr;
	}
	
	const FName NodeName = Optimus::GetUniqueNameForScope(this, InNode->GetFName());
	
	FOptimusNodeGraphAction_DuplicateNode *DuplicateNodeAction = new FOptimusNodeGraphAction_DuplicateNode(
		GetCollectionPath(), InNode, NodeName,
		[InPosition](UOptimusNode *InNode) {
			return InNode->SetGraphPositionDirect(InPosition); 
		});
	if (!GetActionStack()->RunAction(DuplicateNodeAction))
	{
		return nullptr;
	}

	return DuplicateNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot());
}


bool UOptimusNodeGraph::DuplicateNodes(
	const TArray<UOptimusNode*> &InNodes,
	const FVector2D& InPosition
	)
{
	return DuplicateNodes(InNodes, InPosition, TEXT("Duplicate"));
}

bool UOptimusNodeGraph::DuplicateNodes(
	const TArray<UOptimusNode*>& InNodes,
	const FVector2D& InPosition,
	const FString& InActionName
	)
{
	// Make sure all the nodes come from the same graph.
	UOptimusNodeGraph* SourceGraph = nullptr;
	for (const UOptimusNode* Node: InNodes)
	{
		if (SourceGraph == nullptr)
		{
			SourceGraph = Node->GetOwningGraph();
		}
		else if (SourceGraph != Node->GetOwningGraph())
		{
			UE_LOGF(LogOptimusCore, Warning, "Nodes to duplicate have to all belong to the same graph.");
			return false;
		}
	}
	

	if (!ensure(SourceGraph != nullptr))
	{
		return false;
	}


	TArray<UOptimusNode*> NodesToDuplicate;
	TArray<UOptimusNodePair*> NodePairsToDuplicate;
	TArray<UOptimusNodeSubGraph*> SubGraphsToDuplicate;
	GatherRelatedObjects(InNodes, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);

	// Figure out the non-clashing names to use, to avoid collisions during actual execution.
	Optimus::FUniqueNameGenerator UniqueNameGenerator(this);

	using FloatType = decltype(FVector2D::X);
	FVector2D TopLeft{std::numeric_limits<FloatType>::max()};
	TMap<UOptimusNode*, FName> NewNodeNameMap;
	for (UOptimusNode* Node: NodesToDuplicate)
	{
		TopLeft = FVector2D::Min(TopLeft, Node->GraphPosition);
		NewNodeNameMap.Add(Node, UniqueNameGenerator.GetUniqueName(Node->GetFName()));
	}
	FVector2D NodeOffset = InPosition - TopLeft;

	const FString GraphPath = GetCollectionPath();
	
	TArray<TPair<FString, FString>> NewNodePairs;
	for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
	{
		// Genernerate new pair
		const UOptimusNode *FirstNode = NodePair->GetFirst();
		const UOptimusNode *SecondNode = NodePair->GetSecond();

		if (ensure(NewNodeNameMap.Contains(FirstNode) && NewNodeNameMap.Contains(SecondNode)))
		{
			FString NewFirstNodePath = ConstructPath(GraphPath, NewNodeNameMap[FirstNode].ToString(), {});
			FString NewSecondNodePath = ConstructPath(GraphPath, NewNodeNameMap[SecondNode].ToString(), {});

			NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
		}	
	}

	// Using GraphName instead of the graph object as key here because this map is to be captured by a lambda to be used during undo/redo 
	TMap<FName , FName> NewSubGraphNameMap;
	for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
	{
		NewSubGraphNameMap.Add(SubGraphToDuplicate->GetFName(), UniqueNameGenerator.GetUniqueName(SubGraphToDuplicate->GetFName()));
	}	
	
	/// Collect the links between these existing nodes. 
	TArray<TPair<FString, FString>> NodeLinks;
	
	for (const UOptimusNodeLink* Link: SourceGraph->GetAllLinks())
	{
		const UOptimusNode *OutputNode = Link->GetNodeOutputPin()->GetOwningNode();
		const UOptimusNode *InputNode = Link->GetNodeInputPin()->GetOwningNode();

		if (NewNodeNameMap.Contains(OutputNode) && NewNodeNameMap.Contains(InputNode))
		{
			// FIXME: This should be a utility function, along with all the other path creation
			// functions.
			FString NodeOutputPinPath = ConstructPath(GraphPath, NewNodeNameMap[OutputNode].ToString(), Link->GetNodeOutputPin()->GetUniqueName().ToString());
			FString NodeInputPinPath = ConstructPath(GraphPath, NewNodeNameMap[InputNode].ToString(), Link->GetNodeInputPin()->GetUniqueName().ToString());
			
			NodeLinks.Add(MakeTuple(NodeOutputPinPath, NodeInputPinPath));
		}
	}

	FString ActionTitle; 
	if (NodesToDuplicate.Num() == 1)
	{
		ActionTitle = FString::Printf(TEXT("%s Node"), *InActionName);
	}
	else
	{
		ActionTitle = FString::Printf(TEXT("%s %d Nodes"), *InActionName, NodesToDuplicate.Num());
	}
	
	{
		FOptimusActionScope ActionScope(*GetActionStack(), ActionTitle);
		
		for (UOptimusNodeSubGraph* SubGraphToDuplicate: SubGraphsToDuplicate)
		{
			DuplicateSubGraph(GetActionStack(), GetCollectionPath(), SubGraphToDuplicate, NewSubGraphNameMap[SubGraphToDuplicate->GetFName()]);
		}

		{
			FOptimusCompoundAction *PreDuplicateRequirementAction = new FOptimusCompoundAction;	
			// Add all the pre-duplicate requirement actions first. This allows certain nodes to set up their
			// operating environment correctly (e.g. a missing variable description for get variable nodes, etc.)
			for (UOptimusNode* Node: NodesToDuplicate)
			{
				Node->PreDuplicateRequirementActions(this, PreDuplicateRequirementAction);
			}
			GetActionStack()->RunAction(PreDuplicateRequirementAction);
		}
		
		// Duplicate the nodes and place them correctly
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			FOptimusNodeGraphAction_DuplicateNode *DuplicateNodeAction = new FOptimusNodeGraphAction_DuplicateNode(
				GetCollectionPath(), Node, NewNodeNameMap[Node],
				[SourceNodeGraphPosition = Node->GraphPosition, NodeOffset, NewSubGraphNameMap](UOptimusNode *InNode)
				{
					if (UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(InNode))
					{
						FName NewSubGraphName = NewSubGraphNameMap[SubGraphNode->GetSerializedSubGraphName()];
						SubGraphNode->InitializeSerializedSubGraphName(NewSubGraphName);
					}
					return InNode->SetGraphPositionDirect(SourceNodeGraphPosition + NodeOffset); 
				});
			
			GetActionStack()->RunAction(DuplicateNodeAction);
		}

		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}
		
		// Add any links that the nodes may have had. These operations are allowed to fail, in which case
		// we just end up with dangling nodes.
		// In the future we would like to introduce connections that are in an error state (will fail compile)
		// but show up as disconnectable wires in the graph (e.g. if a type no longer matches).
		for (const TTuple<FString, FString>& LinkInfo: NodeLinks)
		{
			constexpr bool bCanFail = true;
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(LinkInfo.Key, LinkInfo.Value, bCanFail);
		}
	}

	return true;
}


bool UOptimusNodeGraph::AddLink(UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin)
{
	if (!InNodeOutputPin || !InNodeInputPin)
	{
		return false;
	}

	if (!InNodeOutputPin->CanCannect(InNodeInputPin))
	{
		// FIXME: We should be able to report back the failure reason.
		return false;
	}

	// Swap them if they're the wrong order -- a genuine oversight.
	if (InNodeOutputPin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		Swap(InNodeOutputPin, InNodeInputPin);
	}

	// Check to see if there's an existing link on the _input_ pin. Output pins can have any
	// number of connections coming out.
	TArray<int32> PinLinks = GetAllLinkIndexesToPin(InNodeInputPin);

	// This shouldn't happen, but we'll cover for it anyway.
	checkSlow(PinLinks.Num() <= 1);

	FOptimusCompoundAction* Action = new FOptimusCompoundAction;
		
	for (int32 LinkIndex : PinLinks)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Links[LinkIndex]);
	}

	FOptimusNodeGraphAction_AddLink  *AddLinkAction = new FOptimusNodeGraphAction_AddLink(InNodeOutputPin, InNodeInputPin);

	Action->SetTitle(AddLinkAction->GetTitle());
	Action->AddSubAction(AddLinkAction);

	return GetActionStack()->RunAction(Action);
}


bool UOptimusNodeGraph::RemoveLink(UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin)
{
	if (!InNodeOutputPin || !InNodeInputPin)
	{
		return false;
	}
	
	// Passing in pins of the same direction is a blatant fail.
	if (!ensure(InNodeOutputPin->GetDirection() != InNodeInputPin->GetDirection()))
	{
		return false;
	}

	// Swap them if they're the wrong order -- a genuine oversight.
	if (InNodeOutputPin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		Swap(InNodeOutputPin, InNodeInputPin);
	}

	for (UOptimusNodeLink* Link: Links)
	{
		if (Link->GetNodeOutputPin() == InNodeOutputPin && Link->GetNodeInputPin() == InNodeInputPin)
		{
			return GetActionStack()->RunAction<FOptimusNodeGraphAction_RemoveLink>(Link);
		}
	}

	return false;
}


bool UOptimusNodeGraph::RemoveAllLinks(UOptimusNodePin* InNodePin)
{
	if (!InNodePin)
	{
		return false;
	}

	TArray<int32> LinksToRemove = GetAllLinkIndexesToPin(InNodePin);
	if (LinksToRemove.Num() == 0)
	{
		return false;
	}

	FOptimusCompoundAction* Action = new FOptimusCompoundAction;
	if (LinksToRemove.Num() == 1)
	{
		Action->SetTitlef(TEXT("Remove Link"));
	}
	else
	{
		Action->SetTitlef(TEXT("Remove %d Links"), LinksToRemove.Num());
	}

	for (int32 LinkIndex : LinksToRemove)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Links[LinkIndex]);
	}

	return GetActionStack()->RunAction(Action);
}

bool UOptimusNodeGraph::ConnectAdderPin(
	IOptimusNodeAdderPinProvider* InTargetNode,
	const IOptimusNodeAdderPinProvider::FAdderPinAction& InSelectedAction,
	UOptimusNodePin* InSourcePin
	)
{
	IOptimusNodeAdderPinProvider *AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(InTargetNode);
	
	if (!AdderPinProvider)
	{
		return false;
	}
	
	if (!InSourcePin)
	{
		return false;
	}

	if (InSourcePin->GetDirection() == EOptimusNodePinDirection::Unknown)
	{
		return false;
	}

	
	IOptimusNodeAdderPinProvider::FAdderPinAction FinalAction = InSelectedAction;

	// For now lets default to not disconnect existing link on the source pin
	// Additional logic be added in the future if we want to let the user decide
	if (InSourcePin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		// Check to see if there's an existing link on the _input_ pin.
		TArray<int32> PinLinks = GetAllLinkIndexesToPin(InSourcePin);
		if (PinLinks.Num() > 0)
		{
			FinalAction.bCanAutoLink = false;
		}
	}

	return GetActionStack()->RunAction<FOptimusNodeGraphAction_ConnectAdderPin>(AdderPinProvider, FinalAction, InSourcePin);
}

UOptimusNode* UOptimusNodeGraph::ConvertCustomKernelToFunction(UOptimusNode* InCustomKernel)
{
	UOptimusNode_CustomComputeKernel* CustomKernelNode = Cast<UOptimusNode_CustomComputeKernel>(InCustomKernel);
	if (!CustomKernelNode)
	{
		UE_LOGF(LogOptimusCore, Error, "%ls: Not a custom kernel node.", *InCustomKernel->GetName());
		return nullptr;
	}

	// The node has to have at least one input and one output binding.
	if (CustomKernelNode->InputBindingArray.IsEmpty() || CustomKernelNode->OutputBindingArray.IsEmpty())
	{
		UE_LOGF(LogOptimusCore, Error, "%ls: Need at least one input binding and one output binding.", *CustomKernelNode->GetName());
		return nullptr;
	}

	// FIXME: We need to have a "compiled" state on the node, so that we know it's been successfully compiled.
	if (CustomKernelNode->GetDiagnosticLevel() == EOptimusDiagnosticLevel::Error)
	{
		UE_LOGF(LogOptimusCore, Error, "%ls: Node has an error on it.", *CustomKernelNode->GetName());
		return nullptr;
	}
	
	FOptimusCompoundAction *Action = new FOptimusCompoundAction(TEXT("Create Kernel Function"));

	// Remove all links from the old node but keep their paths so that we can re-connect once the
	// packaged node has been created with the same pins.
	TArray<TPair<FString, FString>> LinkPaths;
	for (const int32 LinkIndex : GetAllLinkIndexesToNode(CustomKernelNode))
	{
		UOptimusNodeLink* Link = Links[LinkIndex];
		LinkPaths.Emplace(Link->GetNodeOutputPin()->GetPinPath(), Links[LinkIndex]->GetNodeInputPin()->GetPinPath());
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Link);
	}

	Action->AddSubAction<FOptimusNodeGraphAction_RemoveNode>(CustomKernelNode);

	FOptimusNodeGraphAction_PackageKernelFunction* PackageNodeAction = new FOptimusNodeGraphAction_PackageKernelFunction(CustomKernelNode, CustomKernelNode->GetFName()); 
	Action->AddSubAction(PackageNodeAction);

	for (const TPair<FString, FString>& LinkInfo: LinkPaths)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_AddLink>(LinkInfo.Key, LinkInfo.Value);
	}

	if (!GetActionStack()->RunAction(Action))
	{
		return nullptr;
	}
	
	return PackageNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot());
}


UOptimusNode* UOptimusNodeGraph::ConvertFunctionToCustomKernel(UOptimusNode* InKernelFunction)
{
	UOptimusNode_ComputeKernelFunction* KernelFunctionNode = Cast<UOptimusNode_ComputeKernelFunction>(InKernelFunction);
	if (!KernelFunctionNode)
	{
		UE_LOGF(LogOptimusCore, Error, "%ls: Not a kernel function node.", *InKernelFunction->GetName());
		return nullptr;
	}

	FOptimusCompoundAction *Action = new FOptimusCompoundAction(TEXT("Unpack Kernel Function"));

	// Remove all links from the old node but keep their paths so that we can re-connect once the
	// packaged node has been created with the same pins.
	TArray<TPair<FString, FString>> LinkPaths;
	for (const int32 LinkIndex : GetAllLinkIndexesToNode(KernelFunctionNode))
	{
		UOptimusNodeLink* Link = Links[LinkIndex];
		LinkPaths.Emplace(Link->GetNodeOutputPin()->GetPinPath(), Links[LinkIndex]->GetNodeInputPin()->GetPinPath());
		Action->AddSubAction<FOptimusNodeGraphAction_RemoveLink>(Link);
	}

	Action->AddSubAction<FOptimusNodeGraphAction_RemoveNode>(KernelFunctionNode);

	FOptimusNodeGraphAction_UnpackageKernelFunction* UnpackageNodeAction = new FOptimusNodeGraphAction_UnpackageKernelFunction(KernelFunctionNode, KernelFunctionNode->GetFName()); 
	Action->AddSubAction(UnpackageNodeAction);

	for (const TPair<FString, FString>& LinkInfo: LinkPaths)
	{
		Action->AddSubAction<FOptimusNodeGraphAction_AddLink>(LinkInfo.Key, LinkInfo.Value);
	}

	if (!GetActionStack()->RunAction(Action))
	{
		return nullptr;
	}
	
	return UnpackageNodeAction->GetNode(GetActionStack()->GetGraphCollectionRoot());
}


UOptimusNode* UOptimusNodeGraph::CollapseNodesToFunction(
	const TArray<UOptimusNode*>& InNodes
	)
{
	return nullptr;
}


UOptimusNode* UOptimusNodeGraph::CollapseNodesToSubGraph(
	const TArray<UOptimusNode*>& InNodes
	)
{
	
	TArray<UOptimusNode*> NodesToCollapse;
	for (UOptimusNode* Node : InNodes)
	{
		if (Cast<IOptimusNonCollapsibleNode>(Node))
		{
			continue;
		}

		NodesToCollapse.Add(Node);
	}

	TArray<UOptimusNode*> NodesToDuplicate;
	TArray<UOptimusNodePair*> NodePairsToDuplicate;
	TArray<UOptimusNodeSubGraph*> SubGraphsToDuplicate;
	GatherRelatedObjects(NodesToCollapse, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);

	const TSet<UOptimusNode*> NodeSet(NodesToDuplicate);

	// Collect all links that go to nodes that are not a part of the group and all links that
	// are within elements of the group. At the same time, collect the bindings that apply.
	TArray<const UOptimusNodeLink*> InternalLinks;
	TArray<const UOptimusNodeLink*> InputLinks;			// Links going into the node set
	TMap<const UOptimusNodeLink*, FName> EntryNodePinNames;
	TArray<const UOptimusNodeLink*> OutputLinks;		// Links coming from the node set
	TMap<const UOptimusNodeLink*, FName> ReturnNodePinNames;
	TArray<FOptimusParameterBinding> InputBindings;
	TArray<FOptimusParameterBinding> OutputBindings;

	TSet<FName> BindingNames;
	BindingNames.Add(UOptimusNodeSubGraph::GraphDefaultComponentPinName);
	auto GetUniqueBindingName = [&BindingNames](FName InName)
	{
		FName NewName = InName;
		while (BindingNames.Contains(NewName))
		{
			NewName.SetNumber(NewName.GetNumber() + 1);
		}

		return NewName;
	};

	// FIXME: The bindings should be ordered by node's Y-coordinate and input number.
	for (const UOptimusNodeLink* Link: Links)
	{
		const bool bInputNodeInSet = NodeSet.Contains(Link->GetNodeInputPin()->GetOwningNode());
		const bool bOutputNodeInSet = NodeSet.Contains(Link->GetNodeOutputPin()->GetOwningNode());

		if (bInputNodeInSet && bOutputNodeInSet)
		{
			InternalLinks.Add(Link);
		}
		else if (bInputNodeInSet)
		{
			InputLinks.Add(Link);

			// The entry pin on our sub-graph is named after the input pin of the node that's
			// being collapsed into it.
			const UOptimusNodePin* InputPin = Link->GetNodeInputPin();
			FOptimusParameterBinding Binding;
			Binding.Name = GetUniqueBindingName(InputPin->GetFName());
			if (!InputPin->IsGroupingPin())
			{
				Binding.DataType = InputPin->GetDataType();
				Binding.DataDomain = InputPin->GetDataDomain();
			}
			else
			{
				// Grouping pin don't have a type currently so use the output pin instead
				Binding.DataType = Link->GetNodeOutputPin()->GetDataType();
				Binding.DataDomain = Link->GetNodeOutputPin()->GetDataDomain();
			}
			InputBindings.Add(Binding);
			BindingNames.Add(Binding.Name);
			EntryNodePinNames.Add(Link) = Binding.Name;
		}
		else if (bOutputNodeInSet)
		{
			OutputLinks.Add(Link);
			
			const UOptimusNodePin* OutputPin = Link->GetNodeOutputPin();
			FOptimusParameterBinding Binding;
			Binding.Name = GetUniqueBindingName(OutputPin->GetFName());
			Binding.DataType = OutputPin->GetDataType();
			Binding.DataDomain = OutputPin->GetDataDomain();
			OutputBindings.Add(Binding);
			BindingNames.Add(Binding.Name);
			ReturnNodePinNames.Add(Link) = Binding.Name;
		}
	}

	TWeakPtr<FOptimusNodeGraphAction_AddNode> WeakAddSubGraphRefNodeAction;
	
	
	{
		FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Collapse Nodes to Sub-graph"));
		
		FName SubGraphName = DefaultSubGraphName;
		SubGraphName = Optimus::GetUniqueNameForScope(this, SubGraphName);
		
		FOptimusNodeGraphAction_AddGraph* CreateGraph = new FOptimusNodeGraphAction_AddGraph(
			GetCollectionPath(), EOptimusNodeGraphType::SubGraph, SubGraphName, INDEX_NONE,
			[InputBindings, OutputBindings](UOptimusNodeGraph* InGraph) -> bool
			{
				UOptimusNodeSubGraph* SubGraph = Cast<UOptimusNodeSubGraph>(InGraph);
				SubGraph->InputBindings = InputBindings;
				SubGraph->OutputBindings = OutputBindings;
				return true;
			});
		GetActionStack()->RunAction(CreateGraph);

		FString SubGraphPath = ConstructSubGraphPath(SubGraphName.ToString());

		// Create the entry and return nodes.
		FBox2D NodeBox(ForceInit);
		for (const UOptimusNode* Node: NodesToDuplicate)
		{
			NodeBox += Node->GetGraphPosition();
		}
		
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::EntryNodeName,
			[NodeBox](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* EntryNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				EntryNode->TerminalType = EOptimusTerminalType::Entry;
				return EntryNode->SetGraphPositionDirect({NodeBox.Min.X - 450.0f, NodeBox.GetCenter().Y});
			});

		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::ReturnNodeName,
			[NodeBox](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* ReturnNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				ReturnNode->TerminalType = EOptimusTerminalType::Return;
				return ReturnNode->SetGraphPositionDirect({NodeBox.Max.X + 750.0f, NodeBox.GetCenter().Y});
			});

		for (UOptimusNodeSubGraph* SubGraphToDuplicate: SubGraphsToDuplicate)
		{
			DuplicateSubGraph(GetActionStack(), SubGraphPath, SubGraphToDuplicate, SubGraphToDuplicate->GetFName());
		}
		
		// Duplicate the nodes into the graph.
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_DuplicateNode>(
				SubGraphPath, Node, Node->GetFName(), [](UOptimusNode*)
				{
					return true;
				});
		}

		TArray<TPair<FString, FString>> NewNodePairs;
		for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
		{
			// Genernerate new pair
			const UOptimusNode *FirstNode = NodePair->GetFirst();
			const UOptimusNode *SecondNode = NodePair->GetSecond();

			FString NewFirstNodePath = ConstructPath(SubGraphPath, FirstNode->GetName(), {});
			FString NewSecondNodePath = ConstructPath(SubGraphPath, SecondNode->GetName(), {});

			NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
		}

		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}
		
		// Create the reference node and connect it.
		FName GraphNodeRefName = DefaultSubGraphRefNodeName;
		GraphNodeRefName = Optimus::GetUniqueNameForScope(this, GraphNodeRefName);
		
		{
			// Workaround for extracting result from the undo stack
			FOptimusCompoundAction* AddSubGraphRefNodeWrapperAction = new FOptimusCompoundAction;
			WeakAddSubGraphRefNodeAction = AddSubGraphRefNodeWrapperAction->AddSubAction<FOptimusNodeGraphAction_AddNode>(
				GetCollectionPath(), UOptimusNode_SubGraphReference::StaticClass(), GraphNodeRefName,
				[NodeBox, SubGraphName](UOptimusNode* InNode)
				{
					UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(InNode); 
					SubGraphNode->InitializeSerializedSubGraphName(SubGraphName);
					return SubGraphNode->SetGraphPositionDirect(NodeBox.GetCenter());
				});

			GetActionStack()->RunAction(AddSubGraphRefNodeWrapperAction);	
		}

		// Remove the nodes we have collapsed
		// Note: This also removes all existing links in the original graph. This has to be done before we add new links,
		// otherwise the node removal and link creation to already-connected inputs will fail.
		RemoveNodes(NodesToDuplicate);
		
		// Create the links in the sub-graph between the duplicated nodes and the entry/return
		// nodes using the internal/input/output links gathered previously.
		for (const UOptimusNodeLink* Link: InternalLinks)
		{
			FString NodeOutputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*SubGraphPath, *Link->NodeOutputPin->GetOwningNode()->GetName(), *Link->NodeOutputPin->GetUniqueName().ToString());
			FString NodeInputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*SubGraphPath, *Link->NodeInputPin->GetOwningNode()->GetName(), *Link->NodeInputPin->GetUniqueName().ToString());
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NodeOutputPinPath, NodeInputPinPath);
		}
		for (const UOptimusNodeLink* Link: InputLinks)
		{
			FString EntryNodePinName = EntryNodePinNames[Link].ToString(); 
			// Once for the Entry -> Sub-graph nodes, and another for Outer graph nodes -> Sub-graph ref node inputs.
			FString NodeOutputPinPath = ConstructPath(SubGraphPath, UOptimusNode_GraphTerminal::EntryNodeName.ToString(), EntryNodePinName);
			FString NodeInputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*SubGraphPath, *Link->NodeInputPin->GetOwningNode()->GetName(), *Link->NodeInputPin->GetUniqueName().ToString());
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NodeOutputPinPath, NodeInputPinPath);
			
			NodeOutputPinPath = Link->NodeOutputPin->GetPinPath();
			NodeInputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*GetCollectionPath(), *GraphNodeRefName.ToString(), *EntryNodePinName);
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NodeOutputPinPath, NodeInputPinPath);
		}
		for (const UOptimusNodeLink* Link: OutputLinks)
		{
			FString ReturnNodePinName = ReturnNodePinNames[Link].ToString();
			
			// Once for the Sub-graph nodes -> Return, and another for Sub-graph ref node outputs -> Outer graph nodes.
			FString NodeOutputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*SubGraphPath, *Link->NodeOutputPin->GetOwningNode()->GetName(), *Link->NodeOutputPin->GetUniqueName().ToString());
			FString NodeInputPinPath =
			ConstructPath(SubGraphPath, UOptimusNode_GraphTerminal::ReturnNodeName.ToString(), ReturnNodePinName);
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NodeOutputPinPath, NodeInputPinPath);
			
			NodeOutputPinPath = FString::Printf(TEXT("%s/%s.%s"),
				*GetCollectionPath(), *GraphNodeRefName.ToString(), *ReturnNodePinName);
			NodeInputPinPath = Link->NodeInputPin->GetPinPath();
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NodeOutputPinPath, NodeInputPinPath);
		}

		// Closes ActionScope
	}

	if (WeakAddSubGraphRefNodeAction.IsValid())
	{
		IOptimusPathResolver* PathResolver = GetPathResolver();
		return WeakAddSubGraphRefNodeAction.Pin()->GetNode(PathResolver);
	}

	return nullptr;
}


TArray<UOptimusNode*> UOptimusNodeGraph::ExpandCollapsedNodes(
	UOptimusNode* InGraphReferenceNode
	)
{
	const bool bIsFunction = IsFunctionReference(InGraphReferenceNode);
	const bool bIsSubGraph = IsSubGraphReference(InGraphReferenceNode);
	if (!bIsFunction && !bIsSubGraph)
	{
		return {};
	}

	TArray<UOptimusNodeLink*> ExternalLinks;	

	for (const int32 LinkIndex : GetAllLinkIndexesToNode(InGraphReferenceNode))
	{
		ExternalLinks.Add(Links[LinkIndex]);
	}

	// Use to move all nodes in the outer graph away from the expansion location
	// such that nodes currently in the subgraph would have enough space to spawn in the outer graph
	
	FVector2D ExpandLocation = InGraphReferenceNode->GetGraphPosition();
	
	TArray<TPair<UOptimusNode*, FVector2D>> NodePositionInfos;
	for (UOptimusNode* Node : Nodes)
	{
		if (Node == InGraphReferenceNode)
		{
			continue;
		}

		NodePositionInfos.Add({Node, Node->GetGraphPosition() - InGraphReferenceNode->GetGraphPosition()});
	}

	
	IOptimusNodeSubGraphReferencer* AsReferencerNode = Cast<IOptimusNodeSubGraphReferencer>(InGraphReferenceNode);
	IOptimusNodePinRouter* AsRouterNode = Cast<IOptimusNodePinRouter>(InGraphReferenceNode);

	if (AsReferencerNode && AsRouterNode)
	{
		UOptimusNodeSubGraph* SubGraph = AsReferencerNode->GetReferencedSubGraph();
		
		TArray<UOptimusNode*> NodesToConsider= SubGraph->Nodes;

		NodesToConsider.RemoveAll([](const UOptimusNode* InNode)
		{
			return Cast<UOptimusNode_GraphTerminal>(InNode);
		});

		TArray<UOptimusNode*> NodesToDuplicate;
		TArray<UOptimusNodePair*> NodePairsToDuplicate;
		TArray<UOptimusNodeSubGraph*> SubGraphsToDuplicate;
		
		GatherRelatedObjects(NodesToConsider, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);
		
		// Figure out the non-clashing names to use, to avoid collisions during actual execution.
		Optimus::FUniqueNameGenerator UniqueNameGenerator(this);
		
		using FloatType = decltype(FVector2D::X);
		FVector2D TopLeft{std::numeric_limits<FloatType>::max()};
		FVector2D BotRight{std::numeric_limits<FloatType>::lowest()};
		FVector2D HalfBoundSize;
		FVector2D Center;
		TMap<UOptimusNode*, FName> NewNodeNameMap;
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			TopLeft = FVector2D::Min(TopLeft, Node->GraphPosition);
			BotRight = FVector2D::Max(BotRight, Node->GraphPosition);
			NewNodeNameMap.Add(Node, UniqueNameGenerator.GetUniqueName(Node->GetFName()));
		}

		HalfBoundSize = (BotRight - TopLeft )/2 + FVector2D(100, 100);
		Center = (BotRight + TopLeft) / 2;
		
		const FString CurrentGraphPath = GetCollectionPath();
	
		TArray<TPair<FString, FString>> NewNodePairs;
		for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
		{
			// Generate new pair
			const UOptimusNode *FirstNode = NodePair->GetFirst();
			const UOptimusNode *SecondNode = NodePair->GetSecond();

			if (ensure(NewNodeNameMap.Contains(FirstNode) && NewNodeNameMap.Contains(SecondNode)))
			{
				FString NewFirstNodePath = ConstructPath(CurrentGraphPath, NewNodeNameMap[FirstNode].ToString(), {});
				FString NewSecondNodePath = ConstructPath(CurrentGraphPath, NewNodeNameMap[SecondNode].ToString(), {});

				NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
			}	
		}	

		// Using GraphName instead of the graph object as key here because this map is to be captured by a lambda to be used during undo/redo 
		TMap<FName , FName> NewSubGraphNameMap;
		for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
		{
			NewSubGraphNameMap.Add(SubGraphToDuplicate->GetFName(), UniqueNameGenerator.GetUniqueName(SubGraphToDuplicate->GetFName()));
		}
		
		TArray<TPair<FString, FString>> LinksToAdd;

		// Record all links linking terminal pins and nodes in the outer graph, saving to LinksToAdd array
		auto CollectLink = [&LinksToAdd, CurrentGraphPath, AsRouterNode, NewNodeNameMap](UOptimusNodePin* InReferenceNodePin)
		{
			const FOptimusRoutedNodePin PinCounterpart = AsRouterNode->GetPinCounterpart(InReferenceNodePin, {});

			for (const UOptimusNodePin* ExternalPin : InReferenceNodePin->GetConnectedPins())
			{
				FString ExternalPinPath = ExternalPin->GetPinPath();
				
				for (const UOptimusNodePin* InternalPin : PinCounterpart.NodePin->GetConnectedPins())
				{
					const UOptimusNode* InternalNode = InternalPin->GetOwningNode();
					FString InternalPinPath = ConstructPath(CurrentGraphPath, NewNodeNameMap[InternalNode].ToString(), InternalPin->GetUniqueName().ToString());

					if (ExternalPin->GetDirection() == EOptimusNodePinDirection::Output)
					{
						LinksToAdd.Add({ExternalPinPath, InternalPinPath});
					}
					else
					{
						LinksToAdd.Add({InternalPinPath, ExternalPinPath});
					}
				}
			}	
		};
		
		for (UOptimusNodePin* Pin : InGraphReferenceNode->GetPins())
		{
			CollectLink(Pin);
		}
		
		// Unconnected component pins should be linked to outer graph explicitly if the subgraph default component pin is connected externally
		TArray<UOptimusNodePin*> ExternalComponentPins = AsReferencerNode->GetDefaultComponentBindingPin()->GetConnectedPins();
		if (ExternalComponentPins.Num() > 0)
		{
			UOptimusNodePin* ExternalComponentPin = ExternalComponentPins[0];
			check(ExternalComponentPin->GetDirection() == EOptimusNodePinDirection::Output);
			
			FString ExternalPinPath = ExternalComponentPin->GetPinPath();
			
			for (UOptimusNode* Node : SubGraph->Nodes)
			{
				if (IOptimusComponentBindingReceiver* ComponentReceiver = Cast<IOptimusComponentBindingReceiver>(Node))
				{
					TArray<UOptimusNodePin*> UnconnectedComponentPin = ComponentReceiver->GetUnconnectedComponentPins();
					for (const UOptimusNodePin* InternalPin : UnconnectedComponentPin)
					{
						check(InternalPin->GetDirection() ==EOptimusNodePinDirection::Input);
						
						FString InternalPinPath = ConstructPath(CurrentGraphPath, NewNodeNameMap[Node].ToString(), InternalPin->GetUniqueName().ToString());
						
						LinksToAdd.Add({ExternalPinPath, InternalPinPath});
					}
				}
			}
		}
		
		TArray<UOptimusNodeLink*> InternalLinks = SubGraph->Links;
		InternalLinks.RemoveAll([](const UOptimusNodeLink* InLink)
		{
			return (Cast<UOptimusNode_GraphTerminal>(InLink->NodeInputPin->GetOwningNode()) || 
				Cast<UOptimusNode_GraphTerminal>(InLink->NodeOutputPin->GetOwningNode()));
		});

		for (UOptimusNodeLink* InternalLink : InternalLinks)
		{
			const UOptimusNodePin* OutputPin = InternalLink->GetNodeOutputPin();
			const UOptimusNode* OutputNode = OutputPin->GetOwningNode();
			const UOptimusNodePin* InputPin = InternalLink->GetNodeInputPin();
			const UOptimusNode* InputNode = InputPin->GetOwningNode();
			
			LinksToAdd.Add({
				ConstructPath(CurrentGraphPath, NewNodeNameMap[OutputNode].ToString(), OutputPin->GetUniqueName().ToString()),
				ConstructPath(CurrentGraphPath, NewNodeNameMap[InputNode].ToString(), InputPin->GetUniqueName().ToString())
			});
		}

		FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Expand Collapsed Nodes"));

		for (TPair<UOptimusNode*, FVector2D> NodePositionInfo: NodePositionInfos)
		{
			FVector2D Offset = NodePositionInfo.Value.GetSignVector() * HalfBoundSize;
			
			GetActionStack()->RunAction<FOptimusNodeAction_MoveNode>(NodePositionInfo.Key, NodePositionInfo.Key->GetGraphPosition() + Offset);
		}

		for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
		{
			DuplicateSubGraph(GetActionStack(), CurrentGraphPath, SubGraphToDuplicate, NewSubGraphNameMap[SubGraphToDuplicate->GetFName()]);
		}
		

		{
			FOptimusCompoundAction *PreDuplicateRequirementAction = new FOptimusCompoundAction;	
			// Add all the pre-duplicate requirement actions first. This allows certain nodes to set up their
			// operating environment correctly (e.g. a missing variable description for get variable nodes, etc.)
			for (UOptimusNode* Node: NodesToDuplicate)
			{
				Node->PreDuplicateRequirementActions(this, PreDuplicateRequirementAction);
			}
			GetActionStack()->RunAction(PreDuplicateRequirementAction);
		}

		// Duplicate the nodes and place them correctly
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_DuplicateNode>(
				GetCollectionPath(), Node, NewNodeNameMap[Node],
				[SourceNodeGraphPosition = Node->GraphPosition, Center, ExpandLocation, NewSubGraphNameMap](UOptimusNode *InNode)
				{
					if (UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(InNode))
					{
						FName NewSubGraphName = NewSubGraphNameMap[SubGraphNode->GetSerializedSubGraphName()];
						SubGraphNode->InitializeSerializedSubGraphName(NewSubGraphName);
					}
					InNode->SetGraphPositionDirect(SourceNodeGraphPosition - Center + ExpandLocation);
					return true;
			});
		}

		// This will remove the subgraph if it is a subgraph node
		RemoveNode(InGraphReferenceNode);
		
		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}

		for (const TPair<FString, FString>& NewLink : LinksToAdd)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(NewLink.Key, NewLink.Value);
		}
	}
	
	return {};
}

bool UOptimusNodeGraph::ConvertToFunction(UOptimusNode* InSubGraphNode)
{
	const bool bIsSubGraph = IsSubGraphReference(InSubGraphNode);
	if (!bIsSubGraph)
	{
		return false;
	}
	
	UOptimusNode_SubGraphReference* SubGraphNode = CastChecked<UOptimusNode_SubGraphReference>(InSubGraphNode);
	UOptimusNodeSubGraph* SubGraph = SubGraphNode->GetReferencedSubGraph();

	UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(GetCollectionRoot());
	if (!ensure(Deformer))
	{
		return false;
	}

	IOptimusPathResolver* PathResolver = GetPathResolver();

	// Ensure the name is unique here instead of relying on AddGraph action
	// because subsequent actions needs to use the final graph path
	// Note: we could change the actions to accept a lambda instead of specific value in the future
	FName FunctionName = TEXT("NewFunction");
	FunctionName = Optimus::GetUniqueNameForScope(Deformer, FunctionName);
	
	const FString FunctionGraphPath = GetFunctionGraphCollectionPath(FunctionName.ToString());
	UOptimusNode_GraphTerminal* SubGraphEntryNode = SubGraph->GetTerminalNode(EOptimusTerminalType::Entry);
	UOptimusNode_GraphTerminal* SubGraphReturnNode = SubGraph->GetTerminalNode(EOptimusTerminalType::Return);
	const FVector2D EntryNodePosition = SubGraphEntryNode->GetGraphPosition();
	const FVector2D ReturnNodePosition = SubGraphReturnNode->GetGraphPosition();
	const FVector2D ReferenceNodePosition = SubGraphNode->GetGraphPosition();
	const FName FunctionNodeName = Optimus::GetUniqueNameForScope(this, TEXT("FunctionReferenceNode"));
	const FString FunctionNodeNameString = FunctionNodeName.ToString();
	const FString CurrentGraphPath = GetCollectionPath();

	TMap<FString, TArray<FString>> ExternalLinksToAdd;

	for (UOptimusNodeLink* Link : GetAllLinks())
	{
		if (Link->GetNodeInputPin()->GetOwningNode() == SubGraphNode)
		{
			FString FunctionNodeInputPinPath = ConstructPath(
				CurrentGraphPath,
				FunctionNodeNameString,
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);

			FString ExistingOutputPinPath = Link->GetNodeOutputPin()->GetPinPath();
			ExternalLinksToAdd.FindOrAdd(ExistingOutputPinPath).Add(FunctionNodeInputPinPath);
		}
		else if (Link->GetNodeOutputPin()->GetOwningNode() == SubGraphNode)
		{
			FString FunctionNodeOutputPinPath = ConstructPath(
				CurrentGraphPath,
				FunctionNodeNameString,
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			FString ExistingInputPinPath = Link->GetNodeInputPin()->GetPinPath();
			ExternalLinksToAdd.FindOrAdd(FunctionNodeOutputPinPath).Add(ExistingInputPinPath);
		}
	}

	TMap<FString, TArray<FString>> InternalLinksToAdd;
	
	for (UOptimusNodeLink* Link : SubGraph->GetAllLinks())
	{
		if (Link->GetNodeInputPin()->GetOwningNode() == SubGraphReturnNode)
		{
			FString ReturnNodeInputPinPath = ConstructPath(
				FunctionGraphPath,
				UOptimusNode_GraphTerminal::ReturnNodeName.ToString(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeOutputPinPath = ConstructPath(
				FunctionGraphPath,
				Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			InternalLinksToAdd.FindOrAdd(InternalNodeOutputPinPath).Add(ReturnNodeInputPinPath);
		}
		else if (Link->GetNodeOutputPin()->GetOwningNode() == SubGraphEntryNode)
		{
			FString EntryNodeOutputPinPath = ConstructPath(
				FunctionGraphPath,
				UOptimusNode_GraphTerminal::EntryNodeName.ToString(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeInputPinPath = ConstructPath(
				FunctionGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			InternalLinksToAdd.FindOrAdd(EntryNodeOutputPinPath).Add(InternalNodeInputPinPath);
		}
		else
		{
			FString OutputPinPath = ConstructPath(
					FunctionGraphPath,
					Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
					Link->GetNodeOutputPin()->GetUniqueName().ToString()
					);

			FString InputPinPath = ConstructPath(
				FunctionGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			
			InternalLinksToAdd.FindOrAdd(OutputPinPath).Add(InputPinPath);
		}
	}	
	
	
	TArray<UOptimusNode*> SubGraphNodes = SubGraph->GetAllNodes();
	
	SubGraphNodes.RemoveAll([](UOptimusNode* InNode)
	{
		return Cast<UOptimusNode_GraphTerminal>(InNode) != nullptr;
	});

	TArray<UOptimusNode*> NodesToDuplicate;
	TArray<UOptimusNodePair*> NodePairsToDuplicate;
	TArray<UOptimusNodeSubGraph* > SubGraphsToDuplicate;
	GatherRelatedObjects(SubGraphNodes, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);

	TArray<TPair<FString, FString>> NewNodePairs;
	for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
	{
		// Generate new pair
		const UOptimusNode *FirstNode = NodePair->GetFirst();
		const UOptimusNode *SecondNode = NodePair->GetSecond();

		FString NewFirstNodePath = ConstructPath(FunctionGraphPath, FirstNode->GetName(), {});
		FString NewSecondNodePath = ConstructPath(FunctionGraphPath, SecondNode->GetName(), {});

		NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
	}	

	{
		FOptimusActionScope(*GetActionStack(), TEXT("Convert To Function"));
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddGraph>(Deformer->GetCollectionPath(), EOptimusNodeGraphType::Function, FunctionName, INDEX_NONE,
			[SubGraph](UOptimusNodeGraph* InGraph)
			{
				UOptimusFunctionNodeGraph* FunctionGraph = CastChecked<UOptimusFunctionNodeGraph>(InGraph);
				FunctionGraph->InputBindings = SubGraph->InputBindings;
				FunctionGraph->OutputBindings = SubGraph->OutputBindings;
				FunctionGraph->Init();
				return true;
			});
	
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			FunctionGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::EntryNodeName,
			[EntryNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* EntryNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				EntryNode->TerminalType = EOptimusTerminalType::Entry;

				return EntryNode->SetGraphPositionDirect(EntryNodePosition);
			});

		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			FunctionGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::ReturnNodeName,
			[ReturnNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* ReturnNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				ReturnNode->TerminalType = EOptimusTerminalType::Return;
				return ReturnNode->SetGraphPositionDirect(ReturnNodePosition);
			});

		for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
		{
			DuplicateSubGraph(GetActionStack(), FunctionGraphPath, SubGraphToDuplicate, SubGraphToDuplicate->GetFName());
		}	
		
		// Copy the nodes to the function graph
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_DuplicateNode>(
				FunctionGraphPath, Node, Node->GetFName(), [](UOptimusNode*)
				{
					return true;
				});
		}

		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}	
		
		// Restore links
		for (const TPair<FString, TArray<FString>>& InternalLinks : InternalLinksToAdd )
		{
			const FString& OutputPinPath = InternalLinks.Key;
			for (const FString& InputPinPath : InternalLinks.Value)
			{
				GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(OutputPinPath,	InputPinPath);
			}
		}
		
		// Create the function ref node
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			GetCollectionPath(), UOptimusNode_FunctionReference::StaticClass(), FunctionNodeName,
			[ReferenceNodePosition, FunctionGraphPath, PathResolver](UOptimusNode* InNode)
			{
				UOptimusNode_FunctionReference* FunctionNode = Cast<UOptimusNode_FunctionReference>(InNode); 
				UOptimusFunctionNodeGraph* FunctionGraph = Cast<UOptimusFunctionNodeGraph>(PathResolver->ResolveGraphPath(FunctionGraphPath));
				FunctionNode->SetReferencedFunctionGraph(FunctionGraph->GetGraphIdentifier());
				
				return FunctionNode->SetGraphPositionDirect(ReferenceNodePosition);
			});

		// Remove the subgraph node
		RemoveNode(SubGraphNode);
		
		// Restore links
		for (const TPair<FString, TArray<FString>>& ExternalLinks : ExternalLinksToAdd )
		{
			const FString& OutputPinPath = ExternalLinks.Key;
			for (const FString& InputPinPath : ExternalLinks.Value)
			{
				GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(OutputPinPath,	InputPinPath);
			}
		}
	}
	
	
	return true;
}

bool UOptimusNodeGraph::ConvertToSubGraph(UOptimusNode* InFunctionNode)
{
	const bool bIsFunctionReference = IsFunctionReference(InFunctionNode);
	if (!bIsFunctionReference)
	{
		return false;
	}
	
	UOptimusNode_FunctionReference* FunctionNode = CastChecked<UOptimusNode_FunctionReference>(InFunctionNode);
	UOptimusFunctionNodeGraph* FunctionGraph = Cast<UOptimusFunctionNodeGraph>(FunctionNode->GetReferencedSubGraph());

	if (!FunctionGraph)
	{
		return false;
	}

	IOptimusPathResolver* PathResolver = GetPathResolver();

	// Ensure the name is unique here instead of relying on AddGraph action
	// because subsequent actions needs to use the final graph path
	// Note: we could change the actions to accept a lambda instead of specific value in the future
	FName SubGraphName = DefaultSubGraphName;
	SubGraphName = Optimus::GetUniqueNameForScope(this, SubGraphName);
	
	const FString SubGraphPath = ConstructSubGraphPath(SubGraphName.ToString());
	UOptimusNode_GraphTerminal* FunctionGraphEntryNode = FunctionGraph->GetTerminalNode(EOptimusTerminalType::Entry);
	UOptimusNode_GraphTerminal* FunctionGraphReturnNode = FunctionGraph->GetTerminalNode(EOptimusTerminalType::Return);
	const FVector2D EntryNodePosition = FunctionGraphEntryNode->GetGraphPosition();
	const FVector2D ReturnNodePosition = FunctionGraphReturnNode->GetGraphPosition();
	const FVector2D ReferenceNodePosition = FunctionNode->GetGraphPosition();
	const FName SubGraphNodeName = Optimus::GetUniqueNameForScope(this, DefaultSubGraphRefNodeName);
	const FString SubGraphNodeNameString = SubGraphNodeName.ToString();
	const FString CurrentGraphPath = GetCollectionPath();

	TMap<FString, TArray<FString>> ExternalLinksToAdd;

	for (UOptimusNodeLink* Link : GetAllLinks())
	{
		if (Link->GetNodeInputPin()->GetOwningNode() == FunctionNode)
		{
			FString FunctionNodeInputPinPath = ConstructPath(
				CurrentGraphPath,
				SubGraphNodeNameString,
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);

			FString ExistingOutputPinPath = Link->GetNodeOutputPin()->GetPinPath();
			ExternalLinksToAdd.FindOrAdd(ExistingOutputPinPath).Add(FunctionNodeInputPinPath);
		}
		else if (Link->GetNodeOutputPin()->GetOwningNode() == FunctionNode)
		{
			FString FunctionNodeOutputPinPath = ConstructPath(
				CurrentGraphPath,
				SubGraphNodeNameString,
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			FString ExistingInputPinPath = Link->GetNodeInputPin()->GetPinPath();
			ExternalLinksToAdd.FindOrAdd(FunctionNodeOutputPinPath).Add(ExistingInputPinPath);
		}
	}

	TMap<FString, TArray<FString>> InternalLinksToAdd;
	
	for (UOptimusNodeLink* Link : FunctionGraph->GetAllLinks())
	{
		if (Link->GetNodeInputPin()->GetOwningNode() == FunctionGraphReturnNode)
		{
			FString ReturnNodeInputPinPath = ConstructPath(
				SubGraphPath,
				UOptimusNode_GraphTerminal::ReturnNodeName.ToString(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeOutputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			InternalLinksToAdd.FindOrAdd(InternalNodeOutputPinPath).Add(ReturnNodeInputPinPath);
		}
		else if (Link->GetNodeOutputPin()->GetOwningNode() == FunctionGraphEntryNode)
		{
			FString EntryNodeOutputPinPath = ConstructPath(
				SubGraphPath,
				UOptimusNode_GraphTerminal::EntryNodeName.ToString(),
				Link->GetNodeOutputPin()->GetUniqueName().ToString()
				);

			FString InternalNodeInputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			InternalLinksToAdd.FindOrAdd(EntryNodeOutputPinPath).Add(InternalNodeInputPinPath);
		}
		else
		{
			FString OutputPinPath = ConstructPath(
					SubGraphPath,
					Link->GetNodeOutputPin()->GetOwningNode()->GetName(),
					Link->GetNodeOutputPin()->GetUniqueName().ToString()
					);

			FString InputPinPath = ConstructPath(
				SubGraphPath,
				Link->GetNodeInputPin()->GetOwningNode()->GetName(),
				Link->GetNodeInputPin()->GetUniqueName().ToString()
				);
			
			InternalLinksToAdd.FindOrAdd(OutputPinPath).Add(InputPinPath);
		}
	}	
	
	
	TArray<UOptimusNode*> SubGraphNodes = FunctionGraph->GetAllNodes();
	
	SubGraphNodes.RemoveAll([](UOptimusNode* InNode)
	{
		return Cast<UOptimusNode_GraphTerminal>(InNode) != nullptr;
	});

	TArray<UOptimusNode*> NodesToDuplicate;
	TArray<UOptimusNodePair*> NodePairsToDuplicate;
	TArray<UOptimusNodeSubGraph*> SubGraphsToDuplicate;
	GatherRelatedObjects(SubGraphNodes, NodesToDuplicate, NodePairsToDuplicate, SubGraphsToDuplicate);

	TArray<TPair<FString, FString>> NewNodePairs;
	for (const UOptimusNodePair* NodePair : NodePairsToDuplicate)
	{
		// Generate new pair
		const UOptimusNode *FirstNode = NodePair->GetFirst();
		const UOptimusNode *SecondNode = NodePair->GetSecond();

		FString NewFirstNodePath = ConstructPath(SubGraphPath, FirstNode->GetName(), {});
		FString NewSecondNodePath = ConstructPath(SubGraphPath, SecondNode->GetName(), {});

		NewNodePairs.Add({NewFirstNodePath, NewSecondNodePath});
	}	

	{
		FOptimusActionScope(*GetActionStack(), TEXT("Convert To SubGraph"));
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddGraph>(GetCollectionPath(), EOptimusNodeGraphType::SubGraph, SubGraphName, INDEX_NONE,
			[FunctionGraph](UOptimusNodeGraph* InGraph)
			{
				UOptimusNodeSubGraph* SubGraph = CastChecked<UOptimusNodeSubGraph>(InGraph);
				SubGraph->InputBindings = FunctionGraph->InputBindings;
				SubGraph->OutputBindings = FunctionGraph->OutputBindings;
				return true;
			});
	
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::EntryNodeName,
			[EntryNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* EntryNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				EntryNode->TerminalType = EOptimusTerminalType::Entry;
				return EntryNode->SetGraphPositionDirect(EntryNodePosition);
			});

		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			SubGraphPath, UOptimusNode_GraphTerminal::StaticClass(), UOptimusNode_GraphTerminal::ReturnNodeName,
			[ReturnNodePosition](UOptimusNode* InNode)
			{
				UOptimusNode_GraphTerminal* ReturnNode = Cast<UOptimusNode_GraphTerminal>(InNode); 
				ReturnNode->TerminalType = EOptimusTerminalType::Return;
				return ReturnNode->SetGraphPositionDirect(ReturnNodePosition);
			});

		for (UOptimusNodeSubGraph* SubGraphToDuplicate : SubGraphsToDuplicate)
		{
			DuplicateSubGraph(GetActionStack(), SubGraphPath , SubGraphToDuplicate, SubGraphToDuplicate->GetFName());
		}
		
		// Copy the nodes to the sub graph
		for (UOptimusNode* Node: NodesToDuplicate)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_DuplicateNode>(
				SubGraphPath, Node, Node->GetFName(), [](UOptimusNode*)
				{
					return true;
				});
		}

		for (const TPair<FString, FString>& NewPair : NewNodePairs)
		{
			GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNodePair>(NewPair.Key, NewPair.Value);
		}	
		
		// Restore links
		for (const TPair<FString, TArray<FString>>& InternalLinks : InternalLinksToAdd )
		{
			const FString& OutputPinPath = InternalLinks.Key;
			for (const FString& InputPinPath : InternalLinks.Value)
			{
				GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(OutputPinPath,	InputPinPath);
			}
		}
		
		// Create the sub graph ref node
		GetActionStack()->RunAction<FOptimusNodeGraphAction_AddNode>(
			GetCollectionPath(), UOptimusNode_SubGraphReference::StaticClass(), SubGraphNodeName,
			[ReferenceNodePosition, SubGraphName](UOptimusNode* InNode)
			{
				UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(InNode);
				SubGraphNode->InitializeSerializedSubGraphName(SubGraphName);
				
				return SubGraphNode->SetGraphPositionDirect(ReferenceNodePosition);
			});

		// Remove the function node
		RemoveNode(FunctionNode);
		
		// Restore links
		for (const TPair<FString, TArray<FString>>& ExternalLinks : ExternalLinksToAdd )
		{
			const FString& OutputPinPath = ExternalLinks.Key;
			for (const FString& InputPinPath : ExternalLinks.Value)
			{
				GetActionStack()->RunAction<FOptimusNodeGraphAction_AddLink>(OutputPinPath,	InputPinPath);
			}
		}
	}
	
	return true;
}


bool UOptimusNodeGraph::IsCustomKernel(UOptimusNode* InNode) const
{
	return Cast<UOptimusNode_CustomComputeKernel>(InNode) != nullptr;
}


bool UOptimusNodeGraph::IsKernelFunction(UOptimusNode* InNode) const
{
	return Cast<UOptimusNode_ComputeKernelFunction>(InNode) != nullptr;
}


bool UOptimusNodeGraph::IsFunctionReference(UOptimusNode* InNode) const
{
	return Cast<UOptimusNode_FunctionReference>(InNode) != nullptr;
}


bool UOptimusNodeGraph::IsSubGraphReference(UOptimusNode* InNode) const
{
	return Cast<UOptimusNode_SubGraphReference>(InNode) != nullptr;
}

UOptimusNodePair* UOptimusNodeGraph::GetNodePair(const UOptimusNode* InNode) const
{
	for (int32 Index = 0; Index < NodePairs.Num(); Index++)
	{
		UOptimusNodePair* NodePair = NodePairs[Index];

		if (NodePair->Contains(InNode))
		{
			return NodePair;
		}
	}

	return nullptr;
}

UOptimusNode* UOptimusNodeGraph::GetNodeCounterpart(const UOptimusNode* InNode) const
{
	if (UOptimusNodePair* NodePair = GetNodePair(InNode))
	{
		return NodePair->GetNodeCounterpart(InNode);
	}

	return nullptr;
}

UOptimusNode* UOptimusNodeGraph::CreateNodeDirect(
	const UClass* InNodeClass,
	FName InName,
	TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
)
{
	check(InNodeClass->IsChildOf(UOptimusNode::StaticClass()));

	UOptimusNode* NewNode = NewObject<UOptimusNode>(this, InNodeClass, InName, RF_Transactional);

	// Configure the node as needed.
	if (InConfigureNodeFunc)
	{
		// Suppress notifications for this node while we're calling its configure callback. 
		TGuardValue<bool> SuppressNotifications(NewNode->bSendNotifications, false);
		
		if (!InConfigureNodeFunc(NewNode))
		{
			Optimus::RemoveObject(NewNode);
			return nullptr;
		}
	}

	NewNode->PostCreateNode();

	AddNodeDirect(NewNode);

	return NewNode;
}


bool UOptimusNodeGraph::AddNodeDirect(UOptimusNode* InNode)
{
	if (InNode == nullptr)
	{
		return false;
	}

	// Re-parent this node if it's not owned directly by us.
	if (InNode->GetOuter() != this)
	{
		UOptimusNodeGraph* OtherGraph = Cast<UOptimusNodeGraph>(InNode->GetOuter());

		// We can't re-parent this node if it still has links.
		if (OtherGraph && OtherGraph->GetAllLinkIndexesToNode(InNode).Num() != 0)
		{
			return false;
		}

		Optimus::RenameObject(InNode, nullptr, this);
	}

	Nodes.Add(InNode);

	Notify(EOptimusGraphNotifyType::NodeAdded, InNode);

	(void)InNode->MarkPackageDirty();

	return true;
}


bool UOptimusNodeGraph::RemoveNodeDirect(
	UOptimusNode* InNode, 
	bool bFailIfLinks
	)
{
	int32 NodeIndex = Nodes.IndexOfByKey(InNode);

	// We should always have a node, unless the bookkeeping went awry.
	check(NodeIndex != INDEX_NONE);
	if (NodeIndex == INDEX_NONE)
	{
		return false;
	}

	// There should be no links to this node.
	if (bFailIfLinks)
	{
		TArray<int32> LinkIndexes = GetAllLinkIndexesToNode(InNode);
		if (LinkIndexes.Num() != 0)
		{
			return false;
		}
	}
	else
	{ 
		RemoveAllLinksToNodeDirect(InNode);
	}

	Nodes.RemoveAt(NodeIndex);

	Notify(EOptimusGraphNotifyType::NodeRemoved, InNode);

	// Unparent this node to a temporary storage and mark it for kill.
	Optimus::RemoveObject(InNode);

	return true;
}

bool UOptimusNodeGraph::AddNodePairDirect(UOptimusNode* InFirstNode, UOptimusNode* InSecondNode)
{
	if (!ensure(Cast<IOptimusNodePairProvider>(InFirstNode) && Cast<IOptimusNodePairProvider>(InSecondNode)))
	{
		return false;
	}

	UOptimusNodePair* NewPair = NewObject<UOptimusNodePair>(this);
	NewPair->First = InFirstNode;
	NewPair->Second = InSecondNode;

	NodePairs.Add(NewPair);

	IOptimusNodePairProvider* FirstProvider = CastChecked<IOptimusNodePairProvider>(InFirstNode);
	IOptimusNodePairProvider* SecondProvider = CastChecked<IOptimusNodePairProvider>(InSecondNode);

	FirstProvider->PairToCounterpartNode(SecondProvider);
	SecondProvider->PairToCounterpartNode(FirstProvider);
	
	return true;
}

bool UOptimusNodeGraph::RemoveNodePairDirect(UOptimusNode* InFirstNode, UOptimusNode* InSecondNode)
{
	if (!ensure(Cast<IOptimusNodePairProvider>(InFirstNode) && Cast<IOptimusNodePairProvider>(InSecondNode)))
	{
		return false;
	}
	
	for (int32 Index = 0; Index < NodePairs.Num(); Index++)
	{
		const UOptimusNodePair* NodePair = NodePairs[Index];

		if (NodePair->Contains(InFirstNode, InSecondNode))
		{
			RemoveNodePairByIndex(Index);
			return true;
		}
	}

	return false;
}


bool UOptimusNodeGraph::AddLinkDirect(UOptimusNodePin* NodeOutputPin, UOptimusNodePin* NodeInputPin)
{
	if (!ensure(NodeOutputPin != nullptr && NodeInputPin != nullptr))
	{
		return false;
	}

	if (!ensure(
		NodeOutputPin->GetDirection() == EOptimusNodePinDirection::Output &&
		NodeInputPin->GetDirection() == EOptimusNodePinDirection::Input))
	{
		return false;
	}

	if (NodeOutputPin == NodeInputPin || NodeOutputPin->GetOwningNode() == NodeInputPin->GetOwningNode())
	{
		return false;
	}

	// Does this link already exist?
	for (const UOptimusNodeLink* Link : Links)
	{
		if (Link->GetNodeOutputPin() == NodeOutputPin && Link->GetNodeInputPin() == NodeInputPin)
		{
			return false;
		}
	}

	UOptimusNodeLink* NewLink = NewObject<UOptimusNodeLink>(this);
	NewLink->NodeOutputPin = NodeOutputPin;
	NewLink->NodeInputPin = NodeInputPin;
	Links.Add(NewLink);

	Notify(EOptimusGraphNotifyType::LinkAdded, NewLink);

	NewLink->MarkPackageDirty();

	return true;
}


bool UOptimusNodeGraph::RemoveLinkDirect(UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin)
{
	if (!InNodeOutputPin || !InNodeInputPin)
	{
		return false;
	}

	check(InNodeOutputPin->GetDirection() == EOptimusNodePinDirection::Output);
	check(InNodeInputPin->GetDirection() == EOptimusNodePinDirection::Input);

	if (InNodeOutputPin->GetDirection() != EOptimusNodePinDirection::Output ||
		InNodeInputPin->GetDirection() != EOptimusNodePinDirection::Input)
	{
		return false;
	}

	for (int32 LinkIndex = 0; LinkIndex < Links.Num(); LinkIndex++)
	{
		const UOptimusNodeLink* Link = Links[LinkIndex];

		if (Link->GetNodeOutputPin() == InNodeOutputPin && Link->GetNodeInputPin() == InNodeInputPin)
		{
			RemoveLinkByIndex(LinkIndex);
			return true;
		}
	}

	return false;
}


bool UOptimusNodeGraph::RemoveAllLinksToPinDirect(UOptimusNodePin* InNodePin)
{
	if (!InNodePin)
	{
		return false;
	}

	TArray<int32> LinksToRemove = GetAllLinkIndexesToPin(InNodePin);

	if (LinksToRemove.Num() == 0)
	{
		return false;
	}

	// Remove the links in reverse order so that we pop off the highest index first.
	for (int32 i = LinksToRemove.Num(); i-- > 0; /**/)
	{
		RemoveLinkByIndex(LinksToRemove[i]);
	}

	return true;
}


bool UOptimusNodeGraph::RemoveAllLinksToNodeDirect(UOptimusNode* InNode)
{
	if (!InNode)
	{
		return false;
	}

	TArray<int32> LinksToRemove = GetAllLinkIndexesToNode(InNode);

	if (LinksToRemove.Num() == 0)
	{
		return false;
	}

	// Remove the links in reverse order so that we pop off the highest index first.
	for (int32 i = LinksToRemove.Num(); i-- > 0; /**/)
	{
		RemoveLinkByIndex(LinksToRemove[i]);
	}

	return true;
}


TArray<UOptimusNodePin*> UOptimusNodeGraph::GetConnectedPins(
	const UOptimusNodePin* InNodePin
	) const
{
	TArray<UOptimusNodePin*> ConnectedPins;
	for (int32 Index: GetAllLinkIndexesToPin(InNodePin))
	{
		const UOptimusNodeLink* Link = Links[Index];

		if (Link->GetNodeInputPin() == InNodePin)
		{
			ConnectedPins.Add(Link->GetNodeOutputPin());
		}
		else if (Link->GetNodeOutputPin() == InNodePin)
		{
			ConnectedPins.Add(Link->GetNodeInputPin());
		}
	}
	return ConnectedPins;
}


TArray<FOptimusRoutedNodePin> UOptimusNodeGraph::GetConnectedPinsWithRouting(
	const UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InContext
	) const
{
	TQueue<FOptimusRoutedNodePin> PinQueue;

	PinQueue.Enqueue({const_cast<UOptimusNodePin*>(InNodePin), InContext});

	TArray<FOptimusRoutedNodePin> RoutedNodePins;
	FOptimusRoutedNodePin WorkingPin;
	while (PinQueue.Dequeue(WorkingPin))
	{
		for (UOptimusNodePin* ConnectedPin: WorkingPin.NodePin->GetConnectedPins())
		{
			if (ensure(ConnectedPin != nullptr))
			{
				// If this connection leads to a router node, find the matching pin on the other side and
				// add it to the queue. Otherwise we're done, and we add the connected pin and the
				// context to the result (in case the user wants to traverse further via that node through
				// the given pin).
				if (const IOptimusNodePinRouter* RouterNode = Cast<IOptimusNodePinRouter>(ConnectedPin->GetOwningNode()))
				{
					const FOptimusRoutedNodePin RoutedPin = RouterNode->GetPinCounterpart(ConnectedPin, WorkingPin.TraversalContext);
					if (RoutedPin.NodePin != nullptr)
					{
						PinQueue.Enqueue(RoutedPin);
					}
				}
				else
				{
					RoutedNodePins.Add({ConnectedPin, WorkingPin.TraversalContext});
				}
			}
		}
	}

	return RoutedNodePins;
}

TArray<FOptimusRoutedNodePin> UOptimusNodeGraph::GetConnectedPinsWithRouting(const UOptimusNodePin* InNodePin, const FOptimusPinTraversalContext& InContext, EOptimusNodePinTraversalDirection Direction) const
{
	if (Direction == EOptimusNodePinTraversalDirection::Default)
	{
		return GetConnectedPinsWithRouting(InNodePin, InContext);
	}
	
	const UOptimusNode* Node = InNodePin->GetOwningNode();
	UOptimusNodePin* NodePin = const_cast<UOptimusNodePin*>(InNodePin);

	FOptimusRoutedNodePin StartPin;

	if (Direction == EOptimusNodePinTraversalDirection::Upstream)
	{
		if (InNodePin->GetDirection() == EOptimusNodePinDirection::Output)
    	{
    		if (const IOptimusNodePinRouter* RouterNode = Cast<const IOptimusNodePinRouter>(Node))
    		{
    			StartPin = RouterNode->GetPinCounterpart(NodePin, InContext);
    		}
    		else
    		{
    			return {{NodePin, InContext}};
    		}
    	}
    	else
    	{
    		StartPin = {NodePin, InContext};
    	}	
	}
	else
	{
		if (InNodePin->GetDirection() == EOptimusNodePinDirection::Input)
		{
			if (const IOptimusNodePinRouter* RouterNode = Cast<const IOptimusNodePinRouter>(Node))
			{
				StartPin = RouterNode->GetPinCounterpart(NodePin, InContext);
			}
			else
			{
				return {{NodePin, InContext}};
			}
		}
		else
		{
			StartPin = {NodePin, InContext};
		}
	}
	
	if (StartPin.NodePin)
	{
		return GetConnectedPinsWithRouting(StartPin.NodePin, StartPin.TraversalContext);
	}

	return {};
}


TSet<UOptimusComponentSourceBinding*> UOptimusNodeGraph::GetComponentSourceBindingsForPin(
	const UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InContext
	) const
{
	TSet<FOptimusRoutedConstNode> VisitedNodes;
	TSet<UOptimusComponentSourceBinding*> Bindings;
	
	TQueue<FOptimusRoutedConstNode> WorkingSet;

	TArray<FOptimusRoutedNodePin> ConnectedPins = GetConnectedPinsWithRouting(InNodePin, InContext, EOptimusNodePinTraversalDirection::Upstream);
	if (ConnectedPins.IsEmpty())
	{
		return {};
	}

	check(ConnectedPins.Num() == 1);
	const UOptimusNode* StartNode = ConnectedPins[0].NodePin->GetOwningNode();
	WorkingSet.Enqueue({StartNode, ConnectedPins[0].TraversalContext});
	
	FOptimusRoutedConstNode WorkItem;
	while (WorkingSet.Dequeue(WorkItem))
	{
		TArray<const UOptimusNodePin*> InputPins; 
		const UOptimusNode* Node = WorkItem.Node;

		if (const IOptimusComponentBindingProvider* ComponentSourceBindingProvider = Cast<const IOptimusComponentBindingProvider>(Node))
		{
			if (UOptimusComponentSourceBinding* Binding = ComponentSourceBindingProvider->GetComponentBinding(WorkItem.TraversalContext))
			{
				Bindings.Add(Binding);
			}
			continue;
		}
		
		if (const IOptimusComputeKernelProvider* KernelProvider = Cast<const IOptimusComputeKernelProvider>(Node))
		{
			const UOptimusNodePin* PrimaryGroupPin = KernelProvider->GetPrimaryGroupPin();

			// Group pin traversal is useful when the kernel does not have any input data pins
			InputPins.Add(PrimaryGroupPin);
			InputPins.Append(PrimaryGroupPin->GetSubPins());
		}
		else
		{
			// Grab all inputs.
			for (const UOptimusNodePin* Pin: Node->GetPins())
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
				{
					InputPins.Add(Pin);
				}
			}
		}
		
		// Traverse in the direction of inputs to outputs (up the graph).
		for (const UOptimusNodePin* Pin: InputPins)
		{
			for (const FOptimusRoutedNodePin& ConnectedPin: Pin->GetConnectedPinsWithRouting(WorkItem.TraversalContext))
			{
				if (ensure(ConnectedPin.NodePin != nullptr))
				{
					const UOptimusNode *NextNode = ConnectedPin.NodePin->GetOwningNode();
					FOptimusRoutedConstNode CollectedNode{NextNode, ConnectedPin.TraversalContext};
					
					if (!VisitedNodes.Contains(CollectedNode))
					{
						VisitedNodes.Add(CollectedNode);
						WorkingSet.Enqueue(CollectedNode);
					}
				}
			}
		}
	}

	return Bindings;
}


bool UOptimusNodeGraph::IsPinMutable(const UOptimusNodePin* InNodePin, const FOptimusPinTraversalContext& InContext) const
{
	TSet<FOptimusRoutedConstNodePin> VisitedPins;
	TQueue<FOptimusRoutedConstNodePin> WorkingSet;

	// If given an input pin, find the other side, since output pin provides mutability definition
	const UOptimusNodePin* StartNodePin;
	FOptimusPinTraversalContext StartContext;
	if (InNodePin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		TArray<FOptimusRoutedNodePin> RoutedPins = GetConnectedPinsWithRouting(InNodePin, InContext);
		if (RoutedPins.IsEmpty())
		{
			// Nothing is connected, pin data will never change
			return false;
		}
		
		check(RoutedPins.Num() == 1);
		StartNodePin = RoutedPins[0].NodePin;
		StartContext = RoutedPins[0].TraversalContext;
	}
	else
	{
		StartNodePin = InNodePin;
		StartContext = InContext;
	}

	WorkingSet.Enqueue({StartNodePin, StartContext});
	
	FOptimusRoutedConstNodePin WorkItem;
	while (WorkingSet.Dequeue(WorkItem))
	{
		TArray<const UOptimusNodePin*> InputPins; 
		const UOptimusNodePin* NodePin = WorkItem.NodePin;
		const UOptimusNode* Node = NodePin->GetOwningNode();

		if (const IOptimusPinMutabilityDefiner* PinMutabilityDefiner = Cast<const IOptimusPinMutabilityDefiner>(Node))
		{
			EOptimusPinMutability Mutability = PinMutabilityDefiner->GetOutputPinMutability(NodePin);
			if (Mutability != EOptimusPinMutability::Undefined)
			{
				if (Mutability == EOptimusPinMutability::Mutable)
				{
					return true;
				}

				// Dead-end on a immutable pin, consume this work item and don't look further
				continue;
			}

			// Mutability is undefined, keep gathering from connected upstream nodes
		}
		
		// Traverse in the direction of inputs to outputs (up the graph).
		for (const UOptimusNodePin* Pin: Node->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
		{
			for (const FOptimusRoutedNodePin& ConnectedPin: Pin->GetConnectedPinsWithRouting(WorkItem.TraversalContext))
			{
				if (ensure(ConnectedPin.NodePin != nullptr))
				{
					FOptimusRoutedConstNodePin CollectedNodePin{ConnectedPin.NodePin, ConnectedPin.TraversalContext};

					if (!VisitedPins.Contains(CollectedNodePin))
					{
						VisitedPins.Add(CollectedNodePin);
						WorkingSet.Enqueue(CollectedNodePin);
					}
				}
			}
		}

		if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(Node))
		{
			if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry && ensure(!NodePin->GetDataDomain().IsSingleton()))
			{
				const UOptimusNodePin* NextPin = LoopTerminal->GetPinCounterpart(NodePin, EOptimusTerminalType::Return, EOptimusNodePinDirection::Output);
				
				FOptimusRoutedConstNodePin CollectedNodePin{NextPin, WorkItem.TraversalContext};	
				if (!VisitedPins.Contains(CollectedNodePin))
				{
					VisitedPins.Add(CollectedNodePin);
					WorkingSet.Enqueue({NextPin, WorkItem.TraversalContext});
				}
			}
		}
	}

	return false;
}

bool UOptimusNodeGraph::DoesNodeHaveMutableInput(const UOptimusNode* InNode, const FOptimusPinTraversalContext& InContext) const
{
	for (UOptimusNodePin* Pin : InNode->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
	{
		if (IsPinMutable(Pin, InContext))
		{
			return true;
		}
	}

	return false;
}

TSet<FOptimusRoutedConstNode> UOptimusNodeGraph::GetLoopEntryTerminalForPin(
	const UOptimusNodePin* InNodePin,
	const FOptimusPinTraversalContext& InContext) const
{
	TSet<FOptimusRoutedConstNode> VisitedNodes;
	TSet<FOptimusRoutedConstNode> EntryTerminals;
	
	TQueue<FOptimusRoutedConstNode> WorkingSet;

	// If given an input pin, find the other side.
	FOptimusRoutedConstNode StartNode;
	if (InNodePin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		TArray<FOptimusRoutedNodePin> RoutedPins = GetConnectedPinsWithRouting(InNodePin, InContext);
		if (RoutedPins.IsEmpty())
		{
			// If the input pin is directly on a compute kernel and not connected to anything
			// we consider it to not have a component source, even if other pins in the same group
			// has one. Pin group level gathering is caller's responsibility
			return {};
		}
		
		check(RoutedPins.Num() == 1);
		StartNode = {RoutedPins[0].NodePin->GetOwningNode(), RoutedPins[0].TraversalContext};
	}
	else
	{
		StartNode = {InNodePin->GetOwningNode(), InContext};
	}

	WorkingSet.Enqueue(StartNode);
	
	FOptimusRoutedConstNode WorkItem;
	while (WorkingSet.Dequeue(WorkItem))
	{
		TArray<const UOptimusNodePin*> InputPins; 
		const UOptimusNode* Node = WorkItem.Node;

		if (const UOptimusNode_LoopTerminal* LoopTerminal = Cast<const UOptimusNode_LoopTerminal>(Node))
		{
			if (LoopTerminal->GetTerminalType() == EOptimusTerminalType::Entry)
			{
				EntryTerminals.Add(WorkItem);
			}
			continue;
		}
		
		if (const IOptimusComputeKernelProvider* KernelProvider = Cast<const IOptimusComputeKernelProvider>(Node))
		{
			const UOptimusNodePin* PrimaryGroupPin = KernelProvider->GetPrimaryGroupPin();

			InputPins.Append(PrimaryGroupPin->GetSubPins());
		}
		else
		{
			// Grab all inputs.
			for (const UOptimusNodePin* Pin: Node->GetPins())
			{
				if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
				{
					InputPins.Add(Pin);
				}
			}
		}
		
		// Traverse in the direction of inputs to outputs (up the graph).
		for (const UOptimusNodePin* Pin: InputPins)
		{
			for (const FOptimusRoutedNodePin& ConnectedPin: Pin->GetConnectedPinsWithRouting(WorkItem.TraversalContext))
			{
				if (ensure(ConnectedPin.NodePin != nullptr))
				{
					UOptimusNode *NextNode = ConnectedPin.NodePin->GetOwningNode();
					FOptimusRoutedConstNode CollectedNode{NextNode, ConnectedPin.TraversalContext};
					
					if (!VisitedNodes.Contains(CollectedNode))
					{
						VisitedNodes.Add(CollectedNode);
						WorkingSet.Enqueue(CollectedNode);
					}
				}
			}
		}
	}

	return EntryTerminals;
}

TSet<FOptimusRoutedConstNode> UOptimusNodeGraph::GetLoopEntryTerminalForNode(
	const UOptimusNode* InNode, 
	const FOptimusPinTraversalContext& InContext) const
{
	TSet<FOptimusRoutedConstNode> ConnectedEntryTerminals; 
	for (UOptimusNodePin* Pin : InNode->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
	{
		ConnectedEntryTerminals.Append(GetLoopEntryTerminalForPin(Pin, InContext));
	}

	return ConnectedEntryTerminals;
}


TArray<const UOptimusNodeLink*> UOptimusNodeGraph::GetPinLinks(
	const UOptimusNodePin* InNodePin
	) const
{
	TArray<const UOptimusNodeLink*> PinLinks;
	for (const int32 Index: GetAllLinkIndexesToPin(InNodePin))
	{
		const UOptimusNodeLink* Link = Links[Index];

		if (Link->GetNodeInputPin() == InNodePin)
		{
			PinLinks.Add(Link);
		}
		else if (Link->GetNodeOutputPin() == InNodePin)
		{
			PinLinks.Add(Link);
		}
	}
	return PinLinks;
}


void UOptimusNodeGraph::RemoveNodePairByIndex(int32 NodePairIndex)
{
	UOptimusNodePair* NodePair = NodePairs[NodePairIndex];

	NodePairs.RemoveAt(NodePairIndex);

	Optimus::RemoveObject(NodePair);
}

void UOptimusNodeGraph::RemoveLinkByIndex(int32 LinkIndex)
{
	UOptimusNodeLink* Link = Links[LinkIndex];

	Links.RemoveAt(LinkIndex);

	Notify(EOptimusGraphNotifyType::LinkRemoved, Link);

	// Unparent the link to a temporary storage and mark it for kill.
	Optimus::RemoveObject(Link);
}

bool UOptimusNodeGraph::DoesLinkFormCycle(const UOptimusNode* InOutputNode, const UOptimusNode* InInputNode) const
{
	if (!InOutputNode || !InInputNode)
	{
		// Invalid nodes -- no cycle;
		return false;
	}

	// Self-connection is a cycle.
	if (InOutputNode == InInputNode)
	{
		return true;
	}

	const UOptimusNode *CycleNode = InOutputNode;

	// Crawl forward from the input pin's node to see if we end up hitting the output pin's node.
	TSet<const UOptimusNode *> ProcessedNodes;
	TQueue<int32> QueuedLinks;

	auto EnqueueIndexes = [&QueuedLinks](TArray<int32> InArray) -> void
	{
		for (int32 Index : InArray)
		{
			QueuedLinks.Enqueue(Index);
		}
	};

	// Enqueue as a work set all links going from the output pins of the node.
	EnqueueIndexes(GetAllLinkIndexesToNode(InInputNode, EOptimusNodePinDirection::Output));
	ProcessedNodes.Add(InInputNode);

	int32 LinkIndex;
	while (QueuedLinks.Dequeue(LinkIndex))
	{
		const UOptimusNodeLink *Link = Links[LinkIndex];

		const UOptimusNode *NextNode = Link->GetNodeInputPin()->GetOwningNode();

		if (NextNode == CycleNode)
		{
			// We hit the node we want to connect from, so this would cause a cycle.
			return true;
		}

		// If we haven't processed the next node yet, enqueue all its output links and mark
		// this next node as done so we don't process it again.
		if (!ProcessedNodes.Contains(NextNode))
		{
			EnqueueIndexes(GetAllLinkIndexesToNode(NextNode, EOptimusNodePinDirection::Output));
			ProcessedNodes.Add(NextNode);
		}
	}

	// We didn't hit our target node.
	return false;
}


void UOptimusNodeGraph::Notify(EOptimusGraphNotifyType InNotifyType, UObject* InSubject) const
{
	if (InNotifyType == EOptimusGraphNotifyType::NodeDiagnosticLevelChanged)
	{
		if (UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(GetCollectionRoot()))
		{
			Deformer->SetStatusFromDiagnostic(Cast<UOptimusNode>(InSubject)->GetDiagnosticLevel());
		}
	}
	else if (InNotifyType != EOptimusGraphNotifyType::NodePositionChanged &&
			 InNotifyType != EOptimusGraphNotifyType::PinExpansionChanged
		)
	{
		if (UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(GetCollectionRoot()))
		{
			Deformer->MarkModified();
		}
	}
	
	GraphNotifyDelegate.Broadcast(InNotifyType, const_cast<UOptimusNodeGraph*>(this), InSubject);
}


void UOptimusNodeGraph::GlobalNotify(EOptimusGlobalNotifyType InNotifyType, UObject* InObject) const
{
	const UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(GetCollectionRoot());
	if (Deformer)
	{
		Deformer->Notify(InNotifyType, InObject);
	}
	
}


TArray<int32> UOptimusNodeGraph::GetAllLinkIndexesToNode(
	const UOptimusNode* InNode,
	EOptimusNodePinDirection InDirection
	) const
{
	TArray<int32> LinkIndexes;
	for (int32 LinkIndex = 0; LinkIndex < Links.Num(); LinkIndex++)
	{
		const UOptimusNodeLink* Link = Links[LinkIndex];
		if (ensure(Link != nullptr && Link->GetNodeOutputPin() != nullptr))
		{
			if ((Link->GetNodeOutputPin()->GetOwningNode() == InNode && InDirection != EOptimusNodePinDirection::Input) ||
				(Link->GetNodeInputPin()->GetOwningNode() == InNode && InDirection != EOptimusNodePinDirection::Output))
			{
				LinkIndexes.Add(LinkIndex);
			}
		}
	}

	return LinkIndexes;
}


TArray<int32> UOptimusNodeGraph::GetAllLinkIndexesToNode(const UOptimusNode* InNode) const
{
	return GetAllLinkIndexesToNode(InNode, EOptimusNodePinDirection::Unknown);
}


TArray<int32> UOptimusNodeGraph::GetAllLinkIndexesToPin(
	const UOptimusNodePin* InNodePin
	) const
{
	TArray<int32> LinkIndexes;
	for (int32 LinkIndex = 0; LinkIndex < Links.Num(); LinkIndex++)
	{
		const UOptimusNodeLink* Link = Links[LinkIndex];

		if ((InNodePin->GetDirection() == EOptimusNodePinDirection::Input &&
			Link->GetNodeInputPin() == InNodePin) ||
			(InNodePin->GetDirection() == EOptimusNodePinDirection::Output &&
				Link->GetNodeOutputPin() == InNodePin))
		{
			LinkIndexes.Add(LinkIndex);
		}
	}

	return LinkIndexes;
}

FString UOptimusNodeGraph::ConstructSubGraphPath(const FString& InSubGraphName) const
{
	return ConstructSubGraphPath(GetCollectionPath(), InSubGraphName);
}

FString UOptimusNodeGraph::ConstructSubGraphPath(const FString& InGraphOwnerPath, const FString& InSubGraphName)
{
	FString SubGraphPath = InGraphOwnerPath + TEXT("/") + InSubGraphName;

	return SubGraphPath;	
}

void UOptimusNodeGraph::PostLoadReplaceAnimAttributeDataInterfaceNodeWithGenericDataInterfaceNode()
{
	TArray<UOptimusNode_AnimAttributeDataInterface*> DeprecatedAnimAtttributeNodes;
	
	for (UOptimusNode* Node : Nodes)
	{
		if (UOptimusNode_AnimAttributeDataInterface* DeprecatedAnimAtttributeNode = Cast<UOptimusNode_AnimAttributeDataInterface>(Node))
		{
			DeprecatedAnimAtttributeNodes.Add(DeprecatedAnimAtttributeNode);
		}
	}
	
	for (UOptimusNode_AnimAttributeDataInterface* Node : DeprecatedAnimAtttributeNodes)
	{
		// Save pin connections
		TArrayView<UOptimusNodePin* const> Pins = Node->GetPins();

		TMap<FName, TArray<UOptimusNodePin*>> ConnectedPinsMap;
		
		for (UOptimusNodePin* Pin : Pins)
		{
			ConnectedPinsMap.Add(Pin->GetFName(), Pin->GetConnectedPins());
		}

		FVector2D Position = Node->GetGraphPosition();
		// Assign to the graph temporarily
		UOptimusComputeDataInterface* NewDataInterfaceData = Node->GetDataInterface(this);
		
		RemoveNodeDirect(Node, false /* remove all links as well */);

		auto BootstrapNodeFunc = [NewDataInterfaceData, Position](UOptimusNode* InNode) -> bool
		{
			UOptimusNode_DataInterface* DataInterfaceNode = Cast<UOptimusNode_DataInterface>(InNode);
			DataInterfaceNode->SetDataInterfaceClass(UOptimusAnimAttributeDataInterface::StaticClass());

			// Replace the data interface
			Optimus::RenameObject(NewDataInterfaceData, nullptr, DataInterfaceNode);
			DataInterfaceNode->DataInterfaceData = NewDataInterfaceData;
			
			InNode->SetGraphPositionDirect(Position);	
			return true;
		};
		
		FName NodeName = Optimus::GetUniqueNameForScope(this, UOptimusNode_DataInterface::StaticClass()->GetFName());	
		UOptimusNode* NewNode = CreateNodeDirect(UOptimusNode_DataInterface::StaticClass(), NodeName, BootstrapNodeFunc);

		// Recover links
		for (UOptimusNodePin* Pin : NewNode->GetPins())
		{
			if (TArray<UOptimusNodePin*>* ConnectedPins = ConnectedPinsMap.Find(Pin->GetFName()))
			{
				for (UOptimusNodePin* ConnectedPin : *ConnectedPins)
				{
					if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
					{
						AddLinkDirect(ConnectedPin, Pin);
					}
					else if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
					{
						AddLinkDirect(Pin, ConnectedPin);
					}
				}
			}
		}
	}	
}


UOptimusActionStack* UOptimusNodeGraph::GetActionStack() const
{
	UOptimusDeformer *Deformer = Cast<UOptimusDeformer>(GetCollectionRoot());
	if (!Deformer)
	{
		return nullptr;
	}

	return Deformer->GetActionStack();
}


IOptimusNodeGraphCollectionOwner* UOptimusNodeGraph::GetCollectionOwner() const
{
	return Cast<IOptimusNodeGraphCollectionOwner>(GetOuter());
}


IOptimusNodeGraphCollectionOwner* UOptimusNodeGraph::GetCollectionRoot() const
{
	if (GetCollectionOwner())
	{
		return GetCollectionOwner()->GetCollectionRoot();
	}
	
	return nullptr;
}


FString UOptimusNodeGraph::GetCollectionPath() const
{
	if (GetCollectionOwner())
	{
		if (const FString ParentPath = GetCollectionOwner()->GetCollectionPath(); !ParentPath.IsEmpty())
		{
			return ParentPath + TEXT("/") + GetName();
		}
	}
	return GetName();
}


UOptimusNodeGraph* UOptimusNodeGraph::FindGraphByName(FName InGraphName) const
{
	for (UOptimusNodeGraph* Graph : GetGraphs())
	{
		if (Graph->GetFName() == InGraphName)
		{
			return Graph;
		}
	}

	return nullptr;
}

UOptimusNodeGraph* UOptimusNodeGraph::CreateGraphDirect(
	EOptimusNodeGraphType InType,
	FName InName,
	TOptional<int32> InInsertBefore
	)
{
	if (InType != EOptimusNodeGraphType::SubGraph)
	{
		UE_LOGF(LogOptimusCore, Warning, "Only subgraphs can be added to other graphs");
		return nullptr;
	}

	UOptimusNodeSubGraph* Graph = NewObject<UOptimusNodeSubGraph>(this, UOptimusNodeSubGraph::StaticClass(), InName, RF_Transactional);

	Graph->SetGraphType(EOptimusNodeGraphType::SubGraph);

	// The Entry/Return nodes will be added by the action.

	if (InInsertBefore.IsSet())
	{
		if (!AddGraphDirect(Graph, InInsertBefore.GetValue()))
		{
			Optimus::RemoveObject(Graph);
			return nullptr;
		}
	}
	
	return Graph;
}


bool UOptimusNodeGraph::AddGraphDirect(
	UOptimusNodeGraph* InGraph, 
	int32 InInsertBefore
	)
{
	if (InGraph == nullptr || InGraph->GetGraphType() != EOptimusNodeGraphType::SubGraph)
	{
		return false;
	}

	// If INDEX_NONE, insert at the end.
	if (InInsertBefore == INDEX_NONE)
	{
		InInsertBefore = SubGraphs.Num();
	}
	else
	{
		InInsertBefore = FMath::Clamp(InInsertBefore, 0, SubGraphs.Num());
	}

	SubGraphs.Insert(InGraph, InInsertBefore);

	GlobalNotify(EOptimusGlobalNotifyType::GraphAdded, InGraph);

	return true;
}


bool UOptimusNodeGraph::RemoveGraphDirect(
	UOptimusNodeGraph* InGraph,
	bool bInDeleteGraph
	)
{
	// Not ours?
	const int32 GraphIndex = SubGraphs.IndexOfByKey(InGraph);
	if (GraphIndex == INDEX_NONE)
	{
		return false;
	}

	if (InGraph->GetGraphType() == EOptimusNodeGraphType::Update)
	{
		return false;
	}

	SubGraphs.RemoveAt(GraphIndex);

	GlobalNotify(EOptimusGlobalNotifyType::GraphRemoved, InGraph);

	if (bInDeleteGraph)
	{
		// Un-parent this graph to a temporary storage and mark it for kill.
		Optimus::RemoveObject(InGraph);
	}

	return true;
}


bool UOptimusNodeGraph::MoveGraphDirect(
	UOptimusNodeGraph* InGraph,
	int32 InInsertBefore
	)
{
	return false;
}

bool UOptimusNodeGraph::RenameGraphDirect(UOptimusNodeGraph* InGraph, const FString& InNewName)
{
	if (Optimus::RenameObject(InGraph, *InNewName, nullptr))
	{
		if (InGraph->GetGraphType() == EOptimusNodeGraphType::SubGraph)
		{
			for (UOptimusNode* Node : GetAllNodes())
			{
				if (UOptimusNode_SubGraphReference* SubGraphNode = Cast<UOptimusNode_SubGraphReference>(Node))
				{
					if (SubGraphNode->GetReferencedSubGraph() == InGraph)
					{
						SubGraphNode->RefreshSerializedSubGraphName();
						Notify(EOptimusGraphNotifyType::NodeDisplayNameChanged, SubGraphNode);
						break;
					}
				}
			}
		}
		
		GlobalNotify(EOptimusGlobalNotifyType::GraphRenamed, InGraph);

		return true;
	}

	return false;
}


bool UOptimusNodeGraph::RenameGraph(
	UOptimusNodeGraph* InGraph,
	const FString& InNewName
	)
{
	// Not ours?
	const int32 GraphIndex = SubGraphs.IndexOfByKey(InGraph);
	if (GraphIndex == INDEX_NONE)
	{
		return false;
	}
		
	if (!IsValidUserGraphName(InNewName))
	{
		return false;
	}
	
	return GetActionStack()->RunAction<FOptimusNodeGraphAction_RenameGraph>(InGraph, FName(*InNewName));;
}

#if WITH_EDITOR
void UOptimusNodeGraph::SetViewLocationAndZoom(const FVector2D& InViewLocation, float InViewZoom)
{
	bViewLocationSet = true;
	ViewLocation = InViewLocation;
	ViewZoom = InViewZoom;
}

bool UOptimusNodeGraph::GetViewLocationAndZoom(FVector2D& OutViewLocation, float& OutViewZoom) const
{
	if (bViewLocationSet)
	{
		OutViewLocation = ViewLocation;
		OutViewZoom = ViewZoom;
	}
	return bViewLocationSet;
}

#endif // WITH_EDITOR


#undef LOCTEXT_NAMESPACE
