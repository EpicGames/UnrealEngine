// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodePair.h"
#include "OptimusNode.h"
#include "IOptimusNodePairProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodePair)



bool UOptimusNodePair::Contains(const UOptimusNode* InFirst, const UOptimusNode* InSecond) const
{
	return (First == InFirst && Second == InSecond) || (First == InSecond && Second == InFirst);	
}

bool UOptimusNodePair::Contains(const UOptimusNode* InNode) const
{
	return First == InNode || Second == InNode;	
}

UOptimusNode* UOptimusNodePair::GetNodeCounterpart(const UOptimusNode* InNode) const
{
	if (First == InNode)
	{
		return Second;
	}

	if (Second == InNode)
	{
		return First;
	}

	return nullptr;
}
