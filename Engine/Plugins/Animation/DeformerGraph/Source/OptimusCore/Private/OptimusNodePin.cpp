// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodePin.h"

#include "Actions/OptimusNodeActions.h"
#include "OptimusActionStack.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusObjectVersion.h"

#include "Containers/Queue.h"
#include "Misc/DefaultValueHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodePin)


#define LOCTEXT_NAMESPACE "OptimusDeformer"



UOptimusNodePin* UOptimusNodePin::GetParentPin()
{
	return Cast<UOptimusNodePin>(GetOuter());
}

const UOptimusNodePin* UOptimusNodePin::GetParentPin() const
{
	return Cast<const UOptimusNodePin>(GetOuter());
}

UOptimusNodePin* UOptimusNodePin::GetNextPin()
{
	const TArrayView<UOptimusNodePin* const> Pins = GetParentPin() ? GetParentPin()->GetSubPins() : GetOwningNode()->GetPins();
	if (Pins.Last() == this)
		return nullptr;
	
	return Pins[Pins.IndexOfByKey(this) + 1];
}

const UOptimusNodePin* UOptimusNodePin::GetNextPin() const
{
	const TArrayView<UOptimusNodePin* const> Pins = GetParentPin() ? GetParentPin()->GetSubPins() : GetOwningNode()->GetPins();
	if (Pins.Last() == this)
		return nullptr;
	
	return Pins[Pins.IndexOfByKey(this) + 1];
}


UOptimusNodePin* UOptimusNodePin::GetRootPin()
{
	UOptimusNodePin* CurrentPin = this;
	while (UOptimusNodePin* ParentPin = CurrentPin->GetParentPin())
	{
		CurrentPin = ParentPin;
	}
	return CurrentPin;
}

const UOptimusNodePin* UOptimusNodePin::GetRootPin() const
{
	const UOptimusNodePin* CurrentPin = this;
	while (const UOptimusNodePin* ParentPin = CurrentPin->GetParentPin())
	{
		CurrentPin = ParentPin;
	}
	return CurrentPin;
}


UOptimusNode* UOptimusNodePin::GetOwningNode() const
{
	const UOptimusNodePin* RootPin = GetRootPin();
	return Cast<UOptimusNode>(RootPin->GetOuter());
}


TArray<FName> UOptimusNodePin::GetPinNamePath() const
{
	TArray<const UOptimusNodePin*> Pins;
	Pins.Reserve(4);
	const UOptimusNodePin* CurrentPin = this;
	while (CurrentPin)
	{
		Pins.Add(CurrentPin);
		CurrentPin = CurrentPin->GetParentPin();
	}

	TArray<FName> Path;
	Path.Reserve(Pins.Num());

	for (int32 i = Pins.Num(); i-- > 0; /**/)
	{
		Path.Add(Pins[i]->GetFName());
	}

	return Path;
}


FName UOptimusNodePin::GetUniqueName() const
{
	return *FString::JoinBy(GetPinNamePath(), TEXT("."), [](const FName& N) { return N.ToString(); });
}


FText UOptimusNodePin::GetDisplayName() const
{
	const FOptimusDataTypeHandle BoolType = FOptimusDataTypeRegistry::Get().FindType(*FBoolProperty::StaticClass());
	const bool bIsBool = GetDataType() == BoolType;

	const FString PinName = !DisplayName.IsNone() ? DisplayName.ToString() : GetName();
	return FText::FromString(FName::NameToDisplayString(PinName, bIsBool));
}


FText UOptimusNodePin::GetTooltipText() const
{
	// FIXME: We probably want a specialized widget for this in SOptimusEditorGraphNode::MakeTableRowWidget
	if (bIsGroupingPin)
	{
		return FText::GetEmpty();
	}

	if (DataDomain.IsSingleton())
	{
		if (DataType->ShaderValueType.IsValid())
		{
			return FText::FormatOrdered(LOCTEXT("OptimusNodePin_Tooltip_ShaderValue", "Name:\t{0}\nType:\t{1} ({2})\nStorage:\tValue"),
				FText::FromString(GetName()), DataType->DisplayName, FText::FromString(DataType->ShaderValueType->ToString()) );
		}
		else
		{
			return FText::FormatOrdered(LOCTEXT("OptimusNodePin_Tooltip_Value", "Name:\t{0}\nType:\t{1}\nStorage:\tValue"),
				FText::FromString(GetName()), DataType->DisplayName);
		}
	}
	else
	{
		return FText::FormatOrdered(LOCTEXT("OptimusNodePin_Tooltip_Resource", "Name:\t{0}\nType:\t{1} ({2})\nStorage:\tResource\nDomain:\t{3}"),
			FText::FromString(GetName()), DataType->DisplayName, FText::FromString(DataType->ShaderValueType->ToString()),
			FText::FromString(DataDomain.GetDisplayName()));
	}
}


FString UOptimusNodePin::GetPinPath() const
{
	return UOptimusNodeGraph::ConstructPath(
		GetOwningNode()->GetOwningGraph()->GetCollectionPath(),
		GetOwningNode()->GetName(),
		GetUniqueName().ToString());
}


TArray<FName> UOptimusNodePin::GetPinNamePathFromString(const FStringView InPinPathString)
{
	// FIXME: This should really become a part of FStringView, or a shared algorithm.
	TArray<FStringView, TInlineAllocator<4>> PinPathParts;
	FStringView PinPathView(InPinPathString);

	int32 Index = INDEX_NONE;
	while(PinPathView.FindChar(TCHAR('.'), Index))
	{
		if (Index > 0)
		{
			PinPathParts.Add(PinPathView.Mid(0, Index));
		}
		PinPathView = PinPathView.Mid(Index + 1);
	}
	if (!PinPathView.IsEmpty())
	{
		PinPathParts.Add(PinPathView);
	}

	TArray<FName> PinPath;
	PinPath.Reserve(PinPathParts.Num());
	for (FStringView PinPathPart : PinPathParts)
	{
		// Don't add names, just return a NAME_None.
		PinPath.Emplace(PinPathPart, FNAME_Find);
	}
	return PinPath;
}


TSet<UOptimusComponentSourceBinding*> UOptimusNodePin::GetComponentSourceBindings(const FOptimusPinTraversalContext& InContext) const
{
	return GetOwningNode()->GetOwningGraph()->GetComponentSourceBindingsForPin(this, InContext);
}

TSet<UOptimusComponentSourceBinding*> UOptimusNodePin::GetComponentSourceBindingsRecursively(const FOptimusPinTraversalContext& InContext) const
{
	TArray<UOptimusNodePin*> PinsToConsider = GetSubPinsRecursively(true);

	TSet<UOptimusComponentSourceBinding*> Bindings;

	for (const UOptimusNodePin* Pin : PinsToConsider)
	{
		Bindings.Append(Pin->GetComponentSourceBindings(InContext));
	}
	
	return Bindings;
}

bool UOptimusNodePin::IsMutable(const FOptimusPinTraversalContext& InContext) const
{
	return GetOwningNode()->GetOwningGraph()->IsPinMutable(this, InContext);
}


FProperty* UOptimusNodePin::GetPropertyFromPin() const
{
	UStruct *ScopeStruct = GetOwningNode()->GetClass();
	TArray<FName> NamePath = GetPinNamePath();

	FProperty* Property = nullptr;
	for (int32 Index = 0; Index < NamePath.Num(); Index++)
	{
		Property = ScopeStruct->FindPropertyByName(NamePath[Index]);
		if (!Property)
		{
			return nullptr;
		}

		if (Index == (NamePath.Num() - 1))
		{
			break;
		}

		FStructProperty *StructProperty = CastField<FStructProperty>(Property);
		if (!StructProperty)
		{
			return nullptr;
		}

		ScopeStruct = StructProperty->Struct;
	}

	return Property;
}

// Returns a pointer to the property represented by this pin. If the function returns nullptr
// then there's no editable property here. Accounts for nested pins.
uint8* UOptimusNodePin::GetPropertyValuePtr() const
{
	// Collect properties up the chain.
	TArray<const FProperty*> PropertyHierarchy;
	PropertyHierarchy.Reserve(4);
	const UOptimusNodePin* CurrentPin = this;
	while (CurrentPin)
	{
		const FProperty *Property = CurrentPin->GetPropertyFromPin();
		if (!Property)
		{
			return nullptr;
		}

		PropertyHierarchy.Add(Property);
		CurrentPin = CurrentPin->GetParentPin();
	}
	
	UObject* NodeObject = GetOwningNode();
	uint8 *NodeData = nullptr;
	for (int32 Index = PropertyHierarchy.Num(); Index-- > 0; /**/)
	{
		const FProperty* Property = PropertyHierarchy[Index];
		if (NodeData)
		{
			NodeData = Property->ContainerPtrToValuePtr<uint8>(NodeData);
		}
		else
		{
			NodeData = Property->ContainerPtrToValuePtr<uint8>(NodeObject);
		}
	}
	return NodeData;
}


FString UOptimusNodePin::GetValueAsString() const
{
	FString ValueString;

	// We can have pins with no underlying properties (e.g. Get/Set Resource nodes).
	// FIXME: Change to support nested properties.
	const FProperty *Property = GetPropertyFromPin();
	const uint8 *ValueData = GetPropertyValuePtr();
	if (Property && ValueData)
	{
		Property->ExportTextItem_Direct(ValueString, ValueData, nullptr, GetOwningNode(), PPF_None);
	}

	return ValueString;
}



bool UOptimusNodePin::SetValueFromString(const FString& InStringValue)
{
	return GetActionStack()->RunAction<FOptimusNodeAction_SetPinValue>(this, InStringValue);
}


bool UOptimusNodePin::SetValueFromStringDirect(const FString& InStringValue)
{
	FProperty* Property = GetPropertyFromPin();
	uint8* ValueData = GetPropertyValuePtr();

	bool bSuccess = false;

	if (ensure(Property) && ValueData)
	{
		UOptimusNode* Node = GetOwningNode();

#if WITH_EDITOR
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);
		Node->PreEditChange(PropertyChain);
#endif

		// FIXME: We need a way to sanitize the input. Trying and failing is not good, since
		// it's unknown whether this may leave the property in an indeterminate state.
		bSuccess = Property->ImportText_Direct(*InStringValue, ValueData, Node, PPF_None) != nullptr;

#if WITH_EDITOR
		// We notify that the value change occurred, whether that's true or not. This way
		// the graph pin value sync will ensure that if an invalid value was entered, it will
		// get reverted back to the true value. 
		FPropertyChangedEvent ChangedEvent(GetRootPin()->GetPropertyFromPin(), EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(PropertyChain, ChangedEvent);
		Node->PostEditChangeChainProperty(ChangedChainEvent);
#endif
		
		Notify(EOptimusGraphNotifyType::PinValueChanged);
	}

	return bSuccess;
}

bool UOptimusNodePin::VerifyValue(const FString& InStringValue) const
{
	const FProperty *Property = GetPropertyFromPin();
	if (!Property)
	{
		// If there's no property, then all values is invalid.
		return false;
	}

	if (CastField<FBoolProperty>(Property))
	{
		// Snarfed from FBoolProperty::ImportText_Internal
		const FCoreTexts& CoreTexts = FCoreTexts::Get();
		return InStringValue == TEXT("1") || 
			   InStringValue == TEXT("True") || 
			   InStringValue == *CoreTexts.True.ToString() || 
			   InStringValue == TEXT("Yes") || 
			   InStringValue == *CoreTexts.Yes.ToString() ||
			   InStringValue == TEXT("0") || 
			   InStringValue == TEXT("False") || 
			   InStringValue == *CoreTexts.False.ToString() || 
			   InStringValue == TEXT("No") || 
			   InStringValue == *CoreTexts.No.ToString();
	}
	else if (CastField<FIntProperty>(Property))
	{
		return FDefaultValueHelper::IsStringValidInteger(InStringValue);
	}
	else if (CastField<FFloatProperty>(Property))
	{
		return FDefaultValueHelper::IsStringValidFloat(InStringValue);
	}
	else if (CastField<FObjectProperty>(Property))
	{
		// FIXME: Verify class + pointer.
		return true;
	}
	else
	{
		return false;
	}
}


TArray<UOptimusNodePin*> UOptimusNodePin::GetSubPinsRecursively(
	bool bInIncludeThisPin
	) const
{
	TArray<UOptimusNodePin*> CollectedPins;
	TQueue<UOptimusNodePin*> PinQueue;

	PinQueue.Enqueue(const_cast<UOptimusNodePin*>(this));
	if (bInIncludeThisPin)
	{
		CollectedPins.Add(const_cast<UOptimusNodePin*>(this));
	}
	UOptimusNodePin *WorkingPin = nullptr;
	while (PinQueue.Dequeue(WorkingPin))
	{
		for (UOptimusNodePin *SubPin: WorkingPin->SubPins)
		{
			CollectedPins.Add(SubPin);
			if (!SubPin->SubPins.IsEmpty())
			{
				PinQueue.Enqueue(SubPin);
			}
		}
	}
	return CollectedPins;
}


TArray<UOptimusNodePin*> UOptimusNodePin::GetConnectedPins() const
{
	UOptimusNodeGraph* Graph = GetOwningNode()->GetOwningGraph();
	if (ensure(Graph != nullptr))
	{
		return Graph->GetConnectedPins(this);
	}
	return {};
}


TArray<FOptimusRoutedNodePin> UOptimusNodePin::GetConnectedPinsWithRouting(
	const FOptimusPinTraversalContext& InContext
	) const
{
	return GetOwningNode()->GetOwningGraph()->GetConnectedPinsWithRouting(this, InContext);
}


bool UOptimusNodePin::CanCannect(const UOptimusNodePin* InOtherPin, FString* OutReason) const
{
	if (!ensure(InOtherPin))
	{
		if (OutReason)
		{
			*OutReason = TEXT("No pin given.");
		}
		return false;
	}

	// Check connection at the node level
	if (!GetOwningNode()->CanConnectPinToPin(*this, *InOtherPin, OutReason))
	{
		return false;
	}

	// Check connection at the node level on the other node
	if (!InOtherPin->GetOwningNode()->CanConnectPinToPin(*InOtherPin, *this, OutReason))
	{
		return false;
	}

	if (IsGroupingPin() || InOtherPin->IsGroupingPin())
	{
		// Allow direct connection from component source to group pin
		FOptimusDataTypeHandle ComponentSourceType = FOptimusDataTypeRegistry::Get().FindType(*UOptimusComponentSourceBinding::StaticClass());

		if (DataType == ComponentSourceType || InOtherPin->DataType == ComponentSourceType)
		{
			return true;
		}

		return false;
	}
	
	const UOptimusNodePin *OutputPin = Direction == EOptimusNodePinDirection::Output ? this : InOtherPin;
	const UOptimusNodePin* InputPin = Direction == EOptimusNodePinDirection::Input ? this : InOtherPin;

	// Check for incompatible types.
	if (!((DataType->ShaderValueType.IsValid() && InOtherPin->DataType->ShaderValueType.IsValid() &&
		  DataType->ShaderValueType == InOtherPin->DataType->ShaderValueType) ||
		 DataType == InOtherPin->DataType))
	{
		// TBD: Automatic conversion.

		if (OutReason)
		{
			*OutReason = FString::Printf(TEXT("Incompatible pin types (%s vs %s)."),
				*OutputPin->GetDataType()->DisplayName.ToString(),
				*InputPin->GetDataType()->DisplayName.ToString());
		}
		return false;
	}


	return FOptimusDataDomain::AreCompatible(OutputPin->DataDomain, InputPin->DataDomain, OutReason);
}


void UOptimusNodePin::SetIsExpanded(bool bInIsExpanded)
{
	// We store the expansion state on the node, since we don't store the pin data when doing
	// delete/undo.
	GetOwningNode()->SetPinExpanded(this, bInIsExpanded);
}


bool UOptimusNodePin::GetIsExpanded() const
{
	return GetOwningNode()->GetPinExpanded(this);
}


void UOptimusNodePin::PostLoad()
{
	UObject::PostLoad();

	for (UOptimusNodePin* SubPin : SubPins)
	{
		SubPin->ConditionalPostLoad();
	}
	
	// If the storage was marked as a value, the domain should now be a singleton.
	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::DataDomainExpansion)
	{
		DataDomain.BackCompFixupLevels();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (StorageType_DEPRECATED == EOptimusNodePinStorageType::Value)
		{
			DataDomain = FOptimusDataDomain();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}


void UOptimusNodePin::InitializeWithData(
    EOptimusNodePinDirection InDirection,
    FOptimusDataDomain InDataDomain,
    FOptimusDataTypeRef InDataTypeRef
	)
{
	Direction = InDirection;
	DataDomain = InDataDomain;
	DataType = InDataTypeRef;
}


void UOptimusNodePin::InitializeWithGrouping(
	EOptimusNodePinDirection InDirection
	)
{
	Direction = InDirection;
	bIsGroupingPin = true;
}


void UOptimusNodePin::AddSubPin(
	UOptimusNodePin* InSubPin,
	UOptimusNodePin* InBeforePin
	)
{
	int32 Index = SubPins.Num();
	if (InBeforePin && ensure(SubPins.IndexOfByKey(InBeforePin) != INDEX_NONE))
	{
		Index = SubPins.IndexOfByKey(InBeforePin); 
	}
	SubPins.Insert(InSubPin, Index);
}


void UOptimusNodePin::ClearSubPins()
{
	for (UOptimusNodePin* Pin: SubPins)
	{
		// Consign them to oblivion.
		Optimus::RemoveObject(Pin);
	}
	SubPins.Reset();
}


bool UOptimusNodePin::SetDataType(FOptimusDataTypeRef InDataType)
{
	FOptimusDataTypeHandle DataTypeHandle = InDataType.Resolve();
	if (!DataTypeHandle)
	{
		return false;
	}

	if (DataTypeHandle == GetDataType())
	{
		return false;
	}

	// Make sure it's compatible with the storage type.
	if (!DataDomain.IsSingleton() &&
		!EnumHasAllFlags(DataTypeHandle->UsageFlags, EOptimusDataTypeUsageFlags::Resource))
	{
		return false;
	}

	DataType = InDataType;

	return true;
}


bool UOptimusNodePin::SetName(FName InName)
{
	if (GetFName() == InName)
	{
		return false;
	}

	Rename(*InName.ToString(), nullptr);

	Notify(EOptimusGraphNotifyType::PinRenamed);

	return true;
}

bool UOptimusNodePin::SetDisplayName(const FName& InDisplayName)
{
	if (DisplayName == InDisplayName)
	{
		return false;
	}
	DisplayName = InDisplayName;
	
	return true;
}

void UOptimusNodePin::Notify(EOptimusGraphNotifyType InNotifyType) const
{
	const UOptimusNodeGraph *Graph = GetOwningNode()->GetOwningGraph();

	Graph->Notify(InNotifyType, const_cast<UOptimusNodePin*>(this));
}


UOptimusActionStack* UOptimusNodePin::GetActionStack() const
{
	return GetOwningNode()->GetActionStack();
}

#undef LOCTEXT_NAMESPACE
