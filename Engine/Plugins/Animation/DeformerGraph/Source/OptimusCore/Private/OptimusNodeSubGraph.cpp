// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNodeSubGraph.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusHelpers.h"
#include "ComponentSources/OptimusSkeletalMeshComponentSource.h"
#include "Nodes/OptimusNode_GraphTerminal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNodeSubGraph)

FName UOptimusNodeSubGraph::GraphDefaultComponentPinName = TEXT("Default Component Binding");
FName UOptimusNodeSubGraph::InputBindingsPropertyName = GET_MEMBER_NAME_CHECKED(UOptimusNodeSubGraph, InputBindings);
FName UOptimusNodeSubGraph::OutputBindingsPropertyName = GET_MEMBER_NAME_CHECKED(UOptimusNodeSubGraph, OutputBindings);


UOptimusNodeSubGraph::UOptimusNodeSubGraph()
{
}

FString UOptimusNodeSubGraph::GetBindingDeclaration(FName BindingName) const
{
	return {};
}

bool UOptimusNodeSubGraph::GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const
{
	return false;
}

bool UOptimusNodeSubGraph::GetBindingSupportReadCheckBoxVisibility(FName BindingName) const
{
	return false;
}

EOptimusDataTypeUsageFlags UOptimusNodeSubGraph::GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const
{
	if (InDataDomain.IsSingleton())
	{
		return EOptimusDataTypeUsageFlags::PinType;
	}

	return EOptimusDataTypeUsageFlags::Resource;
}

UOptimusComponentSourceBinding* UOptimusNodeSubGraph::GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const
{
	UOptimusNode_GraphTerminal* EntryNode = GetTerminalNode(EOptimusTerminalType::Entry);
	if (!ensure(EntryNode))
	{
		return nullptr;
	}

	return EntryNode->GetDefaultComponentBinding(InTraversalContext);
}

UOptimusNode_GraphTerminal* UOptimusNodeSubGraph::GetTerminalNode(EOptimusTerminalType InTerminalType) const
{
	for (UOptimusNode* Node : GetAllNodes())
	{
		if (UOptimusNode_GraphTerminal* TerminalNode = Cast<UOptimusNode_GraphTerminal>(Node))
		{
			if (TerminalNode->TerminalType == InTerminalType)
			{
				return TerminalNode;
			}
		}
	}
	
	return nullptr;
}

#if WITH_EDITOR
void UOptimusNodeSubGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
	{
		if (CastField<FArrayProperty>(PropertyChangedEvent.Property) &&
			PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.GetPropertyName().ToString()) == INDEX_NONE)
		{
			PropertyArrayPasted(PropertyChangedEvent);
		}
		else
		{
			PropertyValueChanged(PropertyChangedEvent);
		}
		
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayAdd)
	{
		PropertyArrayItemAdded(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayRemove)
	{
		PropertyArrayItemRemoved(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayClear)
	{
		PropertyArrayCleared(PropertyChangedEvent);
	}
	else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ArrayMove)
	{
		PropertyArrayItemMoved(PropertyChangedEvent);
	}
}
#endif


void UOptimusNodeSubGraph::SanitizeBinding(FOptimusParameterBinding& InOutBinding, FName InOldName, bool bInAllowParameter)
{
	InOutBinding.Name = GetSanitizedBindingName(InOutBinding.Name, InOldName);
	
	if (!InOutBinding.DataType.IsValid())
	{
		InOutBinding.DataType = FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass());
	}

	if (!bInAllowParameter && InOutBinding.DataDomain.IsSingleton())
	{
		InOutBinding.DataDomain = FOptimusDataDomain(TArray<FName>({UOptimusSkeletalMeshComponentSource::Domains::Vertex}));
	}
}

FName UOptimusNodeSubGraph::GetSanitizedBindingName(FName InNewName, FName InOldName)
{
	FName Name = InNewName;
	
	if (Name == NAME_None)
	{
		Name = TEXT("EmptyName");
	}
	
	Name = Optimus::GetSanitizedNameForHlsl(Name);

	if (Name != InOldName)
	{
		const UOptimusNode_GraphTerminal* EntryNode = GetTerminalNode(EOptimusTerminalType::Entry);
		const UOptimusNode_GraphTerminal* ReturnNode = GetTerminalNode(EOptimusTerminalType::Return);

		Name = UOptimusNode::GetAvailablePinNameStable(EntryNode, Name);
		Name = UOptimusNode::GetAvailablePinNameStable(ReturnNode, Name);
	}
	
	return Name;
}

#if WITH_EDITOR
void UOptimusNodeSubGraph::PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	bool bAllowParameter = false;
	if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsPropertyName)
	{
		BindingArrayPtr = &InputBindings;
		bAllowParameter = true;
		
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsPropertyName)
	{
		BindingArrayPtr = &OutputBindings;
		bAllowParameter = true;
	}

	if (BindingArrayPtr)
	{
		for (FOptimusParameterBinding& Binding : *BindingArrayPtr)
		{
			SanitizeBinding(Binding, NAME_None, bAllowParameter);
		}
		OnBindingArrayPastedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
	}
}

void UOptimusNodeSubGraph::PropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UOptimusNode_GraphTerminal* EntryNode = GetTerminalNode(EOptimusTerminalType::Entry);
	UOptimusNode_GraphTerminal* ReturnNode = GetTerminalNode(EOptimusTerminalType::Return);
	
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;
	TArray<UOptimusNodePin*> BindingPins;
	bool bAllowParameter = false;
	if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsPropertyName)
	{
		BindingArrayPtr = &InputBindings;
		bAllowParameter = true;
		BindingPins = EntryNode->GetBindingPins();
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsPropertyName)
	{
		BindingArrayPtr = &OutputBindings;
		bAllowParameter = false;
		BindingPins = ReturnNode->GetBindingPins();
	}
	
	if (BindingArrayPtr)
	{
		check(BindingArrayPtr->Num() == BindingPins.Num());

		for (int32 Index = 0; Index < BindingArrayPtr->Num(); Index++)
		{
			FOptimusParameterBinding& Binding = (*BindingArrayPtr)[Index];
			SanitizeBinding(Binding, BindingPins[Index]->GetFName(), bAllowParameter);
		}
		
		OnBindingValueChangedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
	}	
}

void UOptimusNodeSubGraph::PropertyArrayItemAdded(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	FOptimusParameterBindingArray* BindingArrayPtr = nullptr;

	bool bAllowParameter = false;
	if (InPropertyChangedEvent.GetMemberPropertyName() == InputBindingsPropertyName)
	{
		BindingArrayPtr = &InputBindings;
		bAllowParameter = true;
	}
	else if (InPropertyChangedEvent.GetMemberPropertyName() == OutputBindingsPropertyName)
	{
		BindingArrayPtr = &OutputBindings;
		bAllowParameter = false;
	}

	if (BindingArrayPtr)
	{
		SanitizeBinding(BindingArrayPtr->Last(), NAME_None, bAllowParameter);
		OnBindingArrayItemAddedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
	}
}

void UOptimusNodeSubGraph::PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	OnBindingArrayItemRemovedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
}

void UOptimusNodeSubGraph::PropertyArrayCleared(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	OnBindingArrayClearedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
}

void UOptimusNodeSubGraph::PropertyArrayItemMoved(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	OnBindingArrayItemMovedDelegate.Broadcast(InPropertyChangedEvent.GetMemberPropertyName());
}
#endif
