// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNode_Comment.h"

#include "OptimusActionStack.h"
#include "OptimusNodeGraph.h"
#include "Actions/OptimusNodeActions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusNode_Comment)

#define LOCTEXT_NAMESPACE "OptimusNode_Comment"

#if WITH_EDITOR
void UOptimusNode_Comment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnPropertyChangedDelegate.ExecuteIfBound(this);
}
#endif

bool UOptimusNode_Comment::SetSize(const UE::Slate::FDeprecateVector2DParameter& InSize)
{
	return GetActionStack()->RunAction<FOptimusCommentNodeAction_ResizeNode>(this, FVector2D(InSize));
}

bool UOptimusNode_Comment::SetSizeDirect(const FVector2f& InNewSize)
{
	if (Size == InNewSize || InNewSize.ContainsNaN())
	{
		return false;
	}
	
	Size = InNewSize;
	
	OnPropertyChangedDelegate.ExecuteIfBound(this);
	
	return true;	
}

void UOptimusNode_Comment::SetComment(const FString& InNewComment)
{
	// Caller should avoid create a transaction if not changing anything
	if (ensure(Comment != InNewComment))
	{
		Modify();
		Comment = InNewComment;
		OnPropertyChangedDelegate.ExecuteIfBound(this);
	}
}

#undef LOCTEXT_NAMESPACE
