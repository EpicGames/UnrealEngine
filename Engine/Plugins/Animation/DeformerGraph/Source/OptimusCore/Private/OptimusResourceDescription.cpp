// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusResourceDescription.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "DataInterfaces/OptimusDataInterfaceRawBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusResourceDescription)


UOptimusDeformer* UOptimusResourceDescription::GetOwningDeformer() const
{
	const UOptimusResourceContainer* Container = CastChecked<UOptimusResourceContainer>(GetOuter());
	return Container ? CastChecked<UOptimusDeformer>(Container->GetOuter()) : nullptr;
}


int32 UOptimusResourceDescription::GetIndex() const
{
	const UOptimusResourceContainer* Container = CastChecked<UOptimusResourceContainer>(GetOuter());
	return Container->Descriptions.IndexOfByKey(this);
}


void UOptimusResourceDescription::PostLoad()
{
	Super::PostLoad();
	
	if (DataInterface)
	{
		// Ensure the DI is in sync with this resource description.
		DataInterface->DataDomain = DataDomain;
		DataInterface->ComponentSourceBinding = ComponentBinding;
	}
	
	// 64-bit float data type is not supported for resources although they were allowed before. Do an in-place upgrade here. 
	const FOptimusDataTypeHandle FloatDataType = FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass());
	const FOptimusDataTypeHandle DoubleDataType = FOptimusDataTypeRegistry::Get().FindType(*FDoubleProperty::StaticClass());
	
	if (DataType == DoubleDataType)
	{
		DataType = FloatDataType;
		(void)MarkPackageDirty();
	}
}


#if WITH_EDITOR
void UOptimusResourceDescription::PostEditChangeProperty(
	FPropertyChangedEvent& PropertyChangedEvent
	)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusResourceDescription, ResourceName))
	{
		UOptimusDeformer *Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Rename the object itself and update the nodes. A lot of this is covered by
			// UOptimusDeformer::RenameResource but since we're inside of a transaction, which
			// has already taken a snapshot of this object, we have to do the remaining 
			// operations on this object under the transaction scope.
			ResourceName = Optimus::GetUniqueNameForScope(GetOuter(), ResourceName);
			Rename(*ResourceName.ToString(), nullptr);
			
			constexpr bool bForceChange = true;
			Deformer->RenameResource(this, ResourceName, bForceChange);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataTypeRef, TypeName))
	{
		UOptimusDeformer *Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Set the resource type again, so that we can remove any links that are now
			// type-incompatible.
			constexpr bool bForceChange = true;
			Deformer->SetResourceDataType(this, DataType, bForceChange);
			if (DataInterface)
			{
				// TBD: The data interface should be referring back to us instead.
				DataInterface->ValueType = DataType->ShaderValueType;
			}
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusResourceDescription, ComponentBinding))
	{
		if (IsValidComponentBinding())
		{
			UOptimusDeformer *Deformer = GetOwningDeformer();
			if (ensure(Deformer))
			{
				// Set a default domain based on the component binding that got set.
				Deformer->SetResourceDataDomain(this, GetDataDomainFromComponentBinding());

				if (DataInterface)
				{
					// TBD: The data interface should be referring back to us instead.
					DataInterface->DataDomain = DataDomain;
					DataInterface->ComponentSourceBinding = ComponentBinding;
				}
			}
		}
	}
	else if (
		PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusResourceDescription, DataDomain) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataDomain, Type) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataDomain, DimensionNames) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataDomain, Expression) )
	{
		UOptimusDeformer *Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Set the resource data domain again, so that we can remove any links that are now
			// domain-incompatible.
			constexpr bool bForceChange = true;
			Deformer->SetResourceDataDomain(this, DataDomain, bForceChange);
			
			if (DataInterface)
			{
				// TBD: The data interface should be referring back to us instead.
				DataInterface->DataDomain = DataDomain;
			}
		}
	}
}


void UOptimusResourceDescription::PreEditUndo()
{
	Super::PreEditUndo();

	ResourceNameForUndo = ResourceName;
}


void UOptimusResourceDescription::PostEditUndo()
{
	Super::PostEditUndo();

	if (ResourceNameForUndo != ResourceName)
	{
		const UOptimusDeformer *Deformer = GetOwningDeformer();
		Deformer->Notify(EOptimusGlobalNotifyType::ResourceRenamed, this);
	}
}

bool UOptimusResourceDescription::IsValidComponentBinding() const
{
	return ComponentBinding.IsValid() && !ComponentBinding->GetComponentSource()->GetExecutionDomains().IsEmpty();
}

FOptimusDataDomain UOptimusResourceDescription::GetDataDomainFromComponentBinding() const
{
	return FOptimusDataDomain(TArray<FName>{ComponentBinding->GetComponentSource()->GetExecutionDomains()[0]});
}

#endif
