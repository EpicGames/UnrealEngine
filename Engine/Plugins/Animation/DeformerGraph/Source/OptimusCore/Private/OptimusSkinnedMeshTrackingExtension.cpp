// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSkinnedMeshTrackingExtension.h"

#include "ComputeWorkerInterface.h"
#include "ScenePrivate.h"
#include "ScenePrimitiveUpdates.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"

IMPLEMENT_SCENE_EXTENSION(FOptimusSkinnedMeshTrackingExtension);

// ---------------------------------------------------------------------------
// Updater
// ---------------------------------------------------------------------------

class FOptimusSkinnedMeshTrackingExtension::FUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FOptimusSkinnedMeshTrackingExtension);

public:
	explicit FUpdater(FOptimusSkinnedMeshTrackingExtension& InExtension)
		: Extension(InExtension)
	{
	}

	virtual void PreSceneUpdate(
		FRDGBuilder& GraphBuilder,
		const FScenePreUpdateChangeSet& ChangeSet) override
	{
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
		{
			Extension.ComponentDataMap.Remove(PrimitiveSceneInfo->PrimitiveComponentId);
		}
	}

private:
	FOptimusSkinnedMeshTrackingExtension& Extension;
};

ISceneExtensionUpdater* FOptimusSkinnedMeshTrackingExtension::CreateUpdater()
{
	return new FUpdater(*this);
}

// ---------------------------------------------------------------------------
// Accessors
// ---------------------------------------------------------------------------

void FOptimusSkinnedMeshTrackingExtension::MarkDeformed(
	FRDGBuilder& GraphBuilder, FSceneInterface* SceneInterface, FPrimitiveComponentId ComponentId, FSkeletalMeshObject* MeshObject,
	EMeshDeformerOutputBuffer OutputBuffers, FName ExecutionGroup)
{
	if (!SceneInterface || !MeshObject || !ComponentId.IsValid())
	{
		return;
	}

	FOptimusSkinnedMeshTrackingExtension* Extension = Get(SceneInterface);
	if (!Extension)
	{
		return;
	}

	FComponentData& Data = Extension->ComponentDataMap.FindOrAdd(ComponentId);

	// Start fresh when it is a new frame
	if (Data.DeformedFrame != GFrameNumberRenderThread)
	{
		Data.DeformedFrame = GFrameNumberRenderThread;
		
		const int32 LodIndex = MeshObject->GetLOD();

		// Avoid using previous position from the new lod during LOD transition, 
		// because it likely contains position data too outdated to be used to 
		// compute motion vectors for the current frame
		bool bInvalidatePreviousPosition = false;
		if (LodIndex != Data.LastLodIndex)
		{
			Data.LastLodIndex = LodIndex;
			bInvalidatePreviousPosition = true;
		}

		// Add a pass to override vertex factory buffers with buffers allocated by Optimus
		FSkeletalMeshDeformerHelpers::UpdateVertexFactoryBufferOverrides(
			GraphBuilder, MeshObject, LodIndex, bInvalidatePreviousPosition);
		
		// Restart output buffer tracking
		Data.EndOfFrameUpdateOutputBuffers = EMeshDeformerOutputBuffer::None;
		Data.BeginInitViewsOutputBuffers = EMeshDeformerOutputBuffer::None;
	}

	if (ExecutionGroup == ComputeTaskExecutionGroup::EndOfFrameUpdate)
	{
		Data.EndOfFrameUpdateOutputBuffers |= OutputBuffers;
	}
	else if (ExecutionGroup == ComputeTaskExecutionGroup::BeginInitViews)
	{
		Data.BeginInitViewsOutputBuffers |= OutputBuffers;
	}
}

EMeshDeformerOutputBuffer FOptimusSkinnedMeshTrackingExtension::GetReadableDeformedBuffers(
	FSceneInterface* SceneInterface, FPrimitiveComponentId ComponentId,
	FName ReaderExecutionGroup, bool bIsPrimaryComponent)
{
	FOptimusSkinnedMeshTrackingExtension* Extension = Get(SceneInterface);
	if (!Extension)
	{
		return EMeshDeformerOutputBuffer::None;
	}

	const FComponentData* Data = Extension->ComponentDataMap.Find(ComponentId);
	if (!Data || Data->DeformedFrame != GFrameNumberRenderThread)
	{
		// Component was not deformed by Optimus this frame
		return EMeshDeformerOutputBuffer::None;
	}

	EMeshDeformerOutputBuffer Result = EMeshDeformerOutputBuffer::None;

	if (bIsPrimaryComponent)
	{
		// Same-component reads are serialized via sort priority — all buffers are safe
		Result = Data->EndOfFrameUpdateOutputBuffers | Data->BeginInitViewsOutputBuffers;
	}
	else
	{
		// Secondary read from BeginInitViews can read EndOfFrameUpdate outputs (earlier group)
		if (ReaderExecutionGroup == ComputeTaskExecutionGroup::BeginInitViews)
		{
			Result = Data->EndOfFrameUpdateOutputBuffers;
		}
	}
	
	return Result;
}

FOptimusSkinnedMeshTrackingExtension* FOptimusSkinnedMeshTrackingExtension::Get(FSceneInterface* SceneInterface)
{
	if (!SceneInterface)
	{
		return nullptr;
	}

	FScene* RenderScene = SceneInterface->GetRenderScene();
	if (!RenderScene)
	{
		return nullptr;
	}

	return RenderScene->GetExtensionPtr<FOptimusSkinnedMeshTrackingExtension>();
}

