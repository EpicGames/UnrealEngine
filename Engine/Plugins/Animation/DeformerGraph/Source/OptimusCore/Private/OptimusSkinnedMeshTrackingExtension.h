// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MeshDeformerInstance.h"
#include "PrimitiveComponentId.h"
#include "SceneExtensions.h"

class FRDGBuilder;
class FScene;
class FSceneInterface;
class FSkeletalMeshObject;

/**
 * Scene extension that tracks per-component deformer state on the render thread.
 * Entries are created on demand when MarkDeformed is called and removed via
 * PreSceneUpdate when primitives leave the scene.
 */
class FOptimusSkinnedMeshTrackingExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(OPTIMUSCORE_API, FOptimusSkinnedMeshTrackingExtension);

public:
	/** Per-component tracking data. Packed to 2 DWords (8 bytes). */
	struct FComponentData
	{
		int8 LastLodIndex = INDEX_NONE;
		EMeshDeformerOutputBuffer EndOfFrameUpdateOutputBuffers = EMeshDeformerOutputBuffer::None;
		EMeshDeformerOutputBuffer BeginInitViewsOutputBuffers = EMeshDeformerOutputBuffer::None;
		uint32 DeformedFrame = 0;
	};

	using ISceneExtension::ISceneExtension;

	//~ ISceneExtension interface
	virtual ISceneExtensionUpdater* CreateUpdater() override;

	/**
	 * RT: Mark a component as deformed by Optimus. 
	 * Adds a RDG pass to make sure the passthrough vertex factory bind to buffers allocated by Optimus.
	 * Also keeps track of the deformer geometry buffers that Optimus writes to during the dispatch of each execution group
	 */
	static void MarkDeformed(
		FRDGBuilder& GraphBuilder, 
		FSceneInterface* SceneInterface, 
		FPrimitiveComponentId ComponentId, 
		FSkeletalMeshObject* MeshObject,
		EMeshDeformerOutputBuffer OutputBuffers, 
		FName ExecutionGroup
		);

	/**
	 * RT: Returns which deformer output buffers are safe to read for a given component.
	 * For primary component reads (bIsPrimaryComponent=true), all recorded output buffers are returned
	 * since deformer instances on the same component are dispatched sequentially.
	 * For secondary component reads, only output buffers from execution groups that dispatched earlier
	 * than the reader's execution group are returned — same-group cross-component reads are excluded
	 * because their relative dispatch order is not guaranteed.
	 */
	static EMeshDeformerOutputBuffer GetReadableDeformedBuffers(
		FSceneInterface* SceneInterface, 
		FPrimitiveComponentId ComponentId,
		FName ReaderExecutionGroup, 
		bool bIsPrimaryComponent);

private:
	class FUpdater;

	static FOptimusSkinnedMeshTrackingExtension* Get(FSceneInterface* SceneInterface);

	/** Map of component ID → tracking data. Accessed only on RT. */
	TMap<FPrimitiveComponentId, FComponentData> ComponentDataMap;
};
