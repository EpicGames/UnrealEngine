// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSource.h"
#include "OptimusHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusSource)

void UOptimusSource::SetSource(const FString& InText)
{
	SourceText = InText;
	
	Modify();
}

#if WITH_EDITOR

void UOptimusSource::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UOptimusSource, AdditionalSources))
	{
		// Avoid recursive source inclusion
		AdditionalSources.Remove(this);
	}
}

#endif

FString UOptimusSource::GetVirtualPath() const 
{
	FString ShaderPathName = GetPathName();
	Optimus::ConvertObjectPathToShaderFilePath(ShaderPathName);
	return ShaderPathName;
}

#if WITH_EDITOR	

FString UOptimusSource::GetNameForShaderTextEditor() const
{
	return GetFName().ToString();
}

bool UOptimusSource::IsShaderTextReadOnly() const
{
	return false;
}

#endif // WITH_EDITOR	
