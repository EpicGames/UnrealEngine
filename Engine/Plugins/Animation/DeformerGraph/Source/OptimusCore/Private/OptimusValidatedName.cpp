// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusValidatedName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusValidatedName)

bool FOptimusValidatedName::IsValid(FString const& InName, FText* OutReason, FText const* InErrorCtx)
{
	static const TCHAR* InvalidCharacters = TEXT("\"\\' ,.|&!~\n\r\t@#/(){}[]=;:^%$`");

	return FName::IsValidXName(InName, InvalidCharacters, OutReason, InErrorCtx);
}

bool FOptimusValidatedName::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		Slot << Name;
		return true;
	}
	if (Tag.Type == NAME_StrProperty)
	{
		FString InName;
		Slot << InName;
		Name = FName(InName);
		return true;
	}

	return false;
}
