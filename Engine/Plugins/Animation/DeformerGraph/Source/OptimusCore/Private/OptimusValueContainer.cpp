// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusValueContainer.h"
#include "OptimusHelpers.h"
#include "OptimusValueContainerStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusValueContainer)

FName UOptimusValueContainerGeneratorClass::ValuePropertyName = TEXT("Value");

void UOptimusValueContainerGeneratorClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	// Force assembly of the reference token stream so that we can be properly handled by the
	// garbage collector.
	AssembleReferenceTokenStream(/*bForce=*/true);
}


UClass* UOptimusValueContainerGeneratorClass::GetClassForType(UPackage* InPackage, FOptimusDataTypeRef InDataType)
{
	const FString ClassName = TEXT("OptimusValueContainer_") + InDataType.TypeName.ToString();

	// Check if the package already owns this class.
	UOptimusValueContainerGeneratorClass *TypeClass = FindObject<UOptimusValueContainerGeneratorClass>(InPackage, *ClassName);
	if (!TypeClass)
	{
		UClass *ParentClass = UOptimusValueContainer::StaticClass();
		// Construct a value node class for this data type
		TypeClass = NewObject<UOptimusValueContainerGeneratorClass>(InPackage, *ClassName);
		TypeClass->SetSuperStruct(ParentClass);
		TypeClass->PropertyLink = ParentClass->PropertyLink;

		// Nodes of this type should not be listed in the node palette.
		TypeClass->ClassFlags |= CLASS_Hidden;

		// Stash the data type so that the node can return it later.
		TypeClass->DataType = InDataType;

		// Create the property chain that represents this value.
		FProperty* DefaultValueProp = InDataType->CreateProperty(TypeClass, ValuePropertyName);
		DefaultValueProp->PropertyFlags |= CPF_Edit;

#if WITH_EDITOR
		const FName CategoryMetaName = TEXT("Category");
		DefaultValueProp->SetMetaData(CategoryMetaName, TEXT("Value"));
#endif

		// AddCppProperty chains backwards.
		TypeClass->AddCppProperty(DefaultValueProp);

		// Finalize the class
		TypeClass->Bind();
		TypeClass->StaticLink(true);

		// Make sure the CDO exists.
		(void)TypeClass->GetDefaultObject();
	}
	return TypeClass;
}

UClass* UOptimusValueContainerGeneratorClass::RefreshClassForType(UPackage* InPackage, FOptimusDataTypeRef InDataType)
{
	const FString ClassName = TEXT("OptimusValueContainer_") + InDataType.TypeName.ToString();

	// Check if the package already owns this class.
	UOptimusValueContainerGeneratorClass *TypeClass = FindObject<UOptimusValueContainerGeneratorClass>(InPackage, *ClassName);

	if (TypeClass)
	{
		const FString DeprecatedClassName = FString(TEXT("Deprecated_")) + ClassName + FGuid::NewGuid().ToString();
		TypeClass->Rename(*DeprecatedClassName, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		// Add in the CLASS_NewerVersionExists class flag to this obliterated class. Just so it won't show up in any global class iteration.
		TypeClass->ClassFlags |= CLASS_NewerVersionExists;
	}

	return GetClassForType(InPackage, InDataType);
}

void UOptimusValueContainer::PostLoad()
{
	Super::PostLoad();

	if (!GetClass()->GetOuter()->IsA<UPackage>())
	{
		// This class should be parented to the package instead of the asset object
		// because the engine no longer supports asset object as UClass outer
		Optimus::RenameObject(GetClass(), nullptr, GetPackage());
	}
}

UOptimusValueContainer* UOptimusValueContainer::MakeValueContainer(UObject* InOwner, FOptimusDataTypeRef InDataTypeRef)
{
	const UClass* Class = UOptimusValueContainerGeneratorClass::GetClassForType(InOwner->GetPackage(), InDataTypeRef);

	return NewObject<UOptimusValueContainer>(InOwner, Class);
}

FOptimusValueContainerStruct UOptimusValueContainer::MakeValueContainerStruct()
{
	FOptimusValueContainerStruct OtherContainer;
	
	UOptimusValueContainerGeneratorClass* Class = Cast<UOptimusValueContainerGeneratorClass>(GetClass());
	const FProperty* ValueProperty = Class->PropertyLink;
	
	FOptimusDataTypeRef DataType = GetValueType();
	if (ensure(ValueProperty) && ensure(DataType.IsValid()))
	{
		TArrayView<const uint8> ValueData(ValueProperty->ContainerPtrToValuePtr<uint8>(this), ValueProperty->GetSize());
		
		OtherContainer.SetType(DataType.Resolve());
		OtherContainer.SetValue(DataType, ValueData);
	}

	return OtherContainer;
}

FOptimusDataTypeRef UOptimusValueContainer::GetValueType() const
{
	UOptimusValueContainerGeneratorClass* Class = Cast<UOptimusValueContainerGeneratorClass>(GetClass());
	if (ensure(Class))
	{
		return Class->DataType;
	}
	return {};	
}

FShaderValueContainer UOptimusValueContainer::GetShaderValue() const
{
	UOptimusValueContainerGeneratorClass* Class = Cast<UOptimusValueContainerGeneratorClass>(GetClass());
	const FProperty* ValueProperty = Class->PropertyLink;
	
	FOptimusDataTypeRef DataType = GetValueType();
	if (ensure(ValueProperty) && ensure(DataType.IsValid()))
	{
		TArrayView<const uint8> ValueData(ValueProperty->ContainerPtrToValuePtr<uint8>(this), ValueProperty->GetSize());
		FShaderValueContainer ValueResult = DataType->MakeShaderValue();
		if (DataType->ConvertPropertyValueToShader(ValueData, ValueResult))
		{
			return ValueResult;
		}
	}
	
	return {};
}
