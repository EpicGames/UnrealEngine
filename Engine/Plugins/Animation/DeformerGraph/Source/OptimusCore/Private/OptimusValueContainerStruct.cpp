// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusValueContainerStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusValueContainerStruct)

const TCHAR* FOptimusValueContainerStruct::ValuePropertyName = TEXT("Value");

bool FOptimusValueContainerStruct::IsInitialized() const
{
	return Value.GetNumPropertiesInBag() == 1;
}

void FOptimusValueContainerStruct::SetType(FOptimusDataTypeRef InDataType)
{
	Value.Reset();
	Value.AddProperty(ValuePropertyName, InDataType->CreateProperty(nullptr, ValuePropertyName));		
}

void FOptimusValueContainerStruct::SetValue(FOptimusDataTypeRef InDataType, TArrayView<const uint8> InValue)
{
	const FProperty* Property = GetValueProperty();

	if (ensure(Property->SameType(InDataType->CreateProperty(nullptr, NAME_None))))
	{
		Property->CopyCompleteValue(Value.GetMutableValue().GetMemory(), InValue.GetData());
	}
}

FShaderValueContainer FOptimusValueContainerStruct::GetShaderValue(FOptimusDataTypeRef InDataType) const
{
	const FProperty* Property = GetValueProperty();

	if (ensure(InDataType.IsValid()) && ensure(Property))
	{
		TArrayView<const uint8> ValueData(Property->ContainerPtrToValuePtr<uint8>(Value.GetValue().GetMemory()), Property->GetSize());
		FShaderValueContainer ValueResult = InDataType->MakeShaderValue();
		if (InDataType->ConvertPropertyValueToShader(ValueData, ValueResult))
		{
			return ValueResult;
		}
	}
	
	return {};
}

FString FOptimusValueContainerStruct::GetValueAsString() const
{
	if (const FProperty* Property = GetValueProperty())
	{
		FString ValueStr;
		Property->ExportTextItem_InContainer(ValueStr, GetValueMemory(), nullptr, nullptr, PPF_None);

		return ValueStr;
	}

	return {};
}

const FProperty* FOptimusValueContainerStruct::GetValueProperty() const
{
	check(Value.GetNumPropertiesInBag() == 1);
	
	const UPropertyBag* BagStruct = Value.GetPropertyBagStruct();
	TConstArrayView<FPropertyBagPropertyDesc> Descs = BagStruct->GetPropertyDescs();

	const FPropertyBagPropertyDesc& ValueDesc = Descs[0];

	return ValueDesc.CachedProperty;
}

const uint8* FOptimusValueContainerStruct::GetValueMemory() const
{
	return Value.GetValue().GetMemory();
}


