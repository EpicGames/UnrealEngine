// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusVariableDescription.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusHelpers.h"
#include "OptimusObjectVersion.h"
#include "OptimusValueContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusVariableDescription)


void UOptimusVariableDescription::SetDataType(FOptimusDataTypeRef InDataType)
{
	if (InDataType != DataType)
	{
		DataType = InDataType;
		DefaultValueStruct.SetType(InDataType);
	}
	else
	{
		if (!DefaultValueStruct.IsInitialized())
		{
			DefaultValueStruct.SetType(InDataType);
		}
	}
}


UOptimusDeformer* UOptimusVariableDescription::GetOwningDeformer() const
{
	const UOptimusVariableContainer* Container = CastChecked<UOptimusVariableContainer>(GetOuter());
	return Container ? CastChecked<UOptimusDeformer>(Container->GetOuter()) : nullptr;
}


int32 UOptimusVariableDescription::GetIndex() const
{
	const UOptimusVariableContainer* Container = CastChecked<UOptimusVariableContainer>(GetOuter());
	return Container->Descriptions.IndexOfByKey(this);
}


void UOptimusVariableDescription::PostLoad()
{
	Super::PostLoad();

	
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (GetLinkerCustomVersion(FOptimusObjectVersion::GUID) < FOptimusObjectVersion::PropertyBagValueContainer)
	{
		if (DefaultValue_DEPRECATED)
		{
			DefaultValue_DEPRECATED->ConditionalPostLoad();
			DefaultValueStruct = DefaultValue_DEPRECATED->MakeValueContainerStruct();
			DefaultValue_DEPRECATED = nullptr;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// 32-bit float data type is not supported for variables although they were allowed before. Do an in-place upgrade here. 
	const FOptimusDataTypeHandle FloatDataType = FOptimusDataTypeRegistry::Get().FindType(*FFloatProperty::StaticClass());
	const FOptimusDataTypeHandle DoubleDataType = FOptimusDataTypeRegistry::Get().FindType(*FDoubleProperty::StaticClass());

	if (DataType == FloatDataType)
	{
		TValueOrError<float, EPropertyBagResult> SavedValue = DefaultValueStruct.Value.GetValueFloat(FOptimusValueContainerStruct::ValuePropertyName);
		SetDataType(DoubleDataType);
		if (SavedValue.HasValue())
		{
			DefaultValueStruct.Value.SetValueDouble(FOptimusValueContainerStruct::ValuePropertyName, SavedValue.GetValue());
		}
	}

	if (!DefaultValueStruct.IsInitialized())
	{
		SetDataType(DataType);
	}
}


#if WITH_EDITOR

void UOptimusVariableDescription::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UOptimusVariableDescription, VariableName))
	{
		UOptimusDeformer* Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			VariableName = Optimus::GetUniqueNameForScope(GetOuter(), VariableName);
			Rename(*VariableName.ToString(), nullptr);

			constexpr bool bForceChange = true;
			Deformer->RenameVariable(this, VariableName, bForceChange);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptimusDataType, TypeName))
	{
		UOptimusDeformer* Deformer = GetOwningDeformer();
		if (ensure(Deformer))
		{
			// Keep the default value in sync
			DefaultValueStruct.SetType(DataType);
			
			// Set the variable type again, so that we can remove any links that are now type-incompatible.
			constexpr bool bForceChange = true;
			Deformer->SetVariableDataType(this, DataType, bForceChange);
		}
	}
}


void UOptimusVariableDescription::PreEditUndo()
{
	UObject::PreEditUndo();

	VariableNameForUndo = VariableName;
}


void UOptimusVariableDescription::PostEditUndo()
{
	UObject::PostEditUndo();

	if (VariableNameForUndo != VariableName)
	{
		const UOptimusDeformer *Deformer = GetOwningDeformer();
		Deformer->Notify(EOptimusGlobalNotifyType::VariableRenamed, this);
	}
}
#endif
