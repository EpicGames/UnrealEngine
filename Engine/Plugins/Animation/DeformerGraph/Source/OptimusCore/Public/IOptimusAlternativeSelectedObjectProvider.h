// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusAlternativeSelectedObjectProvider.generated.h"


UINTERFACE(MinimalAPI)
class UOptimusAlternativeSelectedObjectProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusAlternativeSelectedObjectProvider
{
	GENERATED_BODY()

public:
	virtual UObject* GetObjectToShowWhenSelected() const = 0;
};
