// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusComponentBindingProvider.generated.h"


class AActor;
class UActorComponent;
class UOptimusComponentSourceBinding;
struct FOptimusPinTraversalContext;


UINTERFACE(MinimalAPI)
class UOptimusComponentBindingProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusComponentBindingProvider
{
	GENERATED_BODY()

public:
	/** Returns the component binding for the node that this interface is implemented on */
	virtual UOptimusComponentSourceBinding *GetComponentBinding(const FOptimusPinTraversalContext& InContext) const = 0;
};
