// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusComponentBindingReceiver.generated.h"


class UOptimusNodePin;


UINTERFACE(MinimalAPI)
class UOptimusComponentBindingReceiver :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusComponentBindingReceiver
{
	GENERATED_BODY()

public:
	/** Returns unconnected component binding pins that would be using the default binding implicitly */
	virtual TArray<UOptimusNodePin*> GetUnconnectedComponentPins() const = 0;
};
