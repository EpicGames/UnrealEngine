// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusComputeKernelDataInterface.generated.h"

class UOptimusComponentSourceBinding;

UINTERFACE(MinimalAPI)
class UOptimusComputeKernelDataInterface :
	public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that provides a mechanism for compute kernel to setup its kernel data interface
 */
class IOptimusComputeKernelDataInterface
{
public:
	GENERATED_BODY()
	
	virtual void SetExecutionDomain(const FString& InExecutionDomain) = 0;
	virtual void SetComponentBinding(const UOptimusComponentSourceBinding* InBinding) = 0;
	virtual const FString& GetExecutionDomain() const = 0;
	virtual const TCHAR* GetReadNumThreadsFunctionName() const = 0;
	virtual const TCHAR* GetReadNumThreadsPerInvocationFunctionName() const = 0;
	virtual const TCHAR* GetReadThreadIndexOffsetFunctionName() const = 0;
	
};
