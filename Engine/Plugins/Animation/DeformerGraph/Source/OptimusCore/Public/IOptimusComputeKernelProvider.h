// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "UObject/Interface.h"
#include "Misc/TVariant.h"

#include "IOptimusComputeKernelProvider.generated.h"

class UComputeDataInterface;
class UOptimusComputeDataInterface;
class UOptimusComponentSourceBinding;
class UOptimusKernelSource;
class UOptimusNode;
class UOptimusNodePin;
class UOptimusNode_LoopTerminal;
struct FOptimusPinTraversalContext;
struct FOptimusKernelConstantContainer;
struct FOptimusExecutionDomain;
struct FOptimusRoutedConstNodePin;

// Maps the data interface's data binding index to the function we would like to have present
// during kernel compilation to read/write values from/to that data interface's resource.
struct FOptimus_InterfaceBinding
{
	const UComputeDataInterface* DataInterface;
	int32 DataInterfaceBindingIndex;
	FString BindingFunctionName;
	FString BindingFunctionNamespace;
};
using FOptimus_InterfaceBindingMap = TMap<int32 /* Kernel Index */, FOptimus_InterfaceBinding>;

// A map that goes from a value/variable node to a compute shader input parameter.
struct FOptimus_KernelParameterBinding
{
	/** The node to retrieve the value from */
	const UOptimusNode* ValueNode;
	
	/** The name of the shader parameter */ 
	FString ParameterName;
	
	/** The value type of the parameter */
	FShaderValueTypeHandle ValueType;
};
using FOptimus_KernelParameterBindingList = TArray<FOptimus_KernelParameterBinding>;

using FOptimus_ComputeKernelResult = TVariant<UOptimusKernelSource* /* Kernel */, FText /* Error */>;

struct FOptimus_KernelConnection
{
	UComputeDataInterface* DataInterface;
	const UOptimusNodePin* ConnectedPin;
};

using FOptimus_KernelInputMap = TMap<const UOptimusNodePin*, FOptimus_KernelConnection>;
using FOptimus_KernelOutputMap = TMap<const UOptimusNodePin*, TArray<FOptimus_KernelConnection>>;

UINTERFACE(MinimalAPI)
class UOptimusComputeKernelProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that provides a mechanism to identify and work with node graph owners.
 */
class IOptimusComputeKernelProvider
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/**
	 * Return an UOptimusKernelSource object, from a compute kernel node state that implements
	 * this interface.
	 * @param InKernelSourceOuter The outer object that will own the new kernel source.
	 * @param InTraversalContext The current context being used to traverse from the graph. Used mainly to keep track of graph nesting.
	 * @param InOutKernelDataInterface
	 * @param OutInputDataBindings
	 * @param OutOutputDataBindings
	 */
	virtual FOptimus_ComputeKernelResult CreateComputeKernel(
		UObject* InKernelSourceOuter,
		const FOptimusPinTraversalContext& InTraversalContext,
		const FOptimus_KernelInputMap& InKernelInputs,
		const FOptimus_KernelOutputMap& InKernelOutputs,
		UComputeDataInterface* InOutKernelDataInterface,
		FOptimus_InterfaceBindingMap& OutInputDataBindings,
		FOptimus_InterfaceBindingMap& OutOutputDataBindings
	) const = 0;
#endif

	/** Returns the execution domain that this kernel should iterate over */
	virtual FOptimusExecutionDomain GetExecutionDomain() const = 0;
	
	/** Used for traversing up the graph to identify the component source of the compute kernel */
	virtual const UOptimusNodePin* GetPrimaryGroupPin() const = 0;
	
	/** Each kernel may have its own data interface responsible for passing kernel related data to GPU */
	virtual UComputeDataInterface* MakeKernelDataInterface(UObject* InOuter) const = 0;

	/** Check if a specific pin needs to support atomic operation */
	virtual bool DoesOutputPinSupportAtomic(const UOptimusNodePin* InPin) const = 0;
	
	/** Check if a specific pin needs to support atomic operation */
	virtual bool DoesOutputPinSupportRead(const UOptimusNodePin* InPin) const = 0;
};
