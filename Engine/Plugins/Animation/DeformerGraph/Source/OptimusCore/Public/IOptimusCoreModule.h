// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusNode.h"


class UOptimusComputeDataInterface;


class IOptimusCoreModule : public IModuleInterface
{
public:
	static IOptimusCoreModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IOptimusCoreModule >(TEXT("OptimusCore"));
	}
	
	/** Register a data interface after the OptimusCore module has loaded */
	template<typename T>
	bool RegisterDataInterfaceClass()
	{
		return RegisterDataInterfaceClass(T::StaticClass());
	}
	
protected:
	virtual bool RegisterDataInterfaceClass(TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass) = 0;
};
