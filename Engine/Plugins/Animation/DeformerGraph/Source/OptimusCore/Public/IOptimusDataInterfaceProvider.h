// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusDataInterfaceProvider.generated.h"


class UOptimusNodePin;
class UOptimusComputeDataInterface;
class UOptimusComponentSourceBinding;
struct FOptimusPinTraversalContext;

UINTERFACE(MinimalAPI)
class UOptimusDataInterfaceProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusDataInterfaceProvider
{
	GENERATED_BODY()

public:
	/**
	 * Returns the data interface class that should be generated from the node that implements
	 * this interface.
	 */
	virtual UOptimusComputeDataInterface* GetDataInterface(UObject *InOuter) const = 0;

	/**
	 * Returns the index on the data interface that this pin on the node represents. E.g. for
	 * input pins, this would represent a write function on the data interface (and _not_ the
	 * proposed pin definitions).
	 * NOTE: Only valid for top-level pins.
	 * @param InPin * The pin to get the data interface function index for.
	 * @return The index of the function on the data interface that this top-level pin represents, or
	 *   INDEX_NONE if it doesn't represent a function (or if the pin isn't top-level).
	 */
	virtual int32 GetDataFunctionIndexFromPin(const UOptimusNodePin* InPin) const = 0;

	/** Returns the component binding that this data interface is bound to (or nullptr if unbound) */
	virtual UOptimusComponentSourceBinding* GetComponentBinding(const FOptimusPinTraversalContext& InContext) const = 0;
};
