// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusDeformerAssetPathAccessor.generated.h"


class UOptimusDeformerInstance;
struct FTopLevelAssetPath;

UINTERFACE(MinimalAPI)
class UOptimusDeformerAssetPathAccessor :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusDeformerAssetPathAccessor
{
	GENERATED_BODY()

public:
	virtual void SetOptimusDeformerAssetPath(const FTopLevelAssetPath& InPath) = 0;
};
