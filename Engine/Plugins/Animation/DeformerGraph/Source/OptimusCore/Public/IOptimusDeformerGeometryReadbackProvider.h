// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#if WITH_EDITORONLY_DATA
#include "Animation/MeshDeformerGeometryReadback.h"
#endif // WWITH_EDITORONLY_DATA
#include "IOptimusDeformerGeometryReadbackProvider.generated.h"


UINTERFACE(MinimalAPI)
class UOptimusDeformerGeometryReadbackProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusDeformerGeometryReadbackProvider
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	virtual bool RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest) = 0;
#endif // WWITH_EDITORONLY_DATA
};
