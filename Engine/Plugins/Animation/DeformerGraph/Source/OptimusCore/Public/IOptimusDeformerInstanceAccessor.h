// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusDeformerInstanceAccessor.generated.h"


class UOptimusDeformerInstance;

UINTERFACE(MinimalAPI)
class UOptimusDeformerInstanceAccessor :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusDeformerInstanceAccessor
{
	GENERATED_BODY()

public:
	virtual void SetDeformerInstance(UOptimusDeformerInstance* InInstance) = 0;
};
