// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusDeprecatedExecutionDataInterface.generated.h"



UINTERFACE(MinimalAPI)
class UOptimusDeprecatedExecutionDataInterface:
	public UInterface
{
	GENERATED_BODY()
};


/** An interface that should be implemented by any legacy Data Interface Node whose data interface is an execution data interface.
 *  such that the serialized execution domain can be extracted and assigned to the kernel data interface
 */
class IOptimusDeprecatedExecutionDataInterface
{
	GENERATED_BODY()

public:
	virtual FName GetSelectedExecutionDomainName() const = 0;
};
