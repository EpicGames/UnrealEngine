// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusExecutionDomainProvider.generated.h"



UINTERFACE(MinimalAPI)
class UOptimusExecutionDomainProvider :
	public UInterface
{
	GENERATED_BODY()
};


/** An interface that should be implemented by any class that has an FOptimusExecutionDomain member.
 *  This is used by the property detail specialization code to provide a list of possible domains.
 */
class IOptimusExecutionDomainProvider
{
	GENERATED_BODY()

public:
	/**
	 * Returns a list of execution domains that the implementer of this interface provides.
	 */
	virtual TArray<FName> GetExecutionDomains() const = 0;
};
