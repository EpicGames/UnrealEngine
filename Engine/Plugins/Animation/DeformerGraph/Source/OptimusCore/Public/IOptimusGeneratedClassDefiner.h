// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"
#include "UObject/Interface.h"

#include "IOptimusGeneratedClassDefiner.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusGeneratedClassDefiner :
	public UInterface
{
	GENERATED_BODY()
};


/** FIXME: A stop-gap shader value provider until we have a proper pin evaluation that handles
  * paths that have a constant, computed, varying and a mix thereof, results.
  */
class IOptimusGeneratedClassDefiner
{
	GENERATED_BODY()

public:
	/** Returns the class whose CDO we can get to get this interface to create a new generated class from
	 *  a string definition. */
	virtual FTopLevelAssetPath GetAssetPathForClassDefiner() const = 0;
	
	/** Export definition values for the given class which are sufficient to pass into a call to
	  * CreateClassFromCreationString to generate a class based on the class that hosts this interface. */
	virtual FString GetClassCreationString() const = 0;

	/** Create a generated class, within the package provided, from the creation string that was originally
	 *  generated by GetClassCreationString. */
	virtual UClass* GetClassFromCreationString(
		UPackage* InPackage,
		const TCHAR* InCreationString
		) const = 0; 
};
