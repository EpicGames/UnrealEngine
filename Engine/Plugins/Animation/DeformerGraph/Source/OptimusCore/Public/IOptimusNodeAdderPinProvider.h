// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "OptimusNode.h"

#include "IOptimusNodeAdderPinProvider.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusNodeAdderPinProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to add pins to node from existing pins
*/
class IOptimusNodeAdderPinProvider
{
	GENERATED_BODY()

public:
	struct FAdderPinAction
	{
		FName DisplayName;
		EOptimusNodePinDirection NewPinDirection;
		FName Key;
		FText ToolTip;
		bool bCanAutoLink = false;;
	};
	
	virtual TArray<FAdderPinAction> GetAvailableAdderPinActions(
		const UOptimusNodePin* InSourcePin,
		EOptimusNodePinDirection InNewPinDirection,
		FString* OutReason = nullptr
		) const = 0;

	// Make sure pins created this way don't have different names each time the function is called. Because
	// this function is called during undo/redo, output has to be stable(same input gives same output)
	virtual TArray<UOptimusNodePin*> TryAddPinFromPin(
		const FAdderPinAction& InSelectedAction,
		UOptimusNodePin* InSourcePin,
		FName InNameToUse
		) = 0;
	
	virtual bool RemoveAddedPins(TConstArrayView<UOptimusNodePin*> InAddedPinsToRemove) = 0;
};
