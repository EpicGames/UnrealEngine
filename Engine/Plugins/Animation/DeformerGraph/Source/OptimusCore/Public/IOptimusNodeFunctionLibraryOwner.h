// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNodeFunctionLibraryOwner.generated.h"


UINTERFACE(MinimalAPI)
class UOptimusNodeFunctionLibraryOwner :
	public UInterface
{
	GENERATED_BODY()
};

/**
  * Interface that provides a mechanism to identify and work with node function library owners
  */
class IOptimusNodeFunctionLibraryOwner
{
	GENERATED_BODY()

public:
	
};
