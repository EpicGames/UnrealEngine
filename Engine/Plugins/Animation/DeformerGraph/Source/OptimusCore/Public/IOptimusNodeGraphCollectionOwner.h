// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusCoreNotify.h"

#include "UObject/Interface.h"

#include "IOptimusNodeGraphCollectionOwner.generated.h"


class IOptimusPathResolver;
class UOptimusNodeGraph;
class UOptimusNode;
class UOptimusNodePin;
class FString;
enum class EOptimusNodeGraphType;


UINTERFACE(MinimalAPI)
class UOptimusNodeGraphCollectionOwner :
	public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that provides a mechanism to identify and work with node graph owners.
 */
class IOptimusNodeGraphCollectionOwner
{
	GENERATED_BODY()

public:
	/** Returns the collection that owns this collection, or nullptr if this is the root
	  * collection
	  */
	virtual IOptimusNodeGraphCollectionOwner* GetCollectionOwner() const = 0;

	/** Returns root collection that owns all the collections
	  */
	virtual IOptimusNodeGraphCollectionOwner* GetCollectionRoot() const = 0;

	/** Returns the path to this graph collection owner. */
	virtual FString GetCollectionPath() const = 0;
	
	/// Returns all immediately owned node graphs.
	virtual const TArray<UOptimusNodeGraph*> &GetGraphs() const = 0;

	virtual UOptimusNodeGraph* FindGraphByName(FName InGraphName) const = 0;

	/// Create a new graph of a given type, with an optional name. The name may be changed to 
	/// fit into the namespace. Only setup and trigger graphs can currently be created directly,
	/// and only a single setup graph. The setup graph is always the first, and the trigger graphs
	/// come after.
	/// @param InType The type of graph to create.
	/// @param InName  The name to give the graph.
	/// @param InInsertBefore The index at which the insert the graph at. This inserts the given
	/// graph before the graph already occupying this location. If unset, then the graph will
	/// not be inserted into the list of editable graphs, although it will be owned by this 
	/// collection.
	/// @return The newly created graph.
	virtual UOptimusNodeGraph* CreateGraphDirect(
		EOptimusNodeGraphType InType, 
		FName InName = NAME_None, 
		TOptional<int32> InInsertBefore = TOptional<int32>(INDEX_NONE)
		) = 0;

	/// @brief Takes an existing graph and adds it to this graph collection. If the graph cannot
	/// be added, the object remains unchanged and this function returns false.
	/// @param InGraph The graph to add.
	/// @param InInsertBefore The index at which the insert the graph at. This inserts the given
	/// graph before the graph already occupying this location.
	/// @return true if the graph was successfully added to the graph.
	virtual bool AddGraphDirect(
		UOptimusNodeGraph* InGraph,
		int32 InInsertBefore = INDEX_NONE
		) = 0;

	/// Remove the given graph.
	/// @param InGraph The graph to remove, owned by this collection. The graph will be unowned 
	/// by this graph collection and marked for GC.
	/// @return true if the graph was successfully removed.
	virtual bool RemoveGraphDirect(
		UOptimusNodeGraph* InGraph,
		bool bDeleteGraph = true
	) = 0;

	/// Re-order the graph relative to the other graphs. The insert location is relative to the
	/// graph list minus the graph being moved.
	/// \note Only trigger graphs can be moved around.
	/// @param InGraph The graph to move, owned by this collection.
	/// @param InInsertBefore The order at which the move the graph to. This inserts the given 
	/// graph before the graph already occupying this location.
	/// @return true if the graph was successfully moved.
	virtual bool MoveGraphDirect(
	    UOptimusNodeGraph* InGraph,
		int32 InInsertBefore) = 0;

	/// Rename the given graph, subject to validation of the name.
	/// @param InGraph The graph to rename, owned by this collection.
	/// @param InNewName The new name to give the graph.
	/// @return true if the graph was successfully renamed.
	virtual bool RenameGraphDirect(
		UOptimusNodeGraph *InGraph,
		const FString &InNewName
		) = 0;
	
	/// Rename the given graph, subject to validation of the name.
	/// @param InGraph The graph to rename, owned by this collection.
	/// @param InNewName The new name to give the graph.
	/// @return true if the graph was successfully renamed.
	virtual bool RenameGraph(
		UOptimusNodeGraph *InGraph,
		const FString &InNewName
		) = 0;
};
