// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNodeGraphProvider.generated.h"

class UOptimusNodeGraph;

UINTERFACE(MinimalAPI)
class UOptimusNodeGraphProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to get a node graph to display
*/
class IOptimusNodeGraphProvider
{
	GENERATED_BODY()

public:
	virtual UOptimusNodeGraph* GetNodeGraphToShow() = 0;
};

