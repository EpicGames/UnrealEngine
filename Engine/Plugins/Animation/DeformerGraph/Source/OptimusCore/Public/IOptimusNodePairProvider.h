// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNodePairProvider.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusNodePairProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to add pins to node from existing pins
*/
class IOptimusNodePairProvider
{
	GENERATED_BODY()

public:

	virtual void PairToCounterpartNode(const IOptimusNodePairProvider* NodePairProvider) = 0;
};

