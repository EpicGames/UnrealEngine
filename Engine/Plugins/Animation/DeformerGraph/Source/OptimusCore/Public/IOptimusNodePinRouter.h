// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNodePinRouter.generated.h"


class IOptimusNodePinRouter;
class UOptimusNode;
class UOptimusNodePin;


/** The context used for doing the pin traversal using the IOptimusNodePinRouter interface.
  * This structure keeps track of where the traversal arrived from, so that we can exit at the
  * same point. This is required because for functions, multiple reference nodes can refer to
  * the same graph, including the same entry/return nodes.
  */
struct FOptimusPinTraversalContext
{
	TArray<const IOptimusNodePinRouter*, TInlineAllocator<4>> ReferenceNesting;

	bool operator==(FOptimusPinTraversalContext const& RHS) const
	{
		return ReferenceNesting == RHS.ReferenceNesting;
	}
	
	friend uint32 GetTypeHash(const FOptimusPinTraversalContext& InContext)
	{
		uint32 Hash = GetTypeHash(InContext.ReferenceNesting.Num());
		
		for (const IOptimusNodePinRouter* Referencer : InContext.ReferenceNesting)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(Referencer));
		}
		return Hash;
	}
};

/** A utility struct for when traversing the graph through routed nodes. */
struct FOptimusRoutedNode
{
	UOptimusNode* Node = nullptr;
	FOptimusPinTraversalContext TraversalContext;

	bool operator==(FOptimusRoutedNode const& RHS) const
	{
		return Node == RHS.Node && TraversalContext == RHS.TraversalContext;
	}

	friend uint32 GetTypeHash(const FOptimusRoutedNode& InRoutedNode)
	{
		uint32 Hash = GetTypeHash(InRoutedNode.Node);
		
		Hash = HashCombineFast(Hash, GetTypeHash(InRoutedNode.TraversalContext));
		return Hash;
	}	
};

struct FOptimusRoutedConstNode
{
	const UOptimusNode* Node = nullptr;
	FOptimusPinTraversalContext TraversalContext;

	bool operator==(FOptimusRoutedConstNode const& RHS) const
	{
		return Node == RHS.Node && TraversalContext == RHS.TraversalContext;
	}

	friend uint32 GetTypeHash(const FOptimusRoutedConstNode& InRoutedNode)
	{
		uint32 Hash = GetTypeHash(InRoutedNode.Node);
		
		Hash = HashCombineFast(Hash, GetTypeHash(InRoutedNode.TraversalContext));
		return Hash;
	}	
};


/** A utility structure to store information on routed pins when getting routed connections.
  * The traversal context can be subsequently passed into the GetConnectedPinsWithRouting 
  * for the pin given in the struct to ensure that the traversal behaves correctly when going
  * through library functions.
  */
struct FOptimusRoutedNodePin
{
	UOptimusNodePin* NodePin = nullptr;
	FOptimusPinTraversalContext TraversalContext;
	
	bool operator==(FOptimusRoutedNodePin const& RHS) const
	{
		return NodePin == RHS.NodePin && TraversalContext == RHS.TraversalContext;
	}

	friend uint32 GetTypeHash(const FOptimusRoutedNodePin& InRoutedNodePin)
	{
		uint32 Hash = GetTypeHash(InRoutedNodePin.NodePin);
		
		Hash = HashCombineFast(Hash, GetTypeHash(InRoutedNodePin.TraversalContext));
		return Hash;
	}		
};

struct FOptimusRoutedConstNodePin
{
	const UOptimusNodePin* NodePin = nullptr;
	FOptimusPinTraversalContext TraversalContext;
	
	bool operator==(FOptimusRoutedConstNodePin const& RHS) const
	{
		return NodePin == RHS.NodePin && TraversalContext == RHS.TraversalContext;
	}
	
	friend uint32 GetTypeHash(const FOptimusRoutedConstNodePin& InRoutedNodePin)
	{
		uint32 Hash = GetTypeHash(InRoutedNodePin.NodePin);
		
		Hash = HashCombineFast(Hash, GetTypeHash(InRoutedNodePin.TraversalContext));
		return Hash;
	}		
};


UINTERFACE(MinimalAPI)
class UOptimusNodePinRouter :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusNodePinRouter
{
	GENERATED_BODY()

public:
	/** Given a pin on this node, which can be a terminal node (entry/return), or a subgraph/
	  * function reference node, return the pin matching it on the other side. E.g. if given
	  * an input pin on a terminal return node, return the output pin on the reference node in
	  * the graph above. If the pin isn't owned by this node, then a nullptr is returned.
	  * @param InNodePin The pin to find the counterpart for.
	  * @param InTraversalContext The context for the routing. This is used by the router to
	  * where it is in the traversal when traversing function references, to understand which
	  * node is the exit node. If a pin from a reference node is used to enter the function sub-graph
	  * the sub-graph reference router node is added to the context. If the pin is used to leave the
	  * sub-graph to go up, the last node in the array is used as the exit node. 
	  * @return The counterpart pin. If the pin given was invalid and has no counterpart, the
	  * NodePin value will be a nullptr and the traversal context undefined.
	  */
	virtual FOptimusRoutedNodePin GetPinCounterpart(
		UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InTraversalContext
	) const = 0; 
};
