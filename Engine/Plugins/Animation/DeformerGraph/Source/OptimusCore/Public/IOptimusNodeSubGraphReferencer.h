// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNodeSubGraphReferencer.generated.h"

class UOptimusNodeGraph;
class UOptimusNodeSubGraph;
class UOptimusComponentSourceBinding;
class UOptimusNodePin;
struct FOptimusPinTraversalContext;

UINTERFACE(MinimalAPI)
class UOptimusNodeSubGraphReferencer :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to get a node graph to display
*/
class IOptimusNodeSubGraphReferencer
{
	GENERATED_BODY()

public:
	virtual UOptimusNodeSubGraph* GetReferencedSubGraph() const = 0;
	virtual UOptimusComponentSourceBinding* GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const = 0;
	virtual UOptimusNodePin* GetDefaultComponentBindingPin() const = 0;
};

