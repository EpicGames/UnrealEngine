// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNonCollapsibleNode.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusNonCollapsibleNode :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to prevent node from being included in a subgraph
*/
class IOptimusNonCollapsibleNode
{
	GENERATED_BODY()

};

