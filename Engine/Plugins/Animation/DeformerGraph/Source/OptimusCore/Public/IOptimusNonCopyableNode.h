// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusNonCopyableNode.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusNonCopyableNode :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to prevent node from being included in a subgraph
*/
class IOptimusNonCopyableNode
{
	GENERATED_BODY()

};

