// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusOutputBufferWriter.generated.h"


enum class EMeshDeformerOutputBuffer : uint8;

UINTERFACE(MinimalAPI)
class UOptimusOutputBufferWriter :
	public UInterface
{
	GENERATED_BODY()
};


/** An interface that should be implemented by data interfaces that overrides some engine internal buffers
 */
class IOptimusOutputBufferWriter
{
	GENERATED_BODY()

public:
	/**
	 * Returns the output buffer that it writes to for the given data interface output function index
	 */
	virtual EMeshDeformerOutputBuffer GetOutputBuffer(int32 InBoundOutputFunctionIndex) const = 0;
};
