// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "OptimusBindingTypes.h"

#include "IOptimusParameterBindingProvider.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusParameterBindingProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to query information about parameter bindings
*/
class IOptimusParameterBindingProvider
{
	GENERATED_BODY()

public:
	virtual FString GetBindingDeclaration(FName BindingName) const = 0;
	virtual bool GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const = 0;
	virtual bool GetBindingSupportReadCheckBoxVisibility(FName BindingName) const = 0;
	virtual EOptimusDataTypeUsageFlags GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const = 0;
};
