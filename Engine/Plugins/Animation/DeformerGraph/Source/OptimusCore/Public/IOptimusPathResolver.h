// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusPathResolver.generated.h"


class IOptimusNodeGraphCollectionOwner;
class UOptimusComponentSourceBinding;
class UOptimusNode;
class UOptimusNodeGraph;
class UOptimusNodePin;
class UOptimusResourceDescription;
class UOptimusVariableDescription;


UINTERFACE(MinimalAPI)
class UOptimusPathResolver :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusPathResolver
{
	GENERATED_BODY()

public:
	/// Takes a collection path string and attempts to resolve it to a specific graph collection.
	/// @return The node graph found from this path, or nullptr if nothing was found.
	virtual IOptimusNodeGraphCollectionOwner* ResolveCollectionPath(const FString& InPath) = 0;
	
	/// Takes a graph path string and attempts to resolve it to a specific graph
	/// @return The node graph found from this path, or nullptr if nothing was found.
	virtual UOptimusNodeGraph* ResolveGraphPath(const FString& InPath) = 0;

	/// Takes a node path string and attempts to resolve it to a specific node
	/// @return The node found from this path, or nullptr if nothing was found.
	virtual UOptimusNode* ResolveNodePath(const FString& InPath) = 0;

	/// Takes a dot-separated path string and attempts to resolve it to a specific pin on a node.
	/// @return The node found from this path, or nullptr if nothing was found.
	virtual UOptimusNodePin* ResolvePinPath(const FString& InPinPath) = 0;
	
	virtual UOptimusResourceDescription* ResolveResource(FName InResourceName) const = 0;
	
	virtual UOptimusVariableDescription* ResolveVariable(FName InVariableName) const = 0;

	virtual UOptimusComponentSourceBinding* ResolveComponentBinding(FName InBindingName) const = 0;
};
