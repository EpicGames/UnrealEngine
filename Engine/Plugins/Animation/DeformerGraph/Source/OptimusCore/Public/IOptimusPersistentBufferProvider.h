// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusPersistentBufferProvider.generated.h"


class FOptimusPersistentBufferPool;

UINTERFACE(MinimalAPI)
class UOptimusPersistentBufferProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusPersistentBufferProvider
{
	GENERATED_BODY()

public:
	virtual void SetBufferPool(TSharedPtr<FOptimusPersistentBufferPool> InBufferPool) = 0;
};
