// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusPinMutabilityDefiner.generated.h"


class UOptimusNodePin;
UINTERFACE(MinimalAPI)
class UOptimusPinMutabilityDefiner :
	public UInterface
{
	GENERATED_BODY()
};

UENUM()
enum class EOptimusPinMutability
{
	Undefined,
	Immutable,
	Mutable,
};

class IOptimusPinMutabilityDefiner
{
	GENERATED_BODY()

public:
	// Returns whether the output pin is mutable, immutable or undefined
	virtual EOptimusPinMutability GetOutputPinMutability(const UOptimusNodePin* InPin) const = 0;
};
