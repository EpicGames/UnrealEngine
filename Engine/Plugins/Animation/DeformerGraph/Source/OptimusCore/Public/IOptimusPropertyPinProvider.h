// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusPropertyPinProvider.generated.h"


class UOptimusNodePin;


UINTERFACE(MinimalAPI)
class UOptimusPropertyPinProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IOptimusPropertyPinProvider
{
	GENERATED_BODY()

public:
	virtual TArray<UOptimusNodePin*> GetPropertyPins() const = 0;
};
