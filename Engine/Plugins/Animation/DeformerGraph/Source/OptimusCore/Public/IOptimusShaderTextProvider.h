// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IOptimusShaderTextProvider.generated.h"

UINTERFACE(MinimalAPI)
class UOptimusShaderTextProvider : public UInterface
{
	GENERATED_BODY()
};

/**
* Interface for shader text edit widgets including the gathering of compilation diagnostics for display.
*/
class IOptimusShaderTextProvider
{
	GENERATED_BODY()

#if WITH_EDITOR	
public:
	/** Get title for text editor tab. */
	virtual FString GetNameForShaderTextEditor() const = 0;
	/** Get test for declaration pane. */
	virtual FString GetDeclarations() const = 0;
	/** Get shader text for editor pane. */
	virtual FString GetShaderText() const = 0;
	/** Set shader text after edit modifications. */
	virtual void SetShaderText(const FString& NewText) = 0;
	/** Return true is the shader text can be edited */
	virtual bool IsShaderTextReadOnly() const = 0;
#endif
};
