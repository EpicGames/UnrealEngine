// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusUnnamedNodePinProvider.generated.h"

class UOptimusNodePin;

UINTERFACE(MinimalAPI)
class UOptimusUnnamedNodePinProvider :
	public UInterface
{
	GENERATED_BODY()
};

/**
* Interface that provides a mechanism to add pins to node from existing pins
*/
class IOptimusUnnamedNodePinProvider
{
	GENERATED_BODY()

public:
	virtual bool IsPinNameHidden(UOptimusNodePin* InPin) const = 0;
	virtual FName GetNameForAdderPin(UOptimusNodePin* InPin) const = 0;
};
