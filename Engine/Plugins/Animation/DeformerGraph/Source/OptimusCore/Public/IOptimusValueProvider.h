// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IOptimusValueProvider.generated.h"

struct FOptimusValueContainerStruct;
struct FOptimusDataTypeRef;
struct FOptimusValueIdentifier;

UINTERFACE()
class UOptimusValueProvider :
	public UInterface
{
	GENERATED_BODY()
};


/** FIXME: A stop-gap shader value provider until we have a proper pin evaluation that handles
  * paths that have a constant, computed, varying and a mix thereof, results.
  */
class IOptimusValueProvider
{
	GENERATED_BODY()

public:
	// Returns the value name.
	virtual FOptimusValueIdentifier GetValueIdentifier() const = 0;
	// Returns the value data type.
	virtual FOptimusDataTypeRef GetValueDataType() const = 0;
	// Returns the stored value.
	virtual FOptimusValueContainerStruct GetValue() const = 0;
};
