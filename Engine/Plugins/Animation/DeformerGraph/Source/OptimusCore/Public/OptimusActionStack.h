// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#include "OptimusActionStack.generated.h"

#define UE_API OPTIMUSCORE_API

class IOptimusPathResolver;
struct FOptimusAction;
struct FOptimusCompoundAction;

// Base action class
UCLASS(MinimalAPI)
class UOptimusActionStack :
	public UObject
{
	GENERATED_BODY()
public:
	UE_API UOptimusActionStack();

	/// Run a heap-constructed action created with operator new. 
	/// The action stack takes ownership of the pointer. If the function fails the pointer is
	/// no longer valid.
	UE_API bool RunAction(FOptimusAction* InAction);

	template<typename T, typename... ArgsType>
	typename TEnableIf<TPointerIsConvertibleFromTo<T, FOptimusAction>::Value, bool>::Type 
	RunAction(ArgsType&& ...Args)
	{
		return RunAction(MakeShared<T>(Forward<ArgsType>(Args)...));
	}

	UE_API IOptimusPathResolver *GetGraphCollectionRoot() const;

	UE_API void SetTransactionScopeFunctions(
		TFunction<int32(UObject* TransactObject, const FString& Title)> InBeginScopeFunc,
		TFunction<void(int32 InTransactionId)> InEndScopeFunc
		);

	UE_API bool Redo();
	UE_API bool Undo();

	// UObject overrides
#if WITH_EDITOR
	UE_API void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif // WITH_EDITOR

protected:
	friend class FOptimusActionScope;

	// Open a new action scope. All subsequent calls to RunAction will add actions to the scope
	// but they won't be run until the last action scope is closed.
	UE_API void OpenActionScope(const FString& InTitle);

	// Close the current action scope. Once all action scopes are closed, the collected actions
	// are finally run in the order they got added.
	UE_API bool CloseActionScope();


private:
	UE_API bool RunAction(TSharedPtr<FOptimusAction> InAction);

	UPROPERTY()
	int32 TransactedActionIndex = 0;

	int32 CurrentActionIndex = 0;

	bool bIsRunningAction = false;

	TArray<TSharedPtr<FOptimusAction>> Actions;

	TArray<TSharedPtr<FOptimusCompoundAction>> ActionScopes;

	TFunction<int(UObject* TransactObject, const FString& Title)> BeginScopeFunc;
	TFunction<void(int InTransactionId)> EndScopeFunc;
};

class FOptimusActionScope
{
public:
	UE_API FOptimusActionScope(UOptimusActionStack& InActionStack, const FString& InTitle);

	UE_API ~FOptimusActionScope();

	UE_API void SetTitle(const FString& InTitle);
private:
	UOptimusActionStack& ActionStack;
};

#undef UE_API
