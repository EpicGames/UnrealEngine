// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "OptimusDataDomain.h"
#include "OptimusDataType.h"
#include "OptimusValidatedName.h"

#include "OptimusBindingTypes.generated.h"

USTRUCT()
struct UE_DEPRECATED(5.1, "Replaced with FOptimusParameterBinding") FOptimus_ShaderBinding
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Binding)
	FName Name;

	UPROPERTY(EditAnywhere, Category = Binding, meta=(UseInResource))
	FOptimusDataTypeRef DataType;

	/** Returns true if the binding is valid and has defined entries */
	bool IsValid() const
	{
		return !Name.IsNone() && DataType.IsValid();
	}
};


USTRUCT()
struct FOptimusParameterBinding
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Binding)
	FOptimusValidatedName Name;

	UPROPERTY(EditAnywhere, Category = Binding)
	FOptimusDataTypeRef DataType;

	/** Returns true if the binding is valid and has defined entries */
	bool IsValid() const
	{
		return !Name.Name.IsNone() && DataType.IsValid();
	}
	
	UPROPERTY(EditAnywhere, Category = Binding)
	FOptimusDataDomain DataDomain;

	// Int type resource can optionally support atomic writes, memory is zero-initialized
	UPROPERTY(EditAnywhere, DisplayName="Support Atomic", Category = Binding)
	bool bSupportAtomicIfCompatibleDataType = false;

	// Optionally support both read and write
	UPROPERTY(EditAnywhere, DisplayName="Support Read", Category = Binding)
	bool bSupportRead = false;
};


USTRUCT()
struct FOptimusParameterBindingArray
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Binding)
	TArray<FOptimusParameterBinding> InnerArray;

	template <typename Predicate>
	const FOptimusParameterBinding* FindByPredicate(Predicate Pred) const
	{
		return InnerArray.FindByPredicate(Pred);
	}

	template <typename Predicate>
	FOptimusParameterBinding* FindByPredicate(Predicate Pred)
	{
		return InnerArray.FindByPredicate(Pred);
	}

	template <typename Predicate>
	int32 IndexOfByPredicate(Predicate Pred) const
	{
		return InnerArray.IndexOfByPredicate(Pred);
	}

	template <typename Predicate>
	int32 IndexOfByPredicate(Predicate Pred)
	{
		return InnerArray.IndexOfByPredicate(Pred);
	}

	bool IsEmpty() const
	{
		return InnerArray.IsEmpty();
	}

	FOptimusParameterBinding& Last(int32 IndexFromTheEnd = 0)
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	const FOptimusParameterBinding& Last(int32 IndexFromTheEnd = 0) const
	{
		return InnerArray.Last(IndexFromTheEnd);
	}

	FOptimusParameterBindingArray& operator=(const TArray<FOptimusParameterBinding>& Rhs)
	{
		InnerArray = Rhs;
		return *this;
	}
	
	int32 Num() const { return InnerArray.Num(); }
	bool IsValidIndex(int32 Index) const { return Index < InnerArray.Num() && Index >= 0; }
	const FOptimusParameterBinding& operator[](int32 InIndex) const { return InnerArray[InIndex]; }
	FOptimusParameterBinding& operator[](int32 InIndex) { return InnerArray[InIndex]; }
	inline	TArray<FOptimusParameterBinding>::RangedForIteratorType      begin()       { return InnerArray.begin(); }
	inline	TArray<FOptimusParameterBinding>::RangedForConstIteratorType begin() const { return InnerArray.begin(); }
	inline	TArray<FOptimusParameterBinding>::RangedForIteratorType      end()         { return InnerArray.end();   }
	inline	TArray<FOptimusParameterBinding>::RangedForConstIteratorType end() const   { return InnerArray.end();   }
};

