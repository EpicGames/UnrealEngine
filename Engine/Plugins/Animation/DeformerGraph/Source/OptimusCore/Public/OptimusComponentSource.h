// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Components/ActorComponent.h"

#include "OptimusComponentSource.generated.h"

#define UE_API OPTIMUSCORE_API


class UOptimusDeformer;


UCLASS(MinimalAPI, Abstract)
class UOptimusComponentSource :
	public UObject
{
	GENERATED_BODY()

public:
	/** Returns the component display name to show in the lister. Should be unique. */
	virtual FText GetDisplayName() const
	PURE_VIRTUAL(UOptimusComponentSource::GetDisplayName, return {}; );

	/** Returns a suggested name for the binding. The name may be modified to preserve uniqueness. */
	virtual FName GetBindingName() const
	PURE_VIRTUAL(UOptimusComponentSource::GetSuggestedBindingName, return {}; );

	/** Returns the actor component that this provider can operate on */
	virtual TSubclassOf<UActorComponent> GetComponentClass() const
	PURE_VIRTUAL(UOptimusComponentSource::GetComponentClass, return {}; );

	/** Returns the available execution contexts. The first context is a preferred context when setting
	  * initial data domain for a resource. */
	virtual TArray<FName> GetExecutionDomains() const
	PURE_VIRTUAL(UOptimusComponentSource::GetExecutionContexts, return {}; );

	/** Returns the current LOD index associated with a component associated with this source. */
	virtual int32 GetLodIndex(const UActorComponent* InComponent) const
	{
		return 0;
	}

	/** Returns the total number of thread invocations a component
	  * requires in case of non-unified dispatch. */
	virtual uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const
	{
		return 0;
	}
	
	/** For a given execution domain, and component LOD index, return the range of the domain as given by 
	 *  the component that is associated with this source.
	 */
	virtual bool GetComponentElementCountsForExecutionDomain(
		FName InDomainName,
		const UActorComponent* InComponent,
		int32 InLodIndex,
		TArray<int32>& OutInvocationElementCounts
		) const
	{
		return false;
	}
	
	/** Returns true if the source can be used by primary bindings. */
	UE_API virtual bool IsUsableAsPrimarySource() const;

	// TODO: Component color for additional indicator wire.

	/** Returns all registered component source objects */
	static UE_API TArray<const UOptimusComponentSource*> GetAllSources();

	/** Returns all execution domains from all available sources */
	static UE_API TSet<FName> GetAllExecutionDomains();

	/** Returns a component source that matches a data interface, or nullptr if nothing does */
	static UE_API const UOptimusComponentSource* GetSourceFromDataInterface(
		const UOptimusComputeDataInterface* InDataInterface
		);
};


UCLASS(MinimalAPI)
class UOptimusComponentSourceBinding :
	public UObject
{
	GENERATED_BODY()

public:
	/** Returns the owning deformer to operate on this variable */
	// FIXME: Move to interface-based system.
	UE_API UOptimusDeformer* GetOwningDeformer() const;

	/** Get the index of this binding within its container */
	UE_API int32 GetIndex() const;

	/** The name to give the binding, to disambiguate it from other bindings of same component type. */
	UPROPERTY(EditAnywhere, Category=Binding, meta = (EditCondition = "!bIsPrimaryBinding", HideEditConditionToggle))
	FName BindingName;

	/** The component type that this binding applies to */
	UPROPERTY(EditAnywhere, Category=Binding)
	TSubclassOf<UOptimusComponentSource> ComponentType;
	
	/** Component tags to automatically bind this component binding to. */
	UPROPERTY(EditAnywhere, Category=Tags, meta=(EditCondition="!bIsPrimaryBinding", HideEditConditionToggle))
	TArray<FName> ComponentTags;

	bool IsPrimaryBinding() const { return bIsPrimaryBinding; }

	static UE_API FName GetPrimaryBindingName();

	UE_API const UOptimusComponentSource* GetComponentSource() const; 

#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API void PreEditUndo() override;
	UE_API void PostEditUndo() override;
#endif
	
protected:
	friend class UOptimusDeformer;
	
	UPROPERTY()
	bool bIsPrimaryBinding = false;

	static UE_API const FName PrimaryBindingName;

private:
#if WITH_EDITORONLY_DATA
	FName BindingNameForUndo;
#endif
};

#undef UE_API
