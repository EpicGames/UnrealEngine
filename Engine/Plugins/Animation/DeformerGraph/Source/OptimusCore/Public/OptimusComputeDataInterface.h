// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"
#include "ComputeFramework/ComputeDataInterface.h"

#include "Templates/SubclassOf.h"

#include "OptimusComputeDataInterface.generated.h"

#define UE_API OPTIMUSCORE_API


class UActorComponent;
class UOptimusNode;

struct FOptimusCDIPinDefinition
{
	struct FDimensionInfo;
	
	// Singleton value read/write. The context name is implied as Optimus::ContextName::Singleton.
	FOptimusCDIPinDefinition(
		const FName InPinName,
		const FString InDataFunctionName,
		const bool bInMutable = true,
		const FName InDisplayName = NAME_None
		) :
		PinName(InPinName),
		DataFunctionName(InDataFunctionName),
		bMutable(bInMutable),
		DisplayName(InDisplayName)
	{ }

	// A single level context lookup.
	FOptimusCDIPinDefinition(
		const FName InPinName,
		const FString InDataFunctionName,
		const FName InContextName,
		const FString InCountFunctionName,
		const bool bInMutable = true,
		const FName InDisplayName = NAME_None
		) :
		PinName(InPinName),
		DataFunctionName(InDataFunctionName),
		DataDimensions{{InContextName, InCountFunctionName}},
		bMutable(bInMutable),
		DisplayName(InDisplayName)
	{ }

	FOptimusCDIPinDefinition(
		const FName InPinName,
		const FString InDataFunctionName,
		const FName InContextName,
		const int32 InMultiplier,
		const FString InCountFunctionName,
		const bool bInMutable = true,
		const FName InDisplayName = NAME_None
		) :
		PinName(InPinName),
		DataFunctionName(InDataFunctionName),
		DataDimensions{{InContextName, InCountFunctionName}},
		DomainMultiplier(FMath::Max(1, InMultiplier)),
		bMutable(bInMutable),
		DisplayName(InDisplayName)
	{ }
	
	FOptimusCDIPinDefinition(
		const FName InPinName,
		const FString InDataFunctionName,
		const std::initializer_list<FDimensionInfo> InContexts,
		const bool bInMutable = true,
		const FName InDisplayName = NAME_None
		) :
		PinName(InPinName),
		DataFunctionName(InDataFunctionName),
		bMutable(bInMutable),
		DisplayName(InDisplayName)
	{
		for (FDimensionInfo ContextInfo: InContexts)
		{
			DataDimensions.Add(ContextInfo);
		}
	}

	
	// The name of the pin as seen by the user.
	FName PinName;

	// The name of the function that underlies the data access by the pin. The data functions
	// are used to either read or write to data interfaces, whether explicit or implicit.
	// The read functions take zero to N uint indices, determined by the number of count 
	// functions below, and return a value. The write functions take zero to N uint indices,
	// followed by the value, with no return value.
	// For example, for a pin that has two context levels, Vertex and Bone, the lookup function
	// would look something like this:
	//    float GetBoneWeight(uint VertexIndex, uint BoneIndex);
	//    
	// And the matching element count functions for this data function would look like:
	//    uint GetVertexCount();
	//    uint GetVertexBoneCount(uint VertexIndex);
	//
	// Using these examples, the indexes to teh GetBoneWeight function would be limited in range
	// like thus:
	//    0 <= VertexIndex < GetVertexCount()   and
	//    0 <= BoneIndex < GetVertexBoneCount(VertexIndex);
	FString DataFunctionName;

	struct FDimensionInfo
	{
		// The data context for a given context level. For pins to be connectable they need to
		// have identical set of contexts, in order.
		FName ContextName;
		
		// The function to calls to get the item count for the data. If there is no count function
		// name then the data is assumed to be a singleton and will be shown as a value pin rather
		// than a resource pin. Otherwise, the number of count functions defines the dimensionality
		// of the lookup. The first count function returns the count required for the context and
		// should accept no arguments. The second count function takes as index any number between
		// zero and the result of the first count function. For example:
		//   uint GetFirstDimCount();
		//   uint GetSecondDimCount(uint FirstDimIndex);
		// These two results then bound the indices used to call the data function.
		FString CountFunctionName;
	};

	// List of nested data contexts.
	TArray<FDimensionInfo> DataDimensions;

	// For single-level domains, how many values per element of that dimension's range. 
	int32 DomainMultiplier = 1;

	bool bMutable = true;
	
	// Name used for display
	FName DisplayName;
};

struct FOptimusCDIPropertyPinDefinition
{
	FName PinName = NAME_None;

	FOptimusDataTypeRef DataType; 
};

UCLASS(MinimalAPI, Abstract, Const)
class UOptimusComputeDataInterface : public UComputeDataInterface
{
	GENERATED_BODY()
	
public:
	DECLARE_DELEGATE_TwoParams(FOnPinDefinitionRenamed, FName /* Old */ , FName /* New */);
	DECLARE_DELEGATE(FOnPinDefinitionChanged);
	
	struct CategoryName
	{
		static UE_API const FName DataInterfaces;
		static UE_API const FName ExecutionDataInterfaces;
		static UE_API const FName OutputDataInterfaces;
	};

	/// Returns the name to show on the node that will proxy this interface in the graph view.
	virtual FString GetDisplayName() const PURE_VIRTUAL(UOptimusComputeDataInterface::GetDisplayName, return {};)

	/// Returns the category for the node.
	virtual FName GetCategory() const { return CategoryName::DataInterfaces; }

	/// Returns the list of pins that will map to the shader functions provided by this data interface.
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const PURE_VIRTUAL(UOptimusComputeDataInterface::GetPinDefinitions, return {};)

	/// Returns the list of pins that are related to the properties of the data interface.
	virtual TArray<FOptimusCDIPropertyPinDefinition> GetPropertyPinDefinitions() const { return {}; }
	
	/// Data interface can use this to set default values/pins
	virtual void Initialize() {};

	UE_API virtual void ExportState(FArchive& Ar);
	UE_API virtual void ImportState(FArchive& Ar);
	
	/// Whether the data interface allow users to add / remove pins
	virtual bool CanPinDefinitionChange() { return false; };

	/// Register delegates for data interface node to update when the data interface changes
	virtual void RegisterPropertyChangeDelegatesForOwningNode(UOptimusNode* InNode) {};
	
	/**
	 * @return Returns the component type that this data interface operates on.
	 */
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const PURE_VIRTUAL(UOptimusComputeDataInterface::GetRequiredComponent, return nullptr;)

	virtual void OnDataTypeChanged(FName InTypeName) {};
	
	/**
	 * Register any additional data types provided by this data interface. 
	 */
	 virtual void RegisterTypes() {}

	/// Returns the list of top-level contexts from this data interface. These can be used to
	/// define driver contexts and resource contexts on a kernel. Each nested context will be
	/// non-empty.
	UE_API TSet<TArray<FName>> GetUniqueNestedContexts() const;
	
	virtual bool IsVisible() const
	{
		return true;
	}

	virtual TOptional<FText> ValidateForCompile() const { return{}; };

	/// Returns all known UOptimusComputeDataInterface-derived classes.
	static UE_API TArray<TSubclassOf<UOptimusComputeDataInterface>> GetAllComputeDataInterfaceClasses();

	/// Returns the list of all nested contexts from all known data interfaces. These can be 
	/// used to define input/output pin contexts on a kernel.
	static UE_API TSet<TArray<FName>> GetUniqueDomainDimensions();

	// Registers types for all known data interfaces.
	static UE_API void RegisterAllTypes();
};

#undef UE_API
