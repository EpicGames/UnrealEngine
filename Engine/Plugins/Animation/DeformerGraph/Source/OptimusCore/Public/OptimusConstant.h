// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OptimusConstant.generated.h"

class UOptimusComponentSourceBinding;
class UOptimusNode;

USTRUCT()
struct FOptimusConstantIdentifier
{
	GENERATED_BODY()

	FOptimusConstantIdentifier() = default;
	FOptimusConstantIdentifier(const UOptimusNode* InNode, const FName& InGroupName, const FName& InConstantName);

	bool IsValid() const
	{
		return NodePath != NAME_None && ConstantName != NAME_None;
	}

	FName GetLocalConstantName() const
	{
		if (GroupName == NAME_None)
		{
			return ConstantName;
		}

		return *(GroupName.ToString() + TEXT(".") + ConstantName.ToString());
	};
	
	friend uint32 GetTypeHash(const FOptimusConstantIdentifier& InIdentifier)
	{
		return HashCombineFast(GetTypeHash(InIdentifier.NodePath),
			HashCombineFast(GetTypeHash(InIdentifier.GroupName), GetTypeHash(InIdentifier.ConstantName)));
	}

	bool operator==(FOptimusConstantIdentifier const& InOther) const
	{
		return NodePath == InOther.NodePath && GroupName == InOther.GroupName && ConstantName == InOther.ConstantName;
	}
	
	UPROPERTY()
	FName NodePath;

	UPROPERTY()
	FName GroupName;
	
	UPROPERTY()
	FName ConstantName;
};

USTRUCT()
struct FOptimusConstantDefinition
{
	GENERATED_BODY()

	FOptimusConstantDefinition() = default;
	FOptimusConstantDefinition(const FOptimusConstantIdentifier& InReferencedConstant) : ReferencedConstant(InReferencedConstant) {}
	FOptimusConstantDefinition(const FString& InExpression) : Expression(InExpression) {}
	
	UPROPERTY()
	FOptimusConstantIdentifier ReferencedConstant;

	UPROPERTY()
	FString Expression;
};

UENUM()
enum class EOptimusConstantType
{
	Input,
	Output
};

USTRUCT()
struct FOptimusConstant
{
	GENERATED_BODY()
	
	FOptimusConstant() = default;
	FOptimusConstant(
		const FOptimusConstantIdentifier& InIdentifier,
		const FOptimusConstantDefinition& InDefinition,
		int32 InComponentBindingIndex,
		EOptimusConstantType InType
		) :
		Identifier(InIdentifier),
		Definition(InDefinition),
		ComponentBindingIndex(InComponentBindingIndex),
		Type(InType)
		{}
	
	UPROPERTY()
	FOptimusConstantIdentifier Identifier;

	UPROPERTY()
	FOptimusConstantDefinition Definition;
	
	UPROPERTY()
	int32 ComponentBindingIndex = 0;
	
	UPROPERTY()
	EOptimusConstantType Type = EOptimusConstantType::Input;
};


USTRUCT()
struct FOptimusConstantIndex
{
	GENERATED_BODY()

	UPROPERTY()
	int32 KernelIndex = INDEX_NONE;

	UPROPERTY()
	EOptimusConstantType Type = EOptimusConstantType::Input;
	
	UPROPERTY()
	int32 ConstantIndex = INDEX_NONE;

	bool IsValid() const
	{
		return KernelIndex != INDEX_NONE && ConstantIndex != INDEX_NONE;
	};
};

struct FOptimusConstantContainerInstance;

USTRUCT()
struct FOptimusKernelConstantContainer
{
	GENERATED_BODY()

	void AddToKernelContainer(const FOptimusConstant& InConstant);
	
	UPROPERTY();
	TArray<FOptimusConstant> InputConstants;
	
	UPROPERTY();
	TArray<FOptimusConstant> OutputConstants;

	UPROPERTY()
	TMap<FName, int32> GroupNameToBindingIndex;
};

USTRUCT()
struct FOptimusConstantContainer
{
	GENERATED_BODY()

public:
	FOptimusKernelConstantContainer& AddContainerForKernel();

	void Reset();
	
private:
	UPROPERTY()
	TArray<FOptimusKernelConstantContainer> KernelContainers;

	friend struct FOptimusConstantContainerInstance;
};

struct FOptimusConstantContainerInstance
{
public:
	bool Initialize(const FOptimusConstantContainer& InContainer, const TMap<int32, TMap<FName, TArray<float>>>& InBindingIndexToConstantValues);
	TArray<float> GetConstantValuePerInvocation(const FOptimusConstantIdentifier& InIdentifier) const;
	
private:
	TMap<FOptimusConstantIdentifier, TArray<float>> ConstantToValuePerInvocation;

	bool EvaluateAndSaveResult(const FOptimusConstantIdentifier InIdentifier, const FString& InExpression, const TMap<FName, TArray<float>>& InDependentConstants);
};







