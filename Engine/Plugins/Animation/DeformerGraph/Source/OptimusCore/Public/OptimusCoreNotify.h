// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"


class UObject;
class UOptimusNodeGraph;


enum class EOptimusGlobalNotifyType
{
	GraphAdded,						/// A new graph has been added (Subject == UOptimusNodeGraph)
	GraphRemoved,					/// A graph is about to be removed (Subject == UOptimusNodeGraph)
	GraphIndexChanged,				/// A graph's index been changed (Subject == UOptimusNodeGraph)
	GraphRenamed,					/// A graph's name has been changed (Subject == UOptimusNodeGraph)

	ComponentBindingAdded,			/// A binding has been added (Subject == UOptimusResourceDescription)
	ComponentBindingRemoved,		/// A binding is about to be removed (Subject == UOptimusResourceDescription)
	ComponentBindingIndexChanged,	/// A binding's index has changed (Subject == UOptimusResourceDescription)
	ComponentBindingRenamed,		/// A binding has been renamed (Subject == UOptimusResourceDescription)
	ComponentBindingSourceChanged,	/// A binding's type has been changed (Subject == UOptimusResourceDescription)

	ResourceAdded,					/// A resource has been added (Subject == UOptimusResourceDescription)
	ResourceRemoved,				/// A resource is about to be removed (Subject == UOptimusResourceDescription)
	ResourceIndexChanged,			/// A resource's index has changed (Subject == UOptimusResourceDescription)
	ResourceRenamed,				/// A resource has been renamed (Subject == UOptimusResourceDescription)
	ResourceTypeChanged,			/// A resource's type has been changed (Subject == UOptimusResourceDescription)
	ResourceDomainChanged,			/// A resource's domain has been changed (Subject == UOptimusResourceDescription)

	VariableAdded,					/// A variable has been added (Subject == UOptimusVariableDescription)
	VariableRemoved,				/// A variable is about to be removed (Subject == UOptimusVariableDescription)
	VariableIndexChanged,			/// A variable's index has changed (Subject == UOptimusVariableDescription)
	VariableRenamed,				/// A variable has been renamed (Subject == UOptimusVariableDescription)
	VariableTypeChanged,			/// A variable's type has been changed (Subject == UOptimusVariableDescription)
	                        		
	ConstantValueChanged,			/// A constant's value has been changed (Subject == UOptimusNode_ConstantValue)
									
	NodeTypeAdded,					/// A new node type has been added (Subject == UClass)
	NodeTypeRemoved,				/// A node type is about to be removed (Subject == UClass)

	DataTypeChanged,				/// A user defined struct type is changed (Subject == nullptr)
};

// A delegate for subscribing / reacting to Optimus global notifications.
DECLARE_MULTICAST_DELEGATE_TwoParams(FOptimusGlobalNotifyDelegate, EOptimusGlobalNotifyType /* type */, UObject* /* subject */);


enum class EOptimusGraphNotifyType
{
	NodeAdded,				/// A new node has been added (Subject == UOptimusNode)
	NodeRemoved,			/// A node has been removed (Subject == UOptimusNode)
	NodeDisplayNameChanged,	/// A node's display name has changed (Subject == UOptimusNode)
	NodePositionChanged,	/// A node's position in the graph has changed (Subject == UOptimusNode)
	NodeDiagnosticLevelChanged,	/// A node's error stat has changed (Subject == UOptimusNode)

	LinkAdded,				/// A link between nodes has been added (Subject == UOptimusNodeLink)
	LinkRemoved,			/// A link between nodes has been removed (Subject == UOptimusNodeLink)

	PinAdded,				/// A pin has been added to a node (Subject = UOptimusNodePin)
	PinRemoved,				/// A pin on a node is being removed (Subject = UOptimusNodePin)
	PinMoved,				/// A pin on a node has been moved (Subject = UOptimusNodePin)
	PinValueChanged,		/// A pin on a node has had its value changed (Subject = UOptimusNodePin)
	PinRenamed,				/// A pin's name has changed (Subject = UOptimusNodePin)
	PinTypeChanged,			/// A pin's underlying type has changed (Subject = UOptimusNodePin)
	PinDataDomainChanged,	/// A pin's data domain has changed (Subject = UOptimusNodePin)
	PinExpansionChanged,	/// A pin's expansion state has changed (Subject = UOptimusNodePin)
};

// A delegate for subscribing / reacting to Optimus graph local notifications.
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOptimusGraphNotifyDelegate, EOptimusGraphNotifyType /* type */, UOptimusNodeGraph*/*graph */, UObject* /* subject */);
