// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#include "OptimusDataDomain.generated.h"

#define UE_API OPTIMUSCORE_API

struct FOptimusExecutionDomain;

namespace Optimus::DomainName
{
	extern OPTIMUSCORE_API const FName Singleton;
	extern OPTIMUSCORE_API const FName Vertex;
	extern OPTIMUSCORE_API const FName Triangle;
	extern OPTIMUSCORE_API const FName Bone;
	extern OPTIMUSCORE_API const FName UVChannel;
	extern OPTIMUSCORE_API const FName Index0;
}

namespace Optimus
{
	OPTIMUSCORE_API FString FormatDimensionNames(const TArray<FName>& InNames);
}


UENUM()
enum class EOptimusDataDomainType
{
	Dimensional = 0,	/** Dimensional, e.g. has zero or more named dimensions of lookup. Zero-dimensional data domain is a singleton, e.g. a parameter */
	Expression = 1,		/** Expression, only a single dimension allowed for now */
};

/** A struct to specify the domain range of a resource buffer, as defined by compute kernels and data
*   interfaces. Data domains can be multi-dimensional, expression-based, or empty. Empty domains on pins imply a single
*   value, like a parameter. 
*
*   Domains come in two flavors, either as a pre-defined list with a multiplier, or as an arithmetic expression. For
*   domains with a multiplier, the multiplier only applies to the innermost dimension (e.g. Vertex.Bone x 2, allows
*   for two values per-bone, but not two values per-bone _and_ per-vertex)
*
*   The expression can take any execution domain, or none (e.g. "Vertex * 2 + 1", "Triangle * 2 + Vertex * 6", "1024").
*   If an expression is used, the domain is one-dimensional.
*   As of now, expression domain comparison is done on the string level, such that "Vertex * 2" and "2 * Vertex" are 
*   not marked as compatible domains. 
*/
USTRUCT()
struct FOptimusDataDomain
{
	GENERATED_BODY()

	FOptimusDataDomain() = default;

	explicit FOptimusDataDomain(TArray<FName> InDimensionNames) :
		DimensionNames(InDimensionNames)
	{}

	explicit FOptimusDataDomain(TArray<FName> InDimensionNames, int32 InMultiplier) :
		DimensionNames(InDimensionNames),
		Multiplier(DimensionNames.Num() == 1 ? FMath::Max(InMultiplier, 1) : 1)
	{
	}

	FOptimusDataDomain(FString InExpression) :
		Type(EOptimusDataDomainType::Expression),
		Expression(MoveTemp(InExpression))
	{}

	UE_API FOptimusDataDomain(const FOptimusExecutionDomain& InExecutionDomain);
	
	UPROPERTY(EditAnywhere, Category = DataDomain)
	EOptimusDataDomainType Type = EOptimusDataDomainType::Dimensional;

	// The name of the context that this resource/kernel applies to.
	UPROPERTY(EditAnywhere, Category = DataDomain, meta=(EditCondition="DomainType==EOptimusDataDomainType::Dimensional"))
	TArray<FName> DimensionNames;
	
	UPROPERTY(EditAnywhere, Category = DataDomain, meta=(ClampMin=1, UIMax=8, SupportDynamicSliderMaxValue="true", EditCondition="DomainType==EOptimusDataDomainType::Dimensional"))
	int32 Multiplier = 1;

	UPROPERTY(EditAnywhere, Category = DataDomain, meta=(EditCondition="DomainType==EOptimusDataDomainType::Expression"))
	FString Expression;

	
	/** Convenience function to check if this data domain is a singleton */
	bool IsSingleton() const
	{
		return Type == EOptimusDataDomainType::Dimensional && DimensionNames.IsEmpty();
	}

	/** Convenience function to check if this data domain is Multi-dimensional*/
	bool IsMultiDimensional() const
	{
		return Type == EOptimusDataDomainType::Dimensional && DimensionNames.Num() > 1;
	}

	int NumDimensions() const
	{
		if (Type == EOptimusDataDomainType::Dimensional)
		{
			return DimensionNames.Num();
		}

		if (Type == EOptimusDataDomainType::Expression)
		{
			return 1;
		}

		checkNoEntry();
		return 0;
	}
	
	/** Convenience function to check if this data domain is One-dimensional*/
	bool IsOneDimensional() const
	{
		return (Type == EOptimusDataDomainType::Dimensional && DimensionNames.Num() == 1 ) ||
			(Type == EOptimusDataDomainType::Expression);
	}

	bool IsFullyDefined() const
	{
		if (Type == EOptimusDataDomainType::Expression && Expression.IsEmpty())
		{
			return false;
		}

		return true;
	}

	/** Convert the data domain to a text serializable string */
	UE_API FString ToString() const;

	/** Create a data domain object from a serializable string generated by the above function. */
	static UE_API FOptimusDataDomain FromString(const FString& InString);

	UE_API bool operator==(const FOptimusDataDomain& InOtherDomain) const;
	bool operator!=(const FOptimusDataDomain& InOtherDomain) const { return !operator==(InOtherDomain); }
	UE_API void PostSerialize(const FArchive& Ar);
	UE_API void BackCompFixupLevels();

	UE_API TOptional<FString> AsExpression() const;

	UE_API TSet<FName> GetUsedConstants() const;

	UE_API FString GetDisplayName() const;

	static UE_API bool AreCompatible(const FOptimusDataDomain& InOutput, const FOptimusDataDomain& InInput, FString* OutReason);
	
private:
	UPROPERTY()
	TArray<FName> LevelNames_DEPRECATED{"Vertex"};
};

template<>
struct TStructOpsTypeTraits<FOptimusDataDomain> : public TStructOpsTypeTraitsBase2<FOptimusDataDomain>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithPostSerialize = true,
	};
};


#undef UE_API
