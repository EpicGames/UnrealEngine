// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "Misc/EnumClassFlags.h"

#include "OptimusDataType.generated.h"

#define UE_API OPTIMUSCORE_API


/** These flags govern how the data type can be used */
UENUM(meta = (Bitflags))
enum class EOptimusDataTypeUsageFlags : uint8
{
	None= 0,
	
	Resource				= 1 << 0,		/** This type can be used in a resource */
	Variable				= 1 << 1,		/** This type can be used in a variable */
	AnimAttributes			= 1 << 2,		/** This type can be used to query a single anim attribute on a single bone*/
	DataInterfaceOutput		= 1 << 3,		/** This type can be used as output of a data interface (or input for terminal data interfaces)*/
	PinType					= 1 << 4,		/** This type can be used as pin type*/
	PerBoneAnimAttribute	= 1 << 5,		/** This type can be used to query a per-bone anim attribute to produce a bone buffer */
	Property				= 1 << 6,		/** This type can be used in a variable, but can only be connected to a property pin on a node*/
};
ENUM_CLASS_FLAGS(EOptimusDataTypeUsageFlags)


/** These flags are for indicating type behaviour */
UENUM(meta = (Bitflags))
enum class EOptimusDataTypeFlags : uint8
{
	None = 0,
	
	IsStructType		= 1 << 0,		/** This is a UScriptStruct-based type. */
	ShowElements		= 1 << 1,		/** If a struct type, show the struct elements. */
};
ENUM_CLASS_FLAGS(EOptimusDataTypeFlags)


USTRUCT()
struct FOptimusDataType
{
	GENERATED_BODY()

	FOptimusDataType() = default;

	// Create an FProperty with the given scope and name, but only if the UsageFlags contains 
	// EOptimusDataTypeUsageFLags::Variable. Otherwise it returns a nullptr.
	UE_API FProperty* CreateProperty(
		UStruct *InScope,
		FName InName
		) const;

	// Convert an FProperty value to a value compatible with the shader parameter data layout.
	// The InValue parameter should point at the memory location governed by the FProperty for
	// this data type, and OutConvertedValue is an array to store the bytes for the converted
	// value. If the function failed, a nullptr is returned and the state of the OutConvertedValue
	// is undefined. Upon success, the return value is a pointer to the value following the
	// converted input value, and the converted output value array will have been grown to
	// accommodate the newly converted value.
	UE_API bool ConvertPropertyValueToShader(
		TArrayView<const uint8> InValue,
		FShaderValueContainer& OutConvertedValue
		) const;

	// Return a value struct that can hold raw shader value of this type
	UE_API FShaderValueContainer MakeShaderValue() const;
	
	// Returns true if the data type can create a FProperty object to represent it.
	UE_API bool CanCreateProperty() const;

	// Returns the total number of array members (recursive) in the shader type
	UE_API int32 GetNumArrays() const;
	
	// Returns the offset from the beginning of shader type of each array typed shader struct member
	UE_API int32 GetArrayShaderValueOffset(int32 InArrayIndex) const;
	
	// Returns the element size of each array typed shader struct member
	UE_API int32 GetArrayElementShaderValueSize(int32 InArrayIndex) const;

	UE_API bool IsArrayType() const;
	
	UPROPERTY()
	FName TypeName;

	UPROPERTY()
	FText DisplayName;

	// Shader value type that goes with this Optimus pin type.
	UPROPERTY()
	FShaderValueTypeHandle ShaderValueType;

	// Size of the shader value that can hold a value of this type. If this type is not a
	// shader value, then this value is zero.
	UPROPERTY()
	int32 ShaderValueSize = 0;
	
	UPROPERTY()
	FName TypeCategory;

	UPROPERTY()
	TWeakObjectPtr<UObject> TypeObject;

	UPROPERTY()
	bool bHasCustomPinColor = false;

	UPROPERTY()
	FLinearColor CustomPinColor = FLinearColor::Black;
	
	UPROPERTY()
	EOptimusDataTypeUsageFlags UsageFlags = EOptimusDataTypeUsageFlags::None;

	UPROPERTY()
	EOptimusDataTypeFlags TypeFlags = EOptimusDataTypeFlags::None;
};


using FOptimusDataTypeHandle = TSharedPtr<const FOptimusDataType>;


/** A reference object for an Optimus data type to use in UObjects and other UStruct-like things */
USTRUCT(BlueprintType)
struct FOptimusDataTypeRef
{
	GENERATED_BODY()

	UE_API FOptimusDataTypeRef(FOptimusDataTypeHandle InTypeHandle = {});

	bool IsValid() const
	{
		// The serialized DataTypeRef can become invalid when we load a deformer graph asset that has dependency on disabled plugins
		// So make sure we always check the data type registry.
		return Resolve().IsValid();
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	UE_API void Set(FOptimusDataTypeHandle InTypeHandle);

	UE_API FOptimusDataTypeHandle Resolve() const;

	const FOptimusDataType* operator->() const
	{
		return Resolve().Get();
	}

	const FOptimusDataType& operator*() const
	{
		return *Resolve().Get();
	}

	bool operator==(const FOptimusDataTypeRef& InOther) const
	{
		return TypeName == InOther.TypeName && TypeObject == InOther.TypeObject;
	}

	bool operator!=(const FOptimusDataTypeRef& InOther) const
	{
		return TypeName != InOther.TypeName || TypeObject != InOther.TypeObject;
	}

	UPROPERTY(EditAnywhere, Category=Type)
	FName TypeName;

	// A soft pointer to the type object helps enforce asset dependency
	UPROPERTY(EditAnywhere, Category=Type)
	TSoftObjectPtr<UObject> TypeObject;
	
	UE_API void PostSerialize(const FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FOptimusDataTypeRef> : public TStructOpsTypeTraitsBase2<FOptimusDataTypeRef>
{
	enum 
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
