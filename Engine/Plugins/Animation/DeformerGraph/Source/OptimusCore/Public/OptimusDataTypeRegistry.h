// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "UObject/GCObject.h"


class FFieldClass;
class UScriptStruct;
struct FShaderValueTypeHandle;
class UUserDefinedStruct;


class FOptimusDataTypeRegistry :
	public FGCObject
{
public:
	DECLARE_EVENT_OneParam(FOptimusDataTypeRegistry, FOnDataTypeChanged, FName /* InTypeName */);
	
	using PropertyCreateFuncT = TFunction<FProperty *(FFieldVariant InScope, FName InName)>;

	/** Defines a function that takes an array view on UE property data and converts it to
	 *  a matching HLSL shader value data. It's expected that the input and output array views
	 *  hold enough data to both read all property data for this type and write out shader values
	 *  matching these data.
	  */
	using PropertyValueConvertFuncT = TFunction<bool(TArrayView<const uint8> InRawValue, FShaderValueContainerView OutShaderValue)>;

	struct FArrayMetadata
	{
		int32 ElementShaderValueSize;
		int32 ShaderValueOffset;
	};

	static const TCHAR* Matrix34TypeName;

	
	virtual ~FOptimusDataTypeRegistry() override;

	static FName GetTypeName(const FFieldClass& InFieldClass);
	static FName GetTypeName(UScriptStruct* InStruct);
	static FName GetTypeName(const FAssetData& InStructAsset);
	
	static FName GetArrayTypeName(const FFieldClass& InFieldClass);
	static FName GetArrayTypeName(UScriptStruct* InStruct);
	static FName GetArrayTypeName(FName InElementTypeName);
	
	/** Get the singleton registry object */
	OPTIMUSCORE_API static FOptimusDataTypeRegistry &Get();

	// Register a POD type that has corresponding types on both the UE and HLSL side.
	OPTIMUSCORE_API bool RegisterType(
		const FFieldClass &InFieldType,
		const FText& InDisplayName,
	    FShaderValueTypeHandle InShaderValueType,
		PropertyCreateFuncT InPropertyCreateFunc,
		PropertyValueConvertFuncT InPropertyValueConvertFunc,
	    FName InPinCategory,
	    TOptional<FLinearColor> InPinColor,
	    EOptimusDataTypeUsageFlags InUsageFlags
	);

	// Register a complex type that has corresponding types on both the UE and HLSL side.
	OPTIMUSCORE_API bool RegisterType(
	    UScriptStruct *InStructType,
	    FShaderValueTypeHandle InShaderValueType,
		TOptional<FLinearColor> InPinColor,
		bool bInShowElements,
	    EOptimusDataTypeUsageFlags InUsageFlags
		);

	OPTIMUSCORE_API bool RegisterType(
	    UScriptStruct *InStructType,
		const FText& InDisplayName,
	    FShaderValueTypeHandle InShaderValueType,
		TOptional<FLinearColor> InPinColor,
		bool bInShowElements,
	    EOptimusDataTypeUsageFlags InUsageFlags
		);

	// Register a complex type that has corresponding types on both the UE and HLSL side
	// that requires a custom conversion function
	OPTIMUSCORE_API bool RegisterType(
		UScriptStruct *InStructType,
		FShaderValueTypeHandle InShaderValueType,
		PropertyValueConvertFuncT InPropertyValueConvertFunc,	
		TOptional<FLinearColor> InPinColor,
		bool bInShowElements,
		EOptimusDataTypeUsageFlags InUsageFlags
		);
	
	OPTIMUSCORE_API bool RegisterType(
		UScriptStruct *InStructType,
		const FText& InDisplayName,
		FShaderValueTypeHandle InShaderValueType,
		PropertyValueConvertFuncT InPropertyValueConvertFunc,
		TOptional<FLinearColor> InPinColor,
		bool bInShowElements,
		EOptimusDataTypeUsageFlags InUsageFlags
		);	

	// Register a complex type that has only has correspondence on the UE side.
	OPTIMUSCORE_API bool RegisterType(
		UScriptStruct *InStructType,
		const FText& InDisplayName,
		TOptional<FLinearColor> InPinColor,
		EOptimusDataTypeUsageFlags InUsageFlags
		);
	
	OPTIMUSCORE_API bool RegisterType(
	    UClass* InClassType,
		const FText& InDisplayName,
	    TOptional<FLinearColor> InPinColor,
	    EOptimusDataTypeUsageFlags InUsageFlags
		);

	// Register a type that only has correspondence on the HLSL side. 
	// Presence of the EOptimusDataTypeFlags::UseInVariable results in an error.
	OPTIMUSCORE_API bool RegisterType(
		FName InTypeName,
		const FText& InDisplayName,
	    FShaderValueTypeHandle InShaderValueType,
	    FName InPinCategory,
	    UObject* InPinSubCategory,
		FLinearColor InPinColor,
	    EOptimusDataTypeUsageFlags InUsageFlags
		);

	// Register a complex type that has corresponding types on both the UE and HLSL side.
	OPTIMUSCORE_API bool RegisterStructType(
		UScriptStruct *InStructType
		);
	
	// Register a array type that has corresponding types on both the UE and HLSL side.
	OPTIMUSCORE_API bool RegisterArrayTypeIfApplicable(
		FOptimusDataTypeHandle InElementDataType
		);
		
	// Refresh struct type info in case a user defined struct is changed.
	OPTIMUSCORE_API void RefreshStructType(
		UUserDefinedStruct *InStructType
		);

	// Unregister a type
	OPTIMUSCORE_API void UnregisterType(FName InTypeName);

	/** Returns all registered types */
	OPTIMUSCORE_API TArray<FOptimusDataTypeHandle> GetAllTypes() const;

	/** Find the registered type associated with the given property's type. Returns an invalid
	  * handle if no registered type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(const FProperty &InProperty) const;

	/** Find the registered array type associated with the given property's type. Returns an invalid
	  * handle if no registered type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(const FProperty &InProperty) const;

	/** Find the registered type associated with the given field class. Returns an invalid
	  * handle if no registered type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(const FFieldClass& InFieldType) const;

	/** Find the registered array type associated with the given field class. Returns an invalid
	  * handle if no registered array type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(const FFieldClass& InFieldType) const;

	/** Find the registered type associated with the given object class. Returns an invalid
	  * handle if no registered type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(const UClass& InClassType) const;
	
	/** Find the registered array type associated with the given object class. Returns an invalid
      * handle if no registered array type is associated.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(const UClass& InClassType) const;
	
	/** Find the registered type with the given name. Returns an invalid handle if no registered 
	  * type with that name exists.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(FName InTypeName) const;

	/** Find the registered array type with the given element type name. Returns an invalid handle if no registered
	  * type with that name exists.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(FName InTypeName) const;

	/** Find the registered array type with the given struct. Returns an invalid handle if no registered 
	  * type with that name exists.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(UScriptStruct* InStruct) const;

	/** Find the registered arraytype with the given element type struct. Returns an invalid handle if no registered 
	  * type with that name exists.
	*/
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(UScriptStruct* InStruct) const;

	/** Find a registered type from a FShaderValueTypeHandle. If multiple types are using the
	  * same shader value type, then the first one found in the registration order will be
	  * returned.
	*/
	// FIXME: We should allow for some kind of type hinting from the HLSL side (e.g. vector4 a color or a vector of four independent scalars).
	OPTIMUSCORE_API FOptimusDataTypeHandle FindType(FShaderValueTypeHandle InValueType) const;

	/** Find a registered type from a FShaderValueTypeHandle. If multiple types are using the
	  * same shader value type, then the first one found in the registration order will be
	  * returned.
	*/
	// FIXME: We should allow for some kind of type hinting from the HLSL side (e.g. vector4 a color or a vector of four independent scalars).
	OPTIMUSCORE_API FOptimusDataTypeHandle FindArrayType(FShaderValueTypeHandle InValueType) const;

	/** A helper function to return a property conversion function. The function can be unbound
	  * and that should be checked prior to calling
	*/
	PropertyValueConvertFuncT FindPropertyValueConvertFunc(FName InTypeName) const;

	/** A helper function to return the array metadata. */
	TArray<FArrayMetadata> FindArrayMetadata(FName InTypeName) const;

	/** A helper function to return the corresponding animation attribute type. */
	UScriptStruct* FindAttributeType(FName InTypeName) const;

	TArray<FOptimusDataTypeHandle> GetAllTypesWithAtomicSupport() const;

	bool DoesTypeSupportAtomic(FOptimusDataTypeHandle InType);
	
	OPTIMUSCORE_API	FOnDataTypeChanged& GetOnDataTypeChanged();

	// -- FGCObject overrides
	FString GetReferencerName() const override { return TEXT("FOptimusDataTypeRegistry"); }
	void AddReferencedObjects(FReferenceCollector& InCollector) override;
	
protected:
	friend class FOptimusCoreModule;
	friend struct FOptimusDataType;

	/** Call during module init to register all known built-in types */
	static void RegisterBuiltinTypes();

	/** Call during module shutdown to release memory */
	static void UnregisterAllTypes();

	/** Call during module init to register asset registry callbacks for struct type registration*/	
	static void RegisterEngineCallbacks();
	
	/** Call during module shutdown to unregister asset registry callbacks*/	
	static void UnregisterEngineCallbacks();

	/** A helper function to return a property create function. The function can be unbound
	 *  and that should be checked prior to calling
	 */
	PropertyCreateFuncT FindPropertyCreateFunc(FName InTypeName) const;


private:
	FOptimusDataTypeRegistry();

	// Callbacks to update registry when user defined structs are loaded/removed/renamed
	void OnFilesLoaded();
	void OnAssetRemoved(const FAssetData& InAssetData);
	void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldName);
	
	// Callbacks to update registry when list of user defined structs supported by the animation attribute system is changed
	void OnAnimationAttributeRegistryChanged(const UScriptStruct* InScriptStruct, bool bIsAdded);

	
	bool RegisterType(
		FName InTypeName,
		TFunction<void(FOptimusDataType &)> InFillFunc,
	    PropertyCreateFuncT InPropertyCreateFunc = {},
	    PropertyValueConvertFuncT InPropertyValueConvertFunc = {},
	    TArray<FArrayMetadata> InArrayMetadata = {}
		);

	/** Check if a type can be registered */
	static EOptimusDataTypeUsageFlags GetStructTypeUsageFlag(UScriptStruct* InStruct);

	struct FTypeInfo
	{
		TSharedPtr<FOptimusDataType> DataType;
		PropertyCreateFuncT PropertyCreateFunc;
		PropertyValueConvertFuncT PropertyValueConvertFunc;
		TArray<FArrayMetadata> ArrayMetadata;
	};

	TMap<FName /* TypeName */, FTypeInfo> RegisteredTypes;
	TArray<FName> RegistrationOrder;

	TSet<FName> TypeWithAtomicSupport;
	
	FOnDataTypeChanged OnDataTypeChanged;
};
