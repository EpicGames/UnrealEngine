// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusNodeFunctionLibraryOwner.h"
#include "IOptimusNodeGraphCollectionOwner.h"
#include "IOptimusPathResolver.h"
#include "OptimusCoreNotify.h"
#include "OptimusDataType.h"
#include "OptimusNodeGraph.h"
#include "OptimusComponentSource.h"
#include "OptimusDataDomain.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusValueContainerStruct.h"
#include "OptimusValue.h"

#include "Animation/MeshDeformer.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/TVariant.h"
#include "Rendering/SkeletalMeshHalfEdgeBufferAccessor.h"

#include "OptimusDeformer.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusComponentDataProvider;
class UOptimusPersistentBufferDataInterface;
class UComputeGraph;
class USkeletalMesh;
class UOptimusActionStack;
class UOptimusComputeGraph;
class UOptimusDeformer;
class UOptimusDeformerInstance;
class UOptimusResourceDescription;
class UOptimusVariableDescription;
class UOptimusFunctionNodeGraph;
enum class EOptimusDiagnosticLevel : uint8;
struct FOptimusCompilerDiagnostic;
struct FOptimusCompoundAction;
struct FOptimusFunctionGraphIdentifier;

DECLARE_MULTICAST_DELEGATE_OneParam(FOptimusCompileBegin, UOptimusDeformer *);
DECLARE_MULTICAST_DELEGATE_OneParam(FOptimusCompileEnd, UOptimusDeformer *);
DECLARE_MULTICAST_DELEGATE_OneParam(FOptimusGraphCompileMessageDelegate, FOptimusCompilerDiagnostic const&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOptimusConstantValueUpdate, TSoftObjectPtr<UObject>, FOptimusValueContainerStruct const&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOptimusSetAllInstancesCanbeActive, bool);

UENUM()
enum class EOptimusDeformerStatus : int32
{
	Compiled,					// Compiled, no warnings, no errors
	CompiledWithWarnings,		// Compiled, has warnings
	Modified,					// Graph has been modified, needs recompilation
	HasErrors					// Graph produced errors at the last compile
};


USTRUCT()
struct FOptimusComputeGraphInfo
{
	GENERATED_BODY()

	UPROPERTY()
	EOptimusNodeGraphType GraphType = EOptimusNodeGraphType::Update;

	UPROPERTY()
	FName GraphName;

	UPROPERTY()
	TObjectPtr<UOptimusComputeGraph> ComputeGraph = nullptr;
};

struct FOptimusNodeGraphCompilationResult
{
	TArray<FOptimusComputeGraphInfo> ComputeGraphInfos;

	TMap<TWeakObjectPtr<const UComputeDataInterface>, FOptimusDataInterfacePropertyOverrideInfo> DataInterfacePropertyOverrideMap;

	TMap<FOptimusValueIdentifier, FOptimusValueDescription> ValueMap;	
};

/** A container class that owns component source bindings. This is used to ensure we don't end up
  * with a namespace clash between graphs, variables, bindings and resources.
  */
UCLASS()
class UOptimusComponentSourceBindingContainer :
	public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TArray<TObjectPtr<UOptimusComponentSourceBinding>> Bindings;
};

/** A container class that owns variable descriptors. This is used to ensure we don't end up
  * with a namespace clash between graphs, variables and resources.
  */
UCLASS()
class UOptimusVariableContainer :
	public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TArray<TObjectPtr<UOptimusVariableDescription>> Descriptions;
};


/** A container class that owns resource descriptors. This is used to ensure we don't end up
  * with a namespace clash between graphs, variables and resources.
  */
UCLASS()
class UOptimusResourceContainer :
	public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY()
	TArray<TObjectPtr<UOptimusResourceDescription>> Descriptions;
};


/**
  * A Deformer Graph is an asset that is used to create and control custom deformations on 
  * skeletal meshes.
  */
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UOptimusDeformer :
	public UMeshDeformer,
	public IInterface_PreviewMeshProvider,
	public IOptimusPathResolver,
	public IOptimusNodeGraphCollectionOwner,
	public IOptimusNodeFunctionLibraryOwner,
	public ISkeletalMeshHalfEdgeBufferAccessor
{
	GENERATED_BODY()

public:

	static UE_API const TCHAR* PublicFunctionsAssetTagName;
	static UE_API const TCHAR* PublicFunctionsWithGuidAssetTagName;

	
	UE_API UOptimusDeformer();

	/** Get the action stack for this deformer graph */
	UE_API UOptimusActionStack *GetActionStack();

	/** Returns the current compilation/error status of the deformer */
	EOptimusDeformerStatus GetStatus() const
	{
		return Status;
	}

	/** Returns the global delegate used to notify on global operations (e.g. graph, variable,
	 *  resource lifecycle events).
	 */
	FOptimusGlobalNotifyDelegate& GetNotifyDelegate() { return GlobalNotifyDelegate; }

	/** Add a setup graph. This graph is executed once when the deformer is first run from a
	  * mesh component. If the graph already exists, this function does nothing and returns 
	  * nullptr.
	  */
	UE_API UOptimusNodeGraph* AddSetupGraph();
	

	/** Add a trigger graph. This graph will be scheduled to execute on next tick, prior to the
	  * update graph being executed, after being triggered from a blueprint.
	  * @param InName The name to give the graph. The name "Setup" cannot be used, since it's a
	  *  reserved name.
	  */
	UE_API UOptimusNodeGraph* AddTriggerGraph(const FString &InName);

	/// Returns the update graph. The update graph will always exist, and there is only one.
	UE_API UOptimusNodeGraph* GetUpdateGraph() const;
	/** Remove a graph and delete it. */
	UE_API bool RemoveGraph(UOptimusNodeGraph* InGraph);

	/** Returns the sub graph reference node that is uniquely associated with the given subgraph */
	UE_API UOptimusNode* GetSubGraphReferenceNode(
		const UOptimusNodeSubGraph* InSubGraph
		) const;

	/// Returns all function graphs with the given access specifier. If InAccessSpecifier is None, it performs no filtering
	UE_API TArray<UOptimusFunctionNodeGraph*> GetFunctionGraphs(FName InAccessSpecifier = NAME_None) const;

	UE_API UOptimusFunctionNodeGraph* FindFunctionByGuid(FGuid InFunctionGraphGuid);
	
	// Variables
	UE_API UOptimusVariableDescription* AddVariable(
		FOptimusDataTypeRef InDataTypeRef,
	    FName InName = NAME_None
		);

	UE_API bool RemoveVariable(
	    UOptimusVariableDescription* InVariableDesc
		);

	UE_API bool RenameVariable(
	    UOptimusVariableDescription* InVariableDesc,
	    FName InNewName,
		bool bInForceChange = false
	    );
	    
	UE_API bool SetVariableDataType(
		UOptimusVariableDescription* InVariableDesc,
		FOptimusDataTypeRef InDataType,
		bool bInForceChange = false
		);

	UE_API TArray<UOptimusNode*> GetNodesUsingVariable(
	    const UOptimusVariableDescription* InVariableDesc
		) const;

	UFUNCTION(BlueprintGetter)
	const TArray<UOptimusVariableDescription*>& GetVariables() const { return Variables->Descriptions; }

	UE_API UOptimusVariableDescription* ResolveVariable(
		FName InVariableName
		) const override;

	// Resources
	UE_API UOptimusResourceDescription* AddResource(
		FOptimusDataTypeRef InDataTypeRef,
	    FName InName = NAME_None
		);

	UE_API bool RemoveResource(
	    UOptimusResourceDescription* InResourceDesc
		);

	UE_API bool RenameResource(
	    UOptimusResourceDescription* InResourceDesc,
	    FName InNewName,
	    bool bInForceChange = false
	    );

	UE_API bool SetResourceDataType(
		UOptimusResourceDescription* InResourceDesc,
		FOptimusDataTypeRef InDataType,
		bool bInForceChange = false
		);

	UE_API bool SetResourceDataDomain(
		UOptimusResourceDescription* InResourceDesc,
		const FOptimusDataDomain& InDataDomain,
		bool bInForceChange = false
		);
	
	UE_API TArray<UOptimusNode*> GetNodesUsingResource(
		const UOptimusResourceDescription* InResourceDesc
		) const;
	
	UFUNCTION(BlueprintGetter)
	const TArray<UOptimusResourceDescription*>& GetResources() const { return Resources->Descriptions; }

	UE_API UOptimusResourceDescription* ResolveResource(
		FName InResourceName
		) const override;

	// Component Bindings
	UE_API UOptimusComponentSourceBinding* AddComponentBinding(
		const UOptimusComponentSource *InComponentSource,
		FName InName = NAME_None
		);

	UE_API bool RemoveComponentBinding(
		UOptimusComponentSourceBinding* InBinding
		);

	UE_API bool RenameComponentBinding(
		UOptimusComponentSourceBinding* InBinding,
		FName InNewName,
		bool bInForceChange = false
		);

	UE_API bool SetComponentBindingSource(
		UOptimusComponentSourceBinding* InBinding,
		const UOptimusComponentSource *InComponentSource,
		bool bInForceChange = false
		);

	UE_API TArray<UOptimusNode*> GetNodesUsingComponentBinding(
		const UOptimusComponentSourceBinding* InBinding
		) const;
	
	UFUNCTION(BlueprintGetter)
	const TArray<UOptimusComponentSourceBinding*>& GetComponentBindings() const { return Bindings->Bindings; }

	UFUNCTION(BlueprintGetter)
	UE_API UOptimusComponentSourceBinding* GetPrimaryComponentBinding() const;

	UE_API UOptimusComponentSourceBinding* ResolveComponentBinding(
		FName InBindingName
		) const override;

	
	/// Graph compilation
#if WITH_EDITOR
	UE_API bool Compile();
#endif

	/** Returns a multicast delegate that can be subscribed to listen for the start of compilation. */
	FOptimusCompileBegin& GetCompileBeginDelegate()  { return CompileBeginDelegate; }
	/** Returns a multicast delegate that can be subscribed to listen for the end of compilation but before shader compilation is complete. */
	FOptimusCompileEnd& GetCompileEndDelegate() { return CompileEndDelegate; }
	/** Returns a multicast delegate that can be subscribed to listen compilation results. Note that the shader compilation results are async and can be returned after the CompileEnd delegate. */
	FOptimusGraphCompileMessageDelegate& GetCompileMessageDelegate() { return CompileMessageDelegate; }

	UE_API void SetAllInstancesCanbeActive(bool bInCanBeActive) const;

	// Mark the deformer as modified.
	UE_API void MarkModified();
	
	/// UObject overrides
	UE_API void Serialize(FArchive& Ar) override;
	UE_API void PostLoad() override;
	UE_API void BeginDestroy() override;
	// Whenever the asset is renamed/moved, generated classes parented to the old package
	// are not moved to the new package automatically (see FAssetRenameManager), so we
	// have to manually perform the move/rename, to avoid invalid reference to the old package
	UE_API void PostRename(UObject* OldOuter, const FName OldName) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// UMeshDeformer overrides
	UE_API UMeshDeformerInstanceSettings* CreateSettingsInstance(
		UMeshComponent* InMeshComponent
		) override;
	
	UE_API UMeshDeformerInstance* CreateInstance(
		UMeshComponent* InMeshComponent,
		UMeshDeformerInstanceSettings* InSettings
		) override;
	
	
	UE_API UOptimusDeformerInstance* CreateOptimusInstance(
		UMeshComponent* InMeshComponent,
		UMeshDeformerInstanceSettings* InSettings
		);
	
	// ISkeletalMeshHalfEdgeBufferAccessor overrides
	UE_API bool IsSkeletalMeshHalfEdgeBufferRequired() const override;
	
	// IInterface_PreviewMeshProvider overrides
	UE_API void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override;
	UE_API USkeletalMesh* GetPreviewMesh() const override;

	// IOptimusPathResolver overrides
	UE_API IOptimusNodeGraphCollectionOwner* ResolveCollectionPath(const FString& InPath) override;
	UE_API UOptimusNodeGraph* ResolveGraphPath(const FString& InGraphPath) override;
	UE_API UOptimusNode* ResolveNodePath(const FString& InNodePath) override;
	UE_API UOptimusNodePin* ResolvePinPath(const FString& InPinPath) override;

	// IOptimusNodeGraphCollectionOwner overrides
	IOptimusNodeGraphCollectionOwner* GetCollectionOwner() const override { return nullptr; }
	IOptimusNodeGraphCollectionOwner* GetCollectionRoot() const override { return const_cast<UOptimusDeformer*>(this); }
	FString GetCollectionPath() const override { return FString(); }

	const TArray<UOptimusNodeGraph*> &GetGraphs() const override { return Graphs; }

	UE_API UOptimusNodeGraph* FindGraphByName(FName InGraphName) const override;

	UOptimusNodeGraph* CreateGraph(
		EOptimusNodeGraphType InType,
		FName InName)
	{ return CreateGraphDirect(InType, InName, TOptional<int32>()); }
	
	UE_API UOptimusNodeGraph* CreateGraphDirect(
	    EOptimusNodeGraphType InType,
	    FName InName,
	    TOptional<int32> InInsertBefore
	    ) override;
	
	UE_API bool AddGraphDirect(
	    UOptimusNodeGraph* InGraph,
		int32 InInsertBefore
		) override;
	
	UE_API bool RemoveGraphDirect(
	    UOptimusNodeGraph* InGraph,
		bool bDeleteGraph
		) override;

	UE_API bool MoveGraphDirect(
	    UOptimusNodeGraph* InGraph,
	    int32 InInsertBefore
	    ) override;

	UE_API bool RenameGraphDirect(
		UOptimusNodeGraph* InGraph,
		const FString& InNewName
		) override;
	
	UE_API bool RenameGraph(
	    UOptimusNodeGraph* InGraph,
	    const FString& InNewName
	    ) override;
	
	UPROPERTY(EditAnywhere, Category=Preview)
	TObjectPtr<USkeletalMesh> Mesh = nullptr;

protected:
	friend class UOptimusComponentSourceBinding;
	friend class UOptimusDeformerInstance;
	friend class UOptimusNodeGraph;
	friend class UOptimusResourceDescription;
	friend class UOptimusVariableDescription;
	friend struct FOptimusComponentBindingAction_AddBinding;
	friend struct FOptimusComponentBindingAction_RemoveBinding;
	friend struct FOptimusComponentBindingAction_RenameBinding;
	friend struct FOptimusComponentBindingAction_SetComponentSource;
	friend struct FOptimusResourceAction_AddResource;
	friend struct FOptimusResourceAction_RemoveResource;
	friend struct FOptimusResourceAction_RenameResource;
	friend struct FOptimusResourceAction_SetDataType;
	friend struct FOptimusResourceAction_SetDataDomain;
	friend struct FOptimusVariableAction_AddVariable;
	friend struct FOptimusVariableAction_RemoveVariable;
	friend struct FOptimusVariableAction_RenameVariable;
	friend struct FOptimusVariableAction_SetDataType;

	/** Create a resource owned by this deformer but does not add it to the list of known
	  * resources. Call AddResource for that */
	UE_API UOptimusResourceDescription* CreateResourceDirect(
		FName InName
		);

	/** Adds a resource that was created by this deformer and is owned by it. */
	UE_API bool AddResourceDirect(
		UOptimusResourceDescription* InResourceDesc,
		const int32 InIndex
		);

	UE_API bool RemoveResourceDirect(
		UOptimusResourceDescription* InResourceDesc
		);

	UE_API bool RenameResourceDirect(
		UOptimusResourceDescription* InResourceDesc,
		FName InNewName
		);
		
	UE_API bool SetResourceDataTypeDirect(
		UOptimusResourceDescription* InResourceDesc,
		FOptimusDataTypeRef InDataType
		);

	UE_API bool SetResourceDataDomainDirect(
		UOptimusResourceDescription* InResourceDesc,
		const FOptimusDataDomain& InDataDomain
		);

	
	/** Create a resource owned by this deformer but does not add it to the list of known
	  * resources. Call AddResource for that */
	UE_API UOptimusVariableDescription* CreateVariableDirect(
		FName InName
		);

	/** Adds a resource that was created by this deformer and is owned by it. */
	UE_API bool AddVariableDirect(
		UOptimusVariableDescription* InVariableDesc,
		const int32 InIndex
		);

	UE_API bool RemoveVariableDirect(
		UOptimusVariableDescription* InVariableDesc
		);

	UE_API bool RenameVariableDirect(
		UOptimusVariableDescription* InVariableDesc,
		FName InNewName
		);

	UE_API bool SetVariableDataTypeDirect(
		UOptimusVariableDescription* InResourceDesc,
		FOptimusDataTypeRef InDataType
		);

	UE_API UOptimusComponentSourceBinding* CreateComponentBindingDirect(
		const UOptimusComponentSource *InComponentSource,
		FName InName = NAME_None
		);

	UE_API bool AddComponentBindingDirect(
		UOptimusComponentSourceBinding* InComponentBinding,
		const int32 InIndex
		);

	UE_API bool RemoveComponentBindingDirect(
		UOptimusComponentSourceBinding* InBinding
		);

	UE_API bool RenameComponentBindingDirect(
		UOptimusComponentSourceBinding* InBinding,
		FName InNewName);
	
	UE_API bool SetComponentBindingSourceDirect(
		UOptimusComponentSourceBinding* InBinding,
		const UOptimusComponentSource *InComponentSource
		);

	UE_API void SetStatusFromDiagnostic(EOptimusDiagnosticLevel InDiagnosticLevel);
	
	UE_API void Notify(EOptimusGlobalNotifyType InNotifyType, UObject *InObject) const;

	// The compute graphs to execute.
	UPROPERTY()
	TArray<FOptimusComputeGraphInfo> ComputeGraphs;
	
	UPROPERTY()
	TMap<TWeakObjectPtr<const UComputeDataInterface>, FOptimusDataInterfacePropertyOverrideInfo> DataInterfacePropertyOverrideMap;
	
	UPROPERTY()
	TMap<FOptimusValueIdentifier, FOptimusValueDescription> ValueMap;

private:
	UE_API void PostLoadFixupMissingComponentBindingsCompat();
	UE_API void PostLoadFixupMismatchedResourceDataDomains();
	UE_API void PostLoadRemoveDeprecatedExecutionNodes();
	UE_API void PostLoadRemoveDeprecatedValueContainerGeneratorClass();
	UE_API void PostLoadMoveValueFromGraphDataInterfaceToDeformerValueMap();

	/** Find a compatible binding with the given data interface. Returns nullptr if no such binding exists */
	UE_API UOptimusComponentSourceBinding* FindCompatibleBindingWithInterface(
		const UOptimusComputeDataInterface* InDataInterface
		) const;
	
	UE_API UOptimusNodeGraph* ResolveGraphPath(const FStringView InPath, FStringView& OutRemainingPath) const;
	UE_API UOptimusNode* ResolveNodePath(const FStringView InPath, FStringView& OutRemainingPath) const;
	UE_API int32 GetUpdateGraphIndex() const;

	UE_API TArray<UOptimusNode*> GetAllNodesOfClass(UClass* InNodeClass) const;
	UE_API void OnGraphRenamedOrRemoved(UOptimusNodeGraph* InGraph) const;
	UE_API void UpdateFunctionReferenceNodeDisplayName(const FOptimusFunctionGraphIdentifier& InRenamedFunction);
	
#if WITH_EDITOR
	/// Compile a node graph to a compute graph. Returns one or two complete compute graphs if compilation succeeded. 
	UE_API FOptimusNodeGraphCompilationResult CompileNodeGraphToComputeGraphs(
		const UOptimusNodeGraph *InNodeGraph,
		TFunction<void(EOptimusDiagnosticLevel, FText, const UObject*)> InErrorReporter
		);
#endif

	UE_API void OnDataTypeChanged(FName InTypeName);

	UPROPERTY(transient)
	TObjectPtr<UOptimusActionStack> ActionStack = nullptr;

	UPROPERTY()
	EOptimusDeformerStatus Status = EOptimusDeformerStatus::Modified;

	UPROPERTY()
	TArray<TObjectPtr<UOptimusNodeGraph>> Graphs;

	UPROPERTY()
	TObjectPtr<UOptimusComponentSourceBindingContainer> Bindings;

	UPROPERTY()
	TObjectPtr<UOptimusVariableContainer> Variables;

	UPROPERTY()
	TObjectPtr<UOptimusResourceContainer> Resources;

	FOptimusGlobalNotifyDelegate GlobalNotifyDelegate;

	FOptimusCompileBegin CompileBeginDelegate;
	
	FOptimusCompileEnd CompileEndDelegate;

	FOptimusGraphCompileMessageDelegate CompileMessageDelegate;

	FOptimusConstantValueUpdate ConstantValueUpdateDelegate;

	FOptimusSetAllInstancesCanbeActive SetAllInstancesCanbeActiveDelegate;
};

#undef UE_API
