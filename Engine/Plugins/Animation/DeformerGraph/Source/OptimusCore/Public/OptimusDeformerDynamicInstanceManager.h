// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MeshDeformerInstance.h"
#include "Animation/MeshDeformerProducer.h"

#if WITH_EDITORONLY_DATA
#include "Animation/MeshDeformerGeometryReadback.h"
#endif // WTIH_EDITORONLY_DATA

#include "OptimusDeformerDynamicInstanceManager.generated.h"

class UControlRig;
class UOptimusDeformerInstance;
class UOptimusDeformer;

UENUM()
enum class EOptimusDeformerExecutionPhase : uint8
{
	AfterDefaultDeformer = 0,
	OverrideDefaultDeformer = 1,
	BeforeDefaultDeformer = 2,
};

/** 
 * Enables composition of multiple deformer instances dynamically
 */
UCLASS(MinimalAPI)
class UOptimusDeformerDynamicInstanceManager : public UMeshDeformerInstance
{
	GENERATED_BODY()

public:
	/** Called to allocate any persistent render resources */
	OPTIMUSCORE_API void AllocateResources() override;;

	/** Called when persistent render resources should be released */
	OPTIMUSCORE_API void ReleaseResources() override;

	/** Enqueue the mesh deformer workload on a scene. */
	OPTIMUSCORE_API void EnqueueWork(FEnqueueWorkDesc const& InDesc) override;

	/** Return the buffers that this deformer can potentially write to */
	OPTIMUSCORE_API EMeshDeformerOutputBuffer GetOutputBuffers() const override;

#if WITH_EDITORONLY_DATA
	/** Readback the deformed geometry as mesh description */
	OPTIMUSCORE_API bool RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest) override;
#endif // WITH_EDITORONLY_DATA
	
	/** InstanceManager is an intermediate instance, call this function to get the instance for the deformer that created this instance manager */
	OPTIMUSCORE_API UMeshDeformerInstance* GetInstanceForSourceDeformer() override;

	/** Begin destroying the managaer */
	OPTIMUSCORE_API void BeginDestroy() override;

	/** Remove associated deformer instances when the object is removed */
	OPTIMUSCORE_API void OnObjectBeginDestroy(IMeshDeformerProducer* InObject);

	/** Add a producer defomer to the manager (from game thread) */
	OPTIMUSCORE_API void AddProducerDeformer(IMeshDeformerProducer* InProducer, FGuid InInstanceGuid, UOptimusDeformer* InDeformer);

	/** Get the object deformer given a guid */
	OPTIMUSCORE_API UOptimusDeformerInstance* GetDeformerInstance(FGuid InInstanceGuid);
	
	/** Enqueue an object deformer instance to be executed later (from anim/physics... threads )*/
	OPTIMUSCORE_API void EnqueueProducerDeformer(FGuid InInstanceGuid, EOptimusDeformerExecutionPhase InExecutionPhase, int32 InExecutionGroup);

	UPROPERTY()
	TObjectPtr<UOptimusDeformerInstance> DefaultInstance;

	UE_DEPRECATED(5.6, "This property is going to be deleted and replaced by the private member GuidToInstanceMap")
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UOptimusDeformerInstance>> GuidToRigDeformerInstanceMap;

private:

	/** Guid to deformer instance map*/
	UPROPERTY()
	TMap<FGuid, TObjectPtr<UOptimusDeformerInstance>> GuidToInstanceMap;
	
	/** Object ptr to deformer guids map */
	TMap<TWeakObjectPtr<UObject>, TArray<FGuid>> ProducerToGuidsMap;

	/** Freshly created deformer instances should be initialized before dispatch */
	TArray<FGuid> GuidsPendingInit;

	/** Instances per execution group per execution phase */
	TMap<EOptimusDeformerExecutionPhase, TMap<int32, TArray<FGuid>>> ExecutionQueueMap;

	/** Enqueue critical section to be thread safe */
	FCriticalSection EnqueueCriticalSection;

#if WITH_EDITORONLY_DATA
	/** Readback requests for the current frame */
	TArray<TUniquePtr<FMeshDeformerGeometryReadbackRequest>> GeometryReadbackRequests;
#endif // WITH_EDITORONLY_DATA
};





