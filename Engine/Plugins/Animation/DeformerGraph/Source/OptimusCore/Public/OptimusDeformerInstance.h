// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MeshDeformerInstance.h"
#include "ComputeFramework/ComputeGraphInstance.h"
#include "Engine/EngineTypes.h"
#include "OptimusValue.h"
#include "UObject/WeakInterfacePtr.h"
#include "IOptimusDeformerGeometryReadbackProvider.h"

#include "OptimusDeformerInstance.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusComponentSource;
enum class EOptimusNodeGraphType;
struct FOptimusPersistentStructuredBuffer;
class FRDGBuffer;
class FRDGBuilder;
class UActorComponent;
class UMeshComponent;
class UOptimusDeformer;
class UOptimusVariableContainer;
class UOptimusVariableDescription;
class UOptimusComponentSourceBinding;
struct FShaderValueContainer;
struct FOptimusValueContainerStruct;
class UComputeDataInterface;

class FOptimusPersistentBufferPool
{
public:
	/** 
	 * Get or allocate buffers for the given resource
	 * If the buffer already exists but has different sizing characteristics the allocation fails. 
	 * The number of buffers will equal the size of the InElementCount array.
	 * But if the allocation fails, the returned array will be empty.
	 */
	void GetResourceBuffers(
		FRDGBuilder& GraphBuilder,
		FName InResourceName,
		int32 InLODIndex,
		int32 InElementStride,
		int32 InRawStride,
		TArray<int32> const& InElementCounts,
		TArray<FRDGBuffer*>& OutBuffers,
		bool& bOutJustAllocated);

	void GetImplicitPersistentBuffers(
		FRDGBuilder& GraphBuilder,
		FName DataInterfaceName,
		int32 LODIndex,
		int32 InElementStride,
		int32 InRawStride,
		TArray<int32> const& InElementCounts,
		TArray<FRDGBuffer*>& OutBuffers,
		bool& bOutJustAllocated);

	/** Release _all_ resources allocated by this pool */
	void ReleaseResources();
	
private:
	 void AllocateBuffers(
		FRDGBuilder& GraphBuilder,
		int32 InElementStride,
		int32 InRawStride,
		TArray<int32> const& InElementCounts,
		TArray<FOptimusPersistentStructuredBuffer>& OutResourceBuffers,
		TArray<FRDGBuffer*>& OutBuffers
		);

	void ValidateAndGetBuffers(
		FRDGBuilder& GraphBuilder,
		int32 InElementStride,
		TArray<int32> const& InElementCounts,
		const TArray<FOptimusPersistentStructuredBuffer>& InResourceBuffers,
		TArray<FRDGBuffer*>& OutBuffers
		) const;
	
	TMap<FName, TMap<int32, TArray<FOptimusPersistentStructuredBuffer>>> ResourceBuffersMap;
	TMap<FName, TMap<int32, TArray<FOptimusPersistentStructuredBuffer>>> ImplicitBuffersMap;
};
using FOptimusPersistentBufferPoolPtr = TSharedPtr<FOptimusPersistentBufferPool>;


/** Structure with cached state for a single compute graph. */
USTRUCT()
struct FOptimusDeformerInstanceExecInfo
{
	GENERATED_BODY()

	FOptimusDeformerInstanceExecInfo();

	/** The name of the graph */
	UPROPERTY()
	FName GraphName;

	/** The graph type. */
	UPROPERTY()
	EOptimusNodeGraphType GraphType;
	
	/** The ComputeGraph asset. */
	UPROPERTY()
	TObjectPtr<UComputeGraph> ComputeGraph = nullptr;

	/** The cached state for the ComputeGraph. */
	UPROPERTY()
	FComputeGraphInstance ComputeGraphInstance;
};


/** Defines a binding between a component provider in the graph and an actor component in the component hierarchy on
 *  the actor whose deformable component we're bound to.
 */
USTRUCT(BlueprintType)
struct FOptimusDeformerInstanceComponentBinding
{
	GENERATED_BODY()

	/** Binding name on deformer graph. */
	UPROPERTY(VisibleAnywhere, Category="Deformer", meta = (DisplayName = "Binding"))
	FName ProviderName;

	/** Component name to bind. This should be sanitized before storage. */
	UPROPERTY(EditAnywhere, Category="Deformer", meta = (DisplayName = "Component"))
 	FName ComponentName;

	/** Get the component on an actor that matches the stored component name. */
	TSoftObjectPtr<UActorComponent> GetActorComponent(AActor const* InActor) const;

	/** Helpers to create ComponentName. */
	OPTIMUSCORE_API static bool GetSanitizedComponentName(FString& InOutName);
	OPTIMUSCORE_API static FName GetSanitizedComponentName(FName InName);
	OPTIMUSCORE_API static FName GetSanitizedComponentName(UActorComponent const* InComponent);
	OPTIMUSCORE_API static TSoftObjectPtr<UActorComponent> GetActorComponent(AActor const* InActor, FString const& InName);
};

/**
 * Delegate called per non-primary binding during GetComponentBindings to supply an explicit component.
 * If bound and returns non-null, the result bypasses tag/class auto-resolution for that binding. 
 */
DECLARE_DELEGATE_RetVal_OneParam(UActorComponent*, FOptimusComponentBindingResolver, FName /*BindingName*/)

UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UOptimusDeformerInstanceSettings :
	public UMeshDeformerInstanceSettings
{
	GENERATED_BODY()

	/** Stored weak pointer to a deformer. This is only required by the details customization for resolving binding class types. */
	UPROPERTY()
	TWeakObjectPtr<UOptimusDeformer> Deformer;

	/** Array of binding descriptions. This is fixed and used by GetComponentBindings() to resolve final bindings for a given context. */
	UPROPERTY(EditAnywhere, Category="Deformer|DeformerSettings", EditFixedSize, meta = (DisplayName = "Component Bindings", EditFixedOrder))
	TArray<FOptimusDeformerInstanceComponentBinding> Bindings;

public:
	/** Optional resolver called for each non-primary binding before tag/class auto-resolution to supply explicit components programmatically. */
	FOptimusComponentBindingResolver ComponentResolver;
	
	/** Setup the object. This initializes the binding names and the primary binding component. */
	UE_API void InitializeSettings(UOptimusDeformer* InDeformer, UMeshComponent* InPrimaryComponent);

	/** Get an array of recommended component bindings, based on the stored settings. */
	UE_API void GetComponentBindings(UOptimusDeformer* InDeformer, UMeshComponent* InPrimaryComponent, TArray<UActorComponent*>& OutComponentBindings) const;

protected:
	friend class FOptimusDeformerInstanceComponentBindingCustomization;

	/** Get a full component source binding object by binding name. Used only by details customization. */
	UE_API UOptimusComponentSourceBinding const* GetComponentBindingByName(FName InBindingName) const;
};


/** 
 * Class representing an instance of an Optimus Mesh Deformer, used in a OptimusDeformerDynamicInstanceManager
 * It contains the per instance deformer variable state and local state for each of the graphs in the deformer.
 */
UCLASS(MinimalAPI, BlueprintType)
class UOptimusDeformerInstance : public UMeshDeformerInstance
{
	GENERATED_BODY()

public:
	/** 
	 * Set the Mesh Component that owns this instance.
	 * Call once before first call to SetupFromDeformer().
	 */
	UE_API void SetMeshComponent(UMeshComponent* InMeshComponent);

	/** 
	 * Set the instance settings that control this deformer instance. The deformer instance is transient whereas
	 * the settings are persistent.
	 */
	UE_API void SetInstanceSettings(UOptimusDeformerInstanceSettings* InInstanceSettings);
	
	/** 
	 * Setup the instance. 
	 * Needs to be called after the UOptimusDeformer creates this instance, and whenever the instance is invalidated.
	 * Invalidation happens whenever any bound Data Providers become invalid.
	 */
	UE_API void SetupFromDeformer(UOptimusDeformer* InDeformer);

	/** Set the value of a boolean variable. */
	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Boolean)"))
	UE_API bool SetBoolVariable(FName InVariableName, bool InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Boolean Array)"))
	UE_API bool SetBoolArrayVariable(FName InVariableName, const TArray<bool>& InValue);
	
	/** Set the value of an integer variable. */
	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer)"))
	UE_API bool SetIntVariable(FName InVariableName, int32 InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer Array)"))
	UE_API bool SetIntArrayVariable(FName InVariableName, const TArray<int32>& InValue);
	

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer2)"))
	UE_API bool SetInt2Variable(FName InVariableName, const FIntPoint& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer2 Array)"))
	UE_API bool SetInt2ArrayVariable(FName InVariableName, const TArray<FIntPoint>& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer3)"))
	UE_API bool SetInt3Variable(FName InVariableName, const FIntVector& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer3 Array)"))
	UE_API bool SetInt3ArrayVariable(FName InVariableName, const TArray<FIntVector>& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer4)"))
	UE_API bool SetInt4Variable(FName InVariableName, const FIntVector4& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Integer4 Array)"))
	UE_API bool SetInt4ArrayVariable(FName InVariableName, const TArray<FIntVector4>& InValue);


	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Float)"))
	UE_API bool SetFloatVariable(FName InVariableName, double InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Float Array)"))
	UE_API bool SetFloatArrayVariable(FName InVariableName, const TArray<double>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector2)"))
	UE_API bool SetVector2Variable(FName InVariableName, const FVector2D& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector2 Array)"))
	UE_API bool SetVector2ArrayVariable(FName InVariableName, const TArray<FVector2D>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector)"))
	UE_API bool SetVectorVariable(FName InVariableName, const FVector& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector Array)"))
	UE_API bool SetVectorArrayVariable(FName InVariableName, const TArray<FVector>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector4)"))
	UE_API bool SetVector4Variable(FName InVariableName, const FVector4& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Vector4 Array)"))
	UE_API bool SetVector4ArrayVariable(FName InVariableName, const TArray<FVector4>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (LinearColor)"))
	UE_API bool SetLinearColorVariable(FName InVariableName, const FLinearColor& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (LinearColor Array)"))
	UE_API bool SetLinearColorArrayVariable(FName InVariableName, const TArray<FLinearColor>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Quat)"))
	UE_API bool SetQuatVariable(FName InVariableName, const FQuat& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Quat Array)"))
	UE_API bool SetQuatArrayVariable(FName InVariableName, const TArray<FQuat>& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Rotator)"))
	UE_API bool SetRotatorVariable(FName InVariableName, const FRotator& InValue);

	UFUNCTION(BlueprintCallable, Category="Deformer", meta=(DisplayName="Set Variable (Rotator Array)"))
	UE_API bool SetRotatorArrayVariable(FName InVariableName, const TArray<FRotator>& InValue);
	
	/** Set the value of a transform variable. */
	UFUNCTION(BlueprintCallable, Category = "Deformer", meta = (DisplayName = "Set Variable (Transform)"))
	UE_API bool SetTransformVariable(FName InVariableName, const FTransform& InValue);

	UFUNCTION(BlueprintCallable, Category = "Deformer", meta = (DisplayName = "Set Variable (Transform Array)"))
	UE_API bool SetTransformArrayVariable(FName InVariableName, const TArray<FTransform>& InValue);
	
	UFUNCTION(BlueprintCallable, Category = "Deformer", meta = (DisplayName = "Set Variable (Name)"))
	UE_API bool SetNameVariable(FName InVariableName, const FName& InValue);

	UFUNCTION(BlueprintCallable, Category = "Deformer", meta = (DisplayName = "Set Variable (Name Array)"))
	UE_API bool SetNameArrayVariable(FName InVariableName, const TArray<FName>& InValue);

	/** Trigger a named trigger graph to run on the next tick */
	UFUNCTION(BlueprintCallable, Category="Deformer")
	UE_API bool EnqueueTriggerGraph(FName InTriggerGraphName);
	
	/** Directly set a graph constant value. */
	UE_API void SetConstantValueDirect(TSoftObjectPtr<UObject> InSourceObject, FOptimusValueContainerStruct const& InValue);

	FOptimusPersistentBufferPoolPtr GetBufferPool() const { return BufferPool; }

	UE_API void SetCanBeActive(bool bInCanBeActive);

	UE_API FOptimusValueContainerStruct GetDataInterfacePropertyOverride(
		const UComputeDataInterface* DataInterface,
		FName PinName
		);

	UE_API const FShaderValueContainer& GetShaderValue(const FOptimusValueIdentifier& InValueId) const;
	
	/** Making sure compute graphs belong to this instance does not run before instances before it */
	int32 GraphSortPriorityOffset = 0;

	/** The execution group name for the current EnqueueWork call (e.g. EndOfFrameUpdate, BeginInitViews).
	 *  Set on game thread before GetRenderProxy() calls, used by read/write proxies to inform the tracking extension. */
	FName CurrentExecutionGroupName = NAME_None;

	/** Get the primary mesh component that owns this deformer instance. */
	UMeshComponent* GetMeshComponent() const { return MeshComponent.Get(); }
protected:
	friend class UOptimusDeformerDynamicInstanceManager;

	UE_API void AllocateResources() override;
	UE_API void ReleaseResources() override;
	UE_API void EnqueueWork(FEnqueueWorkDesc const& InDesc) override;
	UE_API EMeshDeformerOutputBuffer GetOutputBuffers() const override;
#if WITH_EDITORONLY_DATA
	UE_API bool RequestReadbackDeformerGeometry(TUniquePtr<FMeshDeformerGeometryReadbackRequest> InRequest) override;
#endif // WITH_EDITORONLY_DATA
	
	UMeshDeformerInstance* GetInstanceForSourceDeformer() override {return this;};

private:
	/** The Mesh Component that owns this Mesh Deformer Instance. */
	UPROPERTY()
	TWeakObjectPtr<UMeshComponent> MeshComponent;

	/** The settings for this Mesh Deformer Instance. */
	UPROPERTY()
	TWeakObjectPtr<UOptimusDeformerInstanceSettings> InstanceSettings;
	
	/** An array of state. One for each graph owned by the deformer. */
	UPROPERTY()
	TArray<FOptimusDeformerInstanceExecInfo> ComputeGraphExecInfos;

	TMap<FOptimusValueIdentifier, FOptimusValueDescription> ValueMap;
	
	TMap<TWeakObjectPtr<const UComputeDataInterface>, FOptimusDataInterfacePropertyOverrideInfo> DataInterfacePropertyOverrideMap;
	
	TArray<TWeakObjectPtr<UActorComponent>> WeakBoundComponents;
	
	TArray<TWeakObjectPtr<const UOptimusComponentSource>> WeakComponentSources;
	
	// List of graphs that should be run on the next tick. 
	TSet<FName> GraphsToRunOnNextTick;
	FCriticalSection GraphsToRunOnNextTickLock;

	FOptimusPersistentBufferPoolPtr BufferPool;
	FSceneInterface* Scene = nullptr;

	// Data provider responsible for reading back the final deformed geometry after the deformer runs, only assigned when WITH_EDITORONLY_DATA
	TWeakInterfacePtr<IOptimusDeformerGeometryReadbackProvider> WeakGeometryReadbackProvider = nullptr;

	bool bCanBeActive = true;
};

#undef UE_API
