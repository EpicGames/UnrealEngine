// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "OptimusDiagnostic.generated.h"

UENUM()
enum class EOptimusDiagnosticLevel : uint8
{
	None,
	Info,
	Warning,
	Error
};

struct FOptimusCompilerDiagnostic
{
	FOptimusCompilerDiagnostic() = default;
	FOptimusCompilerDiagnostic(const FOptimusCompilerDiagnostic&) = default;
	FOptimusCompilerDiagnostic& operator=(const FOptimusCompilerDiagnostic&) = default;

	// The severity of the issue.
	EOptimusDiagnosticLevel Level = EOptimusDiagnosticLevel::None;

	// The actual diagnostic message
	FText Message;

	// Line location in source
	int32 Line = INDEX_NONE;

	// Starting column (inclusive)
	int32 ColumnStart = INDEX_NONE;

	// Ending column (inclusive)
	int32 ColumnEnd = INDEX_NONE;

	// Absolute path to file associated with this diagnostic
	FString AbsoluteFilePath;

	// UObject associated with this diagnostic
	TWeakObjectPtr<const UObject> Object;
};
