// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#include "OptimusExecutionDomain.generated.h"

#define UE_API OPTIMUSCORE_API

UENUM()
enum class EOptimusExecutionDomainType
{
	DomainName = 0,	
	Expression = 1,		
};

/** A struct to hold onto a single-level domain for controlling a kernel's execution domain. 
  * The reason it's in a struct is so that we can apply a property panel customization for it
  * to make it easier to select from a pre-defined list of execution domains.
*/
USTRUCT()
struct FOptimusExecutionDomain
{
	GENERATED_BODY()

	FOptimusExecutionDomain() = default;
	FOptimusExecutionDomain(FName InExecutionDomainName) :
		Type(EOptimusExecutionDomainType::DomainName),
		Name(InExecutionDomainName)
	{}
	FOptimusExecutionDomain(const FString& InExpression) :
		Type(EOptimusExecutionDomainType::Expression),
		Expression(InExpression)
	{}
	
	UE_API FString AsExpression() const;

	UE_API bool IsDefined() const;
	
private:
	UPROPERTY(EditAnywhere, Category = Domain)
	EOptimusExecutionDomainType Type = EOptimusExecutionDomainType::DomainName;
	
	// The name of the execution domain that this kernel operates on.
	UPROPERTY(EditAnywhere, Category = Domain)
	FName Name;

	UPROPERTY(EditAnywhere, Category = Domain)
	FString Expression;

	friend class FOptimusExecutionDomainCustomization;
	friend struct FOptimusDataDomain;
};



#undef UE_API
