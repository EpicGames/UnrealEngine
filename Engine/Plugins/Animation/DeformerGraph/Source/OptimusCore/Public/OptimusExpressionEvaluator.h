// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"

#define UE_API OPTIMUSCORE_API

template <typename OptionalType> struct TOptional;

namespace Optimus::Expression
{

struct FParseLocation
{
	FParseLocation() = default;
	FParseLocation(int32 InStart, int32 InEndLocation) :
		Start(InStart), 
		End(InEndLocation)
	{}
	FParseLocation(FStringView InParseRange, const TCHAR* InParseStart) :
		Start(static_cast<int32>(InParseRange.begin() - InParseStart)),
		End(static_cast<int32>(InParseRange.end() - InParseStart))
	{}

	/** Merge two parse locations and return a new one */
	FParseLocation Join(const FParseLocation& InLocation) const
	{
		return FParseLocation(FMath::Min(Start, InLocation.Start), FMath::Max(End, InLocation.End));
	}
	
	const int32 Start = INDEX_NONE;
	const int32 End = INDEX_NONE;
};

struct FParseError
{
	FParseError() = default;
	FParseError(FString&& InMessage, FParseLocation InLocation) :
		Message(MoveTemp(InMessage)),
		Location(InLocation)
	{}

	/** The human-readable parsing error */
	const FString Message{};
	
	/** The parse location where the error took place. */
	const FParseLocation Location{};
};

struct FExpressionObject
{
	FString ToString() const;

	/** Returns all named constants used by this expression */
	TSet<FName> GetUsedConstants() const;

	TOptional<FName> GetAsSingleConstant() const;

	void Load(FArchive& Ar);
	void Save(FArchive& Ar) const;
	
private:
	
	enum class EOperator : int32
	{
		Negate,				// Negation operator.
		Add,				// Add last two values on stack and add the result to the stack.
		Subtract,			// Same but subtract 
		Multiply,			// Same but multiply
		Divide,				// Same but divide (div-by-zero returns zero)
		Modulo,				// Same but modulo (mod-by-zero returns zero)
		Power,				// Raise to power
		FloorDivide,    	// Divide and round the result down
	};

	struct FFunctionRef
	{
		FFunctionRef(int32 InIndex) : Index(InIndex) {}
		int32 Index;
	};
	
	friend class FEngine;
	using OpElement = TVariant<EOperator, FName /* Constant */, FFunctionRef, /* Value */ float>;
	TArray<OpElement, TInlineAllocator<8>> Expression;
};

using FParseResult = TVariant<FExpressionObject, FParseError>;
	
/** Simple, fast expression evaluation engine. Can evaluate simple arithmetic expressions,
 *  including user-settable constants. 
 *		1 + 1
 *		foo + (3.0 * bar)
 *
 *  Allowable operands are taken from Python and are currently as follow:
 *      -x				Unary negation
 *      x + y			Addition
 *      x - y			Subtraction
 *      x * y			Multiplication
 *      x / y			Division (div-by-zero returns zero)
 *      x ** y			Raise x to y's power
 *      x // y			Division and rounding down.
 *
 *  Terms can also be grouped by parentheses '(' and ')'.
 *  
 *	Constants that begin with a number, include spaces or any of the above operands/parentheses 
 *	are allowed, but must be then enclosed in single quotes when referenced.
 *	E.g:
 *	   'foo(1)' * 2.0
 *	rather than:
 *	   foo (1) * 2.0
 */


class FEngine
{
public:
	/** Parse an expression and either receive an error or a expression execution object that
	 *  matches this evaluator. */
	UE_API FParseResult Parse(
		FStringView InExpression,
		TFunction<TOptional<float>(FName InConstantName)> InConstantEvaluator = {} 
		) const;

	/** Executes the given expression object.
	 *  \param InExpressionObject The parsed expression object to evaluate.
	 *  \param InConstantEvaluator A function that returns the value of a given constant, if it's known, or none otherwise.
	 */
	UE_API float Execute(
		const FExpressionObject& InExpressionObject,
		TFunctionRef<TOptional<float>(FName InConstantName)> InConstantEvaluator
		) const;
	
	/** Evaluate an expression directly and returns the result. If the expression is incorrectly
	 *  formed, then the optional return value holds nothing.
	 */ 
	UE_API TOptional<float> Evaluate(
		FStringView InExpression,
		TFunctionRef<TOptional<float>(FName InConstantName)> InConstantEvaluator
		) const;

	/** Verify an expression. Does not check for variable validity, only that the expression
	 *  can be evaluated */
	UE_API TOptional<FParseError> Verify(
		FStringView InExpression,
		TFunction<TOptional<float>(FName InConstantName)> InConstantEvaluator = {} 
		) const;
	
private:
	enum class EOperatorToken
	{
		Negate,				// Placeholder for when constructing the op stack. Gets converted to (0 - V)
		Add,				// '+'
		Subtract,			// '-'
		Multiply,			// '*'
		Divide,				// '/'
		Modulo,				// '%'
		Power,				// '**'
		FloorDivide,    	// '//'
		ParenOpen,			// '('
		ParenClose,			// ')'
		Comma,				// ','

		Max					// Must be last
	};
	using FToken = TVariant<EOperatorToken, /* Identifier */ FName, /* Value */ float>;

	struct FTokenParseResult
	{
		FTokenParseResult() = default;
		
		FTokenParseResult(FToken&& InToken, FStringView InTokenRange, const TCHAR* InExpression) :
			Result(TInPlaceType<FToken>(), MoveTemp(InToken)),
			Location(InTokenRange, InExpression)
		{}
		
		FTokenParseResult(FString&& InError, FStringView InTokenRange, const TCHAR* InExpression) :
			Result(TInPlaceType<FString>(), MoveTemp(InError)),
			Location(InTokenRange, InExpression)
		{}
		
		bool IsError() const { return Result.IsType<FString>(); }
		bool IsToken() const { return Result.IsType<FToken>(); }
		
		const FToken& GetToken() const
		{
			return Result.Get<FToken>();
		}
		
		FParseError GetParseError()
		{
			return FParseError(MoveTemp(Result.Get<FString>()), Location);
		}
		
		TVariant<FString, FToken> Result;
		FParseLocation Location{};
	};
	
	/** Parse a floating point value and return the result (or error). Same semantics
	 *  apply as with ParseToken.
	 */
	static UE_API FTokenParseResult ParseFloat(
		FStringView& InOutParseRange,
		FStringView InExpression
		);

	/** Parse a constant reference and return the result (or error). Same semantics
	 *  apply as with ParseToken.
	 */
	static UE_API FTokenParseResult ParseIdentifier(
		FStringView& InOutParseRange,
		FStringView InExpression
		);
	
	/** Parse a single token from the beginning of the range, returning either the token
	 *  or an error, if one was encountered.
	 *  If the token was successfully parsed, then the given string view range gets updated
	 *  with the new starting point at the end of the token just parsed
	 */
	static UE_API FTokenParseResult ParseToken(
		FStringView& InOutParseRange,
		FStringView InExpression
		);

	/** Helper function to get a text version of a token. */
	static UE_API FString TokenToString(
		const FToken& InToken
		);
	
	/** For now we just have built-in functions */
	friend struct FBuiltinFunctions;
	using FunctionType = TFunction<float(TArrayView<const float>)>;
	struct FFunctionInfo
	{
		int32 ArgumentCount;
		FunctionType FunctionPtr;
	};
};

}


static FArchive& operator<<(FArchive& Ar, Optimus::Expression::FExpressionObject& InObject)
{
	if (Ar.IsLoading())
	{
		InObject.Load(Ar);
	}
	else
	{
		InObject.Save(Ar);
	}
	return Ar;
}

#undef UE_API
