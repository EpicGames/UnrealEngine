// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusDeformer.h"

#include "OptimusFunctionNodeGraph.generated.h"

#define UE_API OPTIMUSCORE_API


struct FOptimusFunctionNodeGraphHeaderWithGuid;

USTRUCT()
struct FOptimusFunctionGraphIdentifier
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UOptimusDeformer> Asset;

	UPROPERTY()
	FGuid Guid;

	UOptimusFunctionNodeGraph* Resolve();
	
	bool operator==(const FOptimusFunctionGraphIdentifier& InOtherFunctionGraph) const = default;
};


/**
 * 
 */
UCLASS(MinimalAPI)
class UOptimusFunctionNodeGraph :
	public UOptimusNodeSubGraph
{
	GENERATED_BODY()

public:
	static UE_API FName AccessSpecifierPublicName;
	static UE_API FName AccessSpecifierPrivateName;

	UE_API UOptimusFunctionNodeGraph();
	UE_API void PostLoad() override;
	
	/** The name to give the node based off of this graph */
	UE_API FString GetNodeName() const; 

	UE_API void Init();
	UE_API FOptimusFunctionGraphIdentifier GetGraphIdentifier() const;

	/** The category of the node based of of this graph for listing purposes */ 
	UPROPERTY(EditAnywhere, Category=Settings)
	FName Category = UOptimusNode::CategoryName::Deformers;

	UPROPERTY(EditAnywhere, Category=Settings, meta=(GetOptions="GetAccessSpecifierOptions"))
	FName AccessSpecifier = AccessSpecifierPrivateName;

	UFUNCTION()
	UE_API TArray<FName> GetAccessSpecifierOptions() const;

	UE_API FOptimusFunctionNodeGraphHeaderWithGuid GetHeaderWithGuid() const;
	UE_API FGuid GetGuid() const;

	// Used for PostLoad fixup to ensure function references hold on to the same Guid that the graph
	// would get when it calls PostLoad()
	static UE_API FGuid GetGuidForGraphWithoutGuid(TSoftObjectPtr<UOptimusFunctionNodeGraph> InGraph);
protected:
	
#if WITH_EDITOR
	// UObject override
	UE_API bool CanEditChange(const FProperty* InProperty) const override;
#endif

	UPROPERTY(VisibleAnywhere, Category=Debug, AdvancedDisplay)
	FGuid Guid;
};

#undef UE_API
