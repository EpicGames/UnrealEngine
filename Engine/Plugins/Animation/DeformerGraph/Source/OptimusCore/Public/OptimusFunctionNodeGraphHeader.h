// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "OptimusFunctionNodeGraphHeader.generated.h"

class UOptimusFunctionNodeGraph;
class UOptimusNode_FunctionReference;

USTRUCT()
struct FOptimusFunctionNodeGraphHeader
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UOptimusFunctionNodeGraph> GraphPath;

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	FName Category;
};

USTRUCT()
struct FOptimusFunctionNodeGraphHeaderArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FOptimusFunctionNodeGraphHeader> Headers;
};

USTRUCT()
struct FOptimusFunctionReferenceNodeSet
{
	GENERATED_BODY()

	UPROPERTY()
	TSet<TSoftObjectPtr<UOptimusNode_FunctionReference>> Nodes;
};

USTRUCT()
struct FOptimusFunctionReferenceData
{
	GENERATED_BODY()

	// Group function reference nodes by function node graph path
	// using FSoftObjectPath instead of TSoftObjectPtr such that ExportText(...) is deterministic
	UPROPERTY()
	TMap<FSoftObjectPath, FOptimusFunctionReferenceNodeSet> FunctionReferences;
};
