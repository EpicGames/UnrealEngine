// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "OptimusFunctionNodeGraphHeaderWithGuid.generated.h"

class UOptimusNode_FunctionReference;

USTRUCT()
struct FOptimusFunctionNodeGraphHeaderWithGuid
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid FunctionGraphGuid;

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	FName Category;
};

USTRUCT()
struct FOptimusFunctionNodeGraphHeaderWithGuidArray
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FOptimusFunctionNodeGraphHeaderWithGuid> Headers;
};

