// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Math/TransformCalculus.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/Class.h"
#include "OptimusExpressionEvaluator.h"


class UOptimusComponentSource;
struct FShaderValueContainerView;
class FShaderParametersMetadata;
class FShaderParametersMetadataBuilder;
struct FShaderValueTypeHandle;
class UActorComponent;
class USkeletalMeshComponent;
class FSkeletalMeshLODRenderData;

namespace Optimus
{
	/** Attempts to find an object, first within a specific package, if the dot prefix
	  * points to a known package, otherwise fall back to searching globally. */
	template<typename T>
	T* FindObjectInPackageOrGlobal(const FString& InObjectPath)
	{
		UPackage* Package = nullptr;
		FString PackageName;
		FString PackageObjectName = InObjectPath;
		if (InObjectPath.Split(TEXT("."), &PackageName, &PackageObjectName))
		{
			Package = FindPackage(nullptr, *PackageName);
		}

		T* FoundObject = FindObject<T>(Package, *PackageObjectName);

		// If not found in a specific, search everywhere.
		if (FoundObject == nullptr)
		{
			FoundObject = FindFirstObject<T>(*InObjectPath, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Optimus::FindObjectInPackageOrGlobal"));
		}

		return FoundObject;
	}

	/** Given an object scope, ensure that the given name is unique within that scope.
	    If the name is already unique, it will be returned unchanged. */
	FName GetUniqueNameForScope(UObject *InScopeObj, FName InName);

	/** Given an object scope, generate names that is not only unique within the scope,
	 * but also unique among all names generated before.
	 * Note: It uses GetUniqueNameForScope so using the same scope object and
	 * the same input name does not guarantee the same output name
	 */
	struct FUniqueNameGenerator
	{
		explicit FUniqueNameGenerator(UObject* InScopeObject);
		FName GetUniqueName(FName InName);
		
	private:	
		UObject* ScopeObject = nullptr;
		TArray<FName> GeneratedName;
	};
	
	/** A small helper class to enable binary reads on an archive, since the 
		FObjectReader::Serialize(TArray<uint8>& InBytes) constructor is protected */
	class FBinaryObjectReader : public FObjectReader
	{
	public:
		FBinaryObjectReader(UObject* Obj, const TArray<uint8>& InBytes)
			// FIXME: The constructor is broken. It only needs a const ref.
		    : FObjectReader(const_cast<TArray<uint8>&>(InBytes))
		{
			FObjectReader::SetWantBinaryPropertySerialization(true);
			Obj->Serialize(*this);
		}
	};

	class FBinaryObjectWriter : public FObjectWriter
	{
	public:
		FBinaryObjectWriter(UObject* Obj, TArray<uint8>& OutBytes)
		    : FObjectWriter(OutBytes)
		{
			FObjectWriter::SetWantBinaryPropertySerialization(true);
			Obj->Serialize(*this);
		}
	};

	FName GetSanitizedNameForHlsl(FName InName);

	inline FMatrix44f ConvertFTransformToFMatrix44f(const FTransform& InTransform)
	{
		return TransformConverter<FMatrix44f>::Convert<FMatrix44d>(InTransform.ToMatrixWithScale());	
	};
	
	const void ConvertFTransformToFMatrix3x4(const FTransform& InTransform, FShaderValueContainerView OutShaderValue);
	
	bool RenameObject(UObject* InObjectToRename, const TCHAR* InNewName, UObject* InNewOuter);

	/** Use this function to remove objects during postload safely */
	void RemoveObject(UObject* InObjectToRemove);

	/** Our generated classes are parented to the package, this is a utility function
		to collect them */
	TArray<UClass*> GetClassObjectsInPackage(UPackage* InPackage);

	/** Return the unique type name for registry and kernel code generation if the bInShouldGetUniqueNameForUserDefinedStruct = true
		Otherise, it returns the friendly name for user-facing shader text display*/
	OPTIMUSCORE_API FName GetTypeName(UScriptStruct* InStruct, bool bInShouldGetUniqueNameForUserDefinedStruct = true);
	
	/** Return the unique type name for registry and kernel code generation.
	    Similar to the UScriptStruct version (with bInShouldGetUniqueNameForUserDefinedStruct true) but doesn't require loading the asset. */
	OPTIMUSCORE_API FName GetTypeName(const FAssetData& InStructAsset);

	/** Return the display name for the struct to be shown in type pickers*/
	OPTIMUSCORE_API FText GetTypeDisplayName(UScriptStruct* InStruct);

	/** Helper function to remove guids from member property names for user defined structs	*/
	OPTIMUSCORE_API FName GetMemberPropertyShaderName(UScriptStruct* InStruct, const FProperty* InMemberProperty);

	/** Helpers to convert UObject path to a virtual shader paths. */
	void ConvertObjectPathToShaderFilePath(FString& InOutPath);
	/** Helpers to convert a virtual shader path back to a UObject path. Returns false if the virtual shader path wasn't recognized as a UObject path. */
	bool ConvertShaderFilePathToObjectPath(FString& InOutPath);

	FString GetCookedKernelSource(
		const FString& InObjectPathName,
		const FString& InShaderSource,
		const FString& InKernelName,
		FIntVector InGroupSize,
		const TCHAR* InReadNumThreadsPerInvocationFunctionName,
		const TCHAR* InReadThreadIndexOffsetFunctionName,
		bool bInIsUnifiedDispatch
	);

	const TCHAR* GetKernelInternalNamespaceName();

	Expression::FParseResult ParseExecutionDomainExpression(
		const FString& InExpression,
		TWeakObjectPtr<const UOptimusComponentSource> InComponentSource
		);

	TVariant<bool, FText> IsExecutionDomainUnifiedDispatchOnly(
		const FString& InExpression,
		TWeakObjectPtr<const UOptimusComponentSource> InComponentSource
		);
	
	bool EvaluateExecutionDomainExpressionParseResult(
		const TVariant<Expression::FExpressionObject, Expression::FParseError>& InParseResult,
		TWeakObjectPtr<const UOptimusComponentSource> InComponentSource,
		TWeakObjectPtr<const UActorComponent> InWeakComponent,
		TArray<int32>& OutInvocationThreadCount
		);
	
	bool FindMovedItemInNameArray(const TArray<FName>& Old, const TArray<FName>& New, FName& OutSubjectName, FName& OutNextName);

	FName GenerateUniqueNameFromExistingNames(FName InBaseName, const TArray<FName>& InExistingNames);

	FString MakeUniqueValueName(const FString& InValueName, int32 InUniqueIndex);
	FString ExtractSourceValueName(const FString& InUniqueValueName);

	bool IsSkinWeightProfileAvailable(const FSkeletalMeshLODRenderData& InLODRenderData, FName InSkinWeightProfile);
	
	void RequestSkinWeightProfileForDeformer(USkeletalMeshComponent* InSkeletalMeshComponent, FName InSkinWeightProfile, TOptional<int32> InLOD = {});
}
