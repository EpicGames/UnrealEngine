// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeKernelSource.h"
#include "OptimusKernelSource.generated.h"

UCLASS(MinimalAPI)
class UOptimusKernelSource :
	public UComputeKernelSource
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	void SetSource(FString const& InSource)
	{
		Source = InSource;
	}
	
	//~ Begin UComputeKernelSource Interface.
	FString GetSource() const override
	{
		return Source;
	}
	//~ End UComputeKernelSource Interface.
#endif

protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString Source;
#endif
};
