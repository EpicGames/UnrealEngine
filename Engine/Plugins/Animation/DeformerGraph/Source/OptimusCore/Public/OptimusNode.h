// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusCoreNotify.h"
#include "OptimusDiagnostic.h"

#include "CoreMinimal.h"
#include "OptimusNodePin.h"
#include "UObject/Object.h"

#include "OptimusNode.generated.h"

#define UE_API OPTIMUSCORE_API

struct FOptimusNodePinStorageConfig;
enum class EOptimusNodePinDirection : uint8;
enum class EOptimusNodePinStorageType : uint8;
class UOptimusActionStack;
class UOptimusNodeGraph;
class UOptimusNodePin;
struct FOptimusCompoundAction;
struct FOptimusDataTypeRef;
struct FOptimusParameterBinding;

UCLASS(MinimalAPI, Abstract)
class UOptimusNode : public UObject
{
	GENERATED_BODY()
public:
	struct CategoryName
	{
		OPTIMUSCORE_API static const FName DataInterfaces;
		OPTIMUSCORE_API static const FName Deformers;
		OPTIMUSCORE_API static const FName Resources;
		OPTIMUSCORE_API static const FName Variables;
		OPTIMUSCORE_API static const FName Values;
	};

	struct PropertyMeta
	{
		static const FName Category;
		static const FName Input;
		static const FName Output;
		static const FName Resource;
		OPTIMUSCORE_API static const FName AllowParameters;
	};
public:
	UE_API UOptimusNode();
	UE_API virtual ~UOptimusNode();
	/** 
	 * Returns the node class category. This is used for categorizing the node for display.
	 * @return The node class category.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	virtual FName GetNodeCategory() const PURE_VIRTUAL(, return NAME_None;);

	/** Returns true if the node can be deleted by the user */
	virtual bool CanUserDeleteNode() const { return true; }
	
	/** Recreate the pins from the definition */
	virtual void RecreatePinsFromPinDefinitions() {};

	/** Rename the pin from the definition */
	virtual void RenamePinFromPinDefinition(FName InOld, FName InNew) {};

	/** Update the display name */
	virtual void UpdateDisplayNameFromDataInterface() {};

	/**
	 * Returns the node class name. This name is immutable for the given node class.
	 * @return The node class name.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API FName GetNodeName() const;

	/**
	 * Returns the display name to use on the graphical node in the graph editor.
	 * @return The display name to show to the user.
	*/ 
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API virtual FText GetDisplayName() const;

	/**
	 * Set the display name for this node.
	 * @param InDisplayName The display name to use. Can not be empty. 
	 * @return true if the call was successful and the node name updated.
	 */ 
	UE_API bool SetDisplayName(FText InDisplayName);

	/**
	 * Returns the tool tip.
	 * @return The tool tip to show to the user.
	 */
	UE_API virtual FText GetTooltipText() const;
	
	/**
	 * Sets the position in the graph UI that the node should be shown at.
	 * @param InPosition The coordinates of the node's position.
	 * @return true if setting the position was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool SetGraphPosition(const FVector2D& InPosition);

	/**
	 * Returns the position in the graph UI where the node is shown.
	 * @return The coordinates of the node's position.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	FVector2D GetGraphPosition() const { return GraphPosition; }

	/**
	 * Returns the absolute path of the node. This can be passed to this graph collection root's
	 * IOptimusPathResolver interface to resolve to a node object.
	 * @return The absolute path of this node, rooted within the deformer.
	 */
	UE_API FString GetNodePath() const;

	/** Returns the owning node graph of this node. */
	UE_API UOptimusNodeGraph *GetOwningGraph() const;

	/** Returns the list of all the pins on this node */
	TArrayView<UOptimusNodePin* const> GetPins() const { return Pins; }

	/** Return the pin with a specific name */
	UE_API UOptimusNodePin* GetPinByName(const FName& PinName) const;

	/** Returns the list of pins on this node filtered by direction */
	UE_API TArray<UOptimusNodePin*> GetPinsByDirection(EOptimusNodePinDirection InDirection, bool bInRecursive) const;
	
	/**
	 * Preliminary check for whether valid connection can be made between two existing pins.
	 * Can be overridden by derived nodes to add their own additional checks. 
	 * @param InThisNodesPin The pin on this node that about to be connected
	 * @param InOtherNodesPin The pin that is about to be connect to this node
	 * @param OutReason The reason that the connection is cannot be made if it is invalid
	 * @return true if the other Pin can be connected to the specified side of the node.
	 */
	UE_API bool CanConnectPinToPin(
		const UOptimusNodePin& InThisNodesPin,
		const UOptimusNodePin& InOtherNodesPin,
		FString* OutReason = nullptr
		) const;

	/**
	 * Preliminary check for whether valid connection can be made between an existing pin and
	 * a potentially existing pin on this node. 
	 * @param InOtherPin The pin that is about to be connect to this node
	 * @param InConnectionDirection The input/output side of the node to connect
	 * @param OutReason The reason that the connection is cannot be made if it is invalid
	 * @return true if the other Pin can be connected to the specified side of the node.
	 */
	UE_API bool CanConnectPinToNode(
		const UOptimusNodePin* InOtherPin,
		EOptimusNodePinDirection InConnectionDirection,
		FString* OutReason = nullptr
		) const;
	
	/**
	 * Returns the node's diagnostic level (e.g. error state). For a node, only None, Warning,
	 * and Error are relevant.
	 */
	EOptimusDiagnosticLevel GetDiagnosticLevel() const { return DiagnosticLevel; }

	/**
	 * Sets the node diagnostic level (e.g. error state).
	 */
	UE_API void SetDiagnosticLevel(EOptimusDiagnosticLevel InDiagnosticLevel);

	/** Find the pin associated with the given dot-separated pin path.
	 * @param InPinPath The path of the pin.
	 * @return The pin object, if found, otherwise nullptr.
	 */
	UE_API UOptimusNodePin* FindPin(const FStringView InPinPath) const;

	/** Find the pin from the given path array. */
	UE_API UOptimusNodePin* FindPinFromPath(const TArray<FName>& InPinPath) const;

	/**
	 * Find the pin associated with the given FProperty object(s).
	 * @param InRootProperty The property representing the pin root we're interested in.
	 * @param InSubProperty The property representing the actual pin the value changed on.
	 * @return The pin object, if found, otherwise nullptr.
	 */
	UE_API UOptimusNodePin* FindPinFromProperty(
	    const FProperty* InRootProperty,
	    const FProperty* InSubProperty
		) const;

	/**
	 * Returns the class of all non-deprecated UOptimusNodeBase nodes that are defined,
	 * in no particular order.
	 * @return List of all classes that derive from UOptimusNodeBase.
	 */
	static UE_API TArray<UClass*> GetAllNodeClasses();

	/**
	 * Called just after the node is created, either via direct creation or deletion undo.
	 * By default it creates the pins representing connectable properties.
	 */
	UE_API void PostCreateNode();

	/**
	 * Called just after the node is deserialized, either via loading or pasting.
	 * It does not recreate the pins, instead it only initializes transient data.
	 */
	UE_API void PostDeserializeNode();

	//== UObject overrides
	UE_API void Serialize(FArchive& Ar) override;
	
	// Using "final" here to make sure all derive nodes have InitializeTransientData() automatically called on them during PostLoad(),
	// so please use PostLoadNodeSpecificData instead for any PostLoad fix-ups
	UE_API void PostLoad() override final;

	// Derived nodes should override this function for any PostLoad fix-ups
	UE_API virtual void PostLoadNodeSpecificData();
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	bool IsCreatedFromUI() const {return bCreatedFromUI;}
#endif
	
protected:
	friend class FOptimusEditorClipboard;
	friend class UOptimusNodeGraph;
	friend class UOptimusNodeSubGraph;
	friend class UOptimusNodePin;
	friend class UOptimusDeformer;
	friend struct FOptimusNodeAction_AddRemovePin;
	friend struct FOptimusNodeAction_MoveNode;
	friend struct FOptimusNodeAction_MovePin;
	friend struct FOptimusNodeAction_SetPinType;
	friend struct FOptimusNodeAction_SetPinName;
	friend struct FOptimusNodeAction_SetPinDataDomain;
	friend struct FOptimusNodeGraphAction_PackageKernelFunction;
	friend struct FOptimusNodeGraphAction_UnpackageKernelFunction;
	friend struct FOptimusNodeGraphAction_RemoveNode;
	friend struct FOptimusNodeGraphAction_DuplicateNode;

	/**
	 * Returns a unique name for pin. If given the same pins to compare to, the output
	 * name will always be the same, unlike Optimus::GetUniqueNameForScope, which gives
	 * a different name each time it is called, use this function if pin names are generated during
	 * an action instead of prior to an action. Though ideally there should be only one way to generate
	 * pin names
	 */
	static UE_API FName GetAvailablePinNameStable(const UObject* InNodeOrPin, FName InName);
	
	// Return the action stack for this node.
	UE_API UOptimusActionStack* GetActionStack() const;

	// Called when the node is being constructed
	UE_API virtual void ConstructNode();

	// Allows the node to initialize any transient data that can be derived from serialized properties
	UE_API virtual void InitializeTransientData();

	/** Allows the node to initialize all the pins display names from the definitions */
	virtual void InitializePinsDisplayName() {}

	virtual bool ValidateConnection(
		const UOptimusNodePin& InThisNodesPin,
		const UOptimusNodePin& InOtherNodesPin,
		FString* OutReason
		) const
	{
		return true;
	}

	/** Optional: Perform local node validation for compilation. If failed, return the
	 *  reason as a part of the optional return value. If success return an empty optional
	 *  object.
	 */
	virtual TOptional<FText> ValidateForCompile(const FOptimusPinTraversalContext& InContext) const
	{
		return {};
	}

	/** Called prior to duplicate to allow the node to add its own graph requirements to
	 *  to the list of actions being performed.
	 */
	virtual void PreDuplicateRequirementActions(
		const UOptimusNodeGraph* InTargetGraph, 
		FOptimusCompoundAction *InCompoundAction) {}

	UE_API virtual void ExportState(FArchive& Ar) const;
	UE_API virtual void ImportState(FArchive& Ar);

	UE_API void EnableDynamicPins();

	virtual void OnDataTypeChanged(FName InTypeName) {};

	UE_API UOptimusNodePin* AddPin(
		FName InName,
		EOptimusNodePinDirection InDirection,
		const FOptimusDataDomain& InDataDomain,
		FOptimusDataTypeRef InDataType,
		UOptimusNodePin* InBeforePin = nullptr,
		UOptimusNodePin* InGroupingPin = nullptr
		);

	/** Add a new pin based on a parameter binding definition. Only allowed for top-level pins. */
	UE_API UOptimusNodePin* AddPin(
		const FOptimusParameterBinding& InBinding,
		EOptimusNodePinDirection InDirection,
		UOptimusNodePin* InBeforePin = nullptr
		);

	/** Create a pin and add it to the node in the location specified. */ 
	UE_API UOptimusNodePin* AddPinDirect(
		FName InName,
		EOptimusNodePinDirection InDirection,
		const FOptimusDataDomain& InDataDomain,
		FOptimusDataTypeRef InDataType,
		UOptimusNodePin* InBeforePin = nullptr,
		UOptimusNodePin* InParentPin = nullptr
		);

	/** Add a new pin based on a parameter binding definition. Only allowed for top-level pins. */
	UE_API UOptimusNodePin* AddPinDirect(
		const FOptimusParameterBinding& InBinding,
		EOptimusNodePinDirection InDirection,
		UOptimusNodePin* InBeforePin = nullptr
		);

	/** Add a new grouping pin. This is a pin that takes no connections but is shown as
	  * collapsible in the node UI. */
	UE_API UOptimusNodePin* AddGroupingPin(
		FName InName,
		EOptimusNodePinDirection InDirection,
		UOptimusNodePin* InBeforePin = nullptr
		);
	
	UE_API UOptimusNodePin* AddGroupingPinDirect(
		FName InName,
		EOptimusNodePinDirection InDirection,
		UOptimusNodePin* InBeforePin = nullptr
		);

	// Remove a pin.
	UE_API bool RemovePin(
		UOptimusNodePin* InPin
		);

	// Remove the pin with no undo.
	UE_API bool RemovePinDirect(
		UOptimusNodePin* InPin
		);

	/** Swap two sibling pins */
	UE_API bool MovePin(
		UOptimusNodePin* InPinToMove,
		const UOptimusNodePin* InPinBefore
		);
	
	UE_API bool MovePinDirect(
		UOptimusNodePin* InPinToMove,
		const UOptimusNodePin* InPinBefore
		);

	UE_API bool MovePinToGroupPinDirect(
		UOptimusNodePin* InPinToMove,
		UOptimusNodePin* InGroupPin
		);
	
	/** Set the pin data type. */
	UE_API bool SetPinDataType(
		UOptimusNodePin* InPin,
		FOptimusDataTypeRef InDataType
		);
	
	UE_API bool SetPinDataTypeDirect(
		UOptimusNodePin* InPin,
		FOptimusDataTypeRef InDataType
		);

	/** Set the pin name. */
	// FIXME: Hoist to public
	UE_API bool SetPinName(
		UOptimusNodePin* InPin,
		FName InNewName
		);
	
	UE_API bool SetPinNameDirect(
	    UOptimusNodePin* InPin,
	    FName InNewName
		);

	/// @brief Set a new position of the node in the graph UI.
	/// @param InPosition The coordinates of the new position.
	/// @return true if the position setting was successful (i.e. the coordinates are valid).
	UE_API bool SetGraphPositionDirect(
		const FVector2D &InPosition
		);
	
	/** Set the pin's resource context names. */
	UE_API bool SetPinDataDomain(
		UOptimusNodePin* InPin,
		const FOptimusDataDomain& InDataDomain
		);

	UE_API bool SetPinDataDomainDirect(
		UOptimusNodePin* InPin,
		const FOptimusDataDomain& InDataDomain
		);
	
	UE_API void SetPinExpanded(const UOptimusNodePin* InPin, bool bInExpanded);
	UE_API bool GetPinExpanded(const UOptimusNodePin* InPin) const;

	// A sentinel to indicate whether sending notifications is allowed.
	bool bSendNotifications = true;
	
private:
	UE_API void InsertPinIntoHierarchy(
		UOptimusNodePin* InNewPin, 
		UOptimusNodePin* InParentPin,
		UOptimusNodePin* InInsertBeforePin
		);
	
	UE_API void Notify(
		EOptimusGraphNotifyType InNotifyType
		);

	bool CanNotify() const
	{
		return !bConstructingNode && bSendNotifications;
	}
	
	
	UE_API void CreatePinsFromStructLayout(
		const UStruct *InStruct, 
		UOptimusNodePin *InParentPin = nullptr
		);

	UE_API UOptimusNodePin* CreatePinFromProperty(
	    EOptimusNodePinDirection InDirection,
		const FProperty* InProperty,
		UOptimusNodePin* InParentPin = nullptr
		);

	// The display name to show. This is non-transactional because it is controlled by our 
	// action system rather than the transacting system for undo.
	UPROPERTY(NonTransactional)
	FText DisplayName;

	// Node layout data
	UPROPERTY(NonTransactional)
	FVector2D GraphPosition;

	// The list of pins. Non-transactional for the same reason as above. 
	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UOptimusNodePin> > Pins;

	// The list of pins that should be shown as expanded in the graph view.
	UPROPERTY(NonTransactional)
	TSet<FName> ExpandedPins;

	UPROPERTY()
	EOptimusDiagnosticLevel DiagnosticLevel = EOptimusDiagnosticLevel::None;

	// Set to true if the node is dynamic and can have pins arbitrarily added.
	bool bDynamicPins = false;

	// A sentinel to indicate we're doing node construction.
	bool bConstructingNode = false;
	
#if WITH_EDITOR
	// Optionally one can mark this node as created from UI during node creation for the Editor
	// to do special things when the node is spawned
	bool bCreatedFromUI = false;
#endif
	/// Cached pin lookups
	mutable TMap<TArray<FName>, UOptimusNodePin*> CachedPinLookup;
};

#undef UE_API
