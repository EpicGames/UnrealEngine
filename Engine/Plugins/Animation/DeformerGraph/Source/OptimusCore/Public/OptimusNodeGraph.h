// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusNodeGraphCollectionOwner.h"
#include "IOptimusNodeAdderPinProvider.h"
#include "OptimusCoreNotify.h"
#include "OptimusDataType.h"

#include "Templates/SubclassOf.h"

#include "OptimusNodeGraph.generated.h"

#define UE_API OPTIMUSCORE_API

class IOptimusNodePairProvider;
class IOptimusComputeKernelProvider;
class UOptimusNode_LoopTerminal;
class UOptimusComponentSourceBinding;
struct FOptimusCompoundAction;
struct FOptimusPinTraversalContext;
struct FOptimusRoutedNodePin;
class UOptimusVariableDescription;
class UOptimusResourceDescription;
class UOptimusComputeDataInterface;
class IOptimusNodeGraphCollectionOwner;
class UOptimusActionStack;
class UOptimusNode;
class UOptimusNodePair;
class UOptimusNodeGraph;
class UOptimusNodeSubGraph;
class UOptimusNodeLink;
class UOptimusNodePin;
enum class EOptimusNodePinDirection : uint8;
template<typename T> class TFunction;

/** The use type of a particular graph */ 
UENUM()
enum class EOptimusNodeGraphType
{
	// Execution graphs
	Setup,					/** Called once during an actor's setup event */
	Update,					/** Called on every tick */
	ExternalTrigger,		/** Called when triggered from a blueprint */
	// Storage graphs
	Function,				/** Used to store function graphs */
	SubGraph,				/** Used to store sub-graphs within other graphs */ 
	Transient				/** Used to store nodes during duplication. Never serialized. */
};

enum class EOptimusNodePinTraversalDirection
{
	Default,
	Upstream,
	Downstream
};

namespace Optimus
{
static bool IsExecutionGraphType(EOptimusNodeGraphType InGraphType)
{
	return InGraphType == EOptimusNodeGraphType::Setup ||
		   InGraphType == EOptimusNodeGraphType::Update ||
		   InGraphType == EOptimusNodeGraphType::ExternalTrigger;
}
}


UCLASS(MinimalAPI, BlueprintType)
class UOptimusNodeGraph :
	public UObject,
	public IOptimusNodeGraphCollectionOwner
{
	GENERATED_BODY()

public:
	// Reserved names
	static UE_API const FName SetupGraphName;
	static UE_API const FName UpdateGraphName;
	static UE_API const TCHAR* LibraryRoot;
	static UE_API const FName DefaultSubGraphName;
	static UE_API const FName DefaultSubGraphRefNodeName;
	
	// Function Graphs are addressed in a special way
	static UE_API FString GetFunctionGraphCollectionPath(const FString& InFunctionName);

	
	// Check if the duplication took place at the asset level
	// if so, we have to recreate all constant/attribute nodes such that their class pointers
	// don't point to classes in the source asset. This can happen because generated class
	// in the source package/asset are not duplicated automatically to the new package/asset
	UE_API void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

	UE_API void PostLoad() override;

	UE_API UOptimusNodeGraph *GetParentGraph() const;
	
	/** Verify if the given name is a valid graph name. */
	static UE_API bool IsValidUserGraphName(
		const FString& InGraphName,
		FText* OutFailureReason = nullptr
		);

	static UE_API FString ConstructPath(const FString& GraphPath, const FString& NodeName, const FString& PinPath);
	
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	EOptimusNodeGraphType GetGraphType() const { return GraphType; }

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	bool IsExecutionGraph() const
	{
		return Optimus::IsExecutionGraphType(GraphType); 
	}

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	bool IsFunctionGraph() const
	{
		return GraphType == EOptimusNodeGraphType::Function; 
	}

	UE_API bool IsReadOnly() const;

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API int32 GetGraphIndex() const;


	/// @brief Returns the modify event object that can listened to in case there are changes
	/// to the graph that need to be reacted to.
	/// @return The node core event object.
	UE_API FOptimusGraphNotifyDelegate &GetNotifyDelegate();

	// Editor/python functions. These all obey undo/redo.

	// TODO: Add magic connection from a pin.
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddNode(
		const TSubclassOf<UOptimusNode> InNodeClass,
		const FVector2D& InPosition
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddValueNode(
		FOptimusDataTypeRef InDataTypeRef,
		const FVector2D& InPosition
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddDataInterfaceNode(
		const TSubclassOf<UOptimusComputeDataInterface> InDataInterfaceClass,
		const FVector2D& InPosition
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API TArray<UOptimusNode*> AddLoopTerminalNodes(
		const FVector2D& InPosition
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddCommentNode(
		const FVector2D& InPosition,
		const FVector2D& InSize,
		const FLinearColor& InColor
	);
	
	UE_API UOptimusNode* AddCommentNode(
    		const FVector2D& InPosition,
    		const FVector2D& InSize,
    		const FLinearColor& InColor,
    		bool bCreatedFromUI
    );

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddFunctionReferenceNode(
		UOptimusFunctionNodeGraph* InFunctionGraph,
		const FVector2D& InPosition
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddResourceNode(
		UOptimusResourceDescription* InResourceDesc,
		const FVector2D& InPosition);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddResourceGetNode(
	    UOptimusResourceDescription *InResourceDesc,
	    const FVector2D& InPosition);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddResourceSetNode(
	    UOptimusResourceDescription* InResourceDesc,
	    const FVector2D& InPosition);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddVariableGetNode(
	    UOptimusVariableDescription* InVariableDesc,
	    const FVector2D& InPosition
	    );

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* AddComponentBindingGetNode(
		UOptimusComponentSourceBinding* InComponentBinding,
		const FVector2D& InPosition
		);
	

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RemoveNode(
		UOptimusNode* InNode
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RemoveNodes(
		const TArray<UOptimusNode*>& InNodes
	);

	UE_API int32 RemoveNodesAndCount(
		const TArray<UOptimusNode*>& InNodes
	);

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* DuplicateNode(
		UOptimusNode* InNode,
	    const FVector2D& InPosition
	);

	/// Duplicate a collection of nodes from the same graph, using the InPosition position
	/// to be the top-left origin of the pasted nodes.
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool DuplicateNodes(
		const TArray<UOptimusNode*> &InNodes,
		const FVector2D& InPosition
	);
	UE_API bool DuplicateNodes(
		const TArray<UOptimusNode*> &InNodes,
		const FVector2D& InPosition,
		const FString& InActionName
	);
	
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool AddLink(
		UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin
	);

	/// @brief Removes a single link between two nodes.
	// FIXME: Use UOptimusNodeLink instead.
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RemoveLink(
		UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin
	);

	/// @brief Removes all links to the given pin, whether it's an input or an output pin.
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RemoveAllLinks(
		UOptimusNodePin* InNodePin
	);

	// Node Packaging
	/** Takes a custom kernel and converts to a packaged function. If the given node is not a
	 *  custom kernel or cannot be converted, a nullptr is returned.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* ConvertCustomKernelToFunction(UOptimusNode *InCustomKernel);
	
	/** Takes a kernel function and unpackages to a custom kernel. If the given node is not a 
	 *  kernel function or cannot be converted, a nullptr is returned.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode* ConvertFunctionToCustomKernel(UOptimusNode *InKernelFunction);

	/** Take a set of nodes and collapse them into a single function, replacing the given nodes
	 *  with the new function node and returning it. A new function definition is made available
	 *  as a new Function graph in the package.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode *CollapseNodesToFunction(const TArray<UOptimusNode*>& InNodes);

	/** Take a set of nodes and collapse them into a subgraph, replacing the given nodes
	 *  with a new subgraph node and returning it. 
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API UOptimusNode *CollapseNodesToSubGraph(const TArray<UOptimusNode*>& InNodes);
	
	/** Take a function or subgraph node and expand it in-place, replacing the given function 
	 *  node. The function definition still remains, if a function node was expanded. If a
	 *  sub-graph was expanded, the sub-graph is deleted.
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API TArray<UOptimusNode *> ExpandCollapsedNodes(UOptimusNode* InGraphReferenceNode);
	
	/** Take a subgraph node convert it to a function in-place
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool ConvertToFunction(UOptimusNode* InSubGraphNode);

	/** Take a function node convert it to a subgraph node in-place
	 */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool ConvertToSubGraph(UOptimusNode* InFunctionNode);
	
	/** Returns true if the node in question is a custom kernel node that can be converted to
	  * a kernel function with ConvertCustomKernelToFunction.
	  */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool IsCustomKernel(UOptimusNode *InNode) const;
	
	/** Returns true if the node in question is a kernel function node that can be converted to
	  * a custom kernel using ConvertFunctionToCustomKernel. 
	  */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool IsKernelFunction(UOptimusNode *InNode) const;

	/** Returns true if the node in question is a function reference node that can be expanded 
	 *  into a group of nodes using ExpandFunctionToNodes.
	  */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool IsFunctionReference(UOptimusNode *InNode) const;

	/** Returns true if the node in question is a function sub-graph node that can be expanded 
	 *  into a group of nodes using ExpandFunctionToNodes.
	  */
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool IsSubGraphReference(UOptimusNode *InNode) const;

	/** Returns the node pair given a IOptimusNodePairProvider
	  */
	UE_API UOptimusNodePair* GetNodePair(const UOptimusNode* InNode) const;
	
	/** Returns the paired node for the given IOptimusNodePairProvider
	  */
	UE_API UOptimusNode* GetNodeCounterpart(const UOptimusNode* InNode) const;
	
	/** Returns all pins that have a _direct_ connection to this pin. If nothing is connected 
	  * to this pin, it returns an empty array.
	  */
	UE_API TArray<UOptimusNodePin *> GetConnectedPins(
		const UOptimusNodePin* InNodePin
		) const;

	/** See UOptimusNodePin::GetConnectedRoutedPins for information on what this function
	 *  does.
	 */
	UE_API TArray<FOptimusRoutedNodePin> GetConnectedPinsWithRouting(
		const UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InContext
		) const;
	
	UE_API TArray<FOptimusRoutedNodePin> GetConnectedPinsWithRouting(
		const UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InContext,
		EOptimusNodePinTraversalDirection Direction
		) const;

	/** Get all unique component bindings that lead to this pin. Note that only pins with a zero or single bindings
	 *  are considered valid. We return all of them however for error messaging.
	 */
	UE_API TSet<UOptimusComponentSourceBinding*> GetComponentSourceBindingsForPin(
		const UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InContext
		) const;

	/** Check if a pin represents time varying data */
	UE_API bool IsPinMutable(
		const UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InContext
		) const;

	/** Check if a Node has mutable input pins */
	UE_API bool DoesNodeHaveMutableInput(
		const UOptimusNode* InNode,
		const FOptimusPinTraversalContext& InContext
		) const;

	/** Gather connected loop entry terminals */
	UE_API TSet<FOptimusRoutedConstNode> GetLoopEntryTerminalForPin(
		const UOptimusNodePin* InNodePin,
		const FOptimusPinTraversalContext& InContext = {}	
		) const;

	/** Gather connected loop entry terminals */
	UE_API TSet<FOptimusRoutedConstNode> GetLoopEntryTerminalForNode(
		const UOptimusNode* InNode,
		const FOptimusPinTraversalContext& InContext = {}
		) const;

	UE_API TArray<const UOptimusNodeLink *> GetPinLinks(const UOptimusNodePin* InNodePin) const;

	/// Check to see if connecting these two nodes will form a graph cycle.
	/// @param InOutputNode The node from which the link originates
	/// @param InInputNode The node to which the link ends
	/// @return True if connecting these two nodes will result in a graph cycle.
	UE_API bool DoesLinkFormCycle(
		const UOptimusNode* InOutputNode, 
		const UOptimusNode* InInputNode) const;
		
	/// Add a new pin to the target node with the type of source pin
	/// and connect the source pin to the new pin
	/// @param InTargetNode The node to add the pin to, it has to have an adder pin
	/// @param InSelectedAction The selected action that defines where the new pin should be added
	/// @param InSourcePin The pin to create the new pin and to connect to the new pin
	/// @return True if a new pin is created.
	UE_API bool ConnectAdderPin(
		IOptimusNodeAdderPinProvider* InTargetNode,
		const IOptimusNodeAdderPinProvider::FAdderPinAction& InSelectedAction,
		UOptimusNodePin* InSourcePin
		);

	const TArray<UOptimusNode*>& GetAllNodes() const { return Nodes; }
	const TArray<UOptimusNodeLink*>& GetAllLinks() const { return Links; }
	const TArray<UOptimusNodePair*>& GetAllNodePairs() const { return NodePairs; }

	UE_API UOptimusActionStack* GetActionStack() const;      

	/// IOptimusNodeGraphCollectionOwner overrides
	UE_API IOptimusNodeGraphCollectionOwner* GetCollectionOwner() const override;
	UE_API IOptimusNodeGraphCollectionOwner* GetCollectionRoot() const override;
	UE_API FString GetCollectionPath() const override;

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	const TArray<UOptimusNodeGraph*> &GetGraphs() const override { return SubGraphs; }

	UE_API UOptimusNodeGraph* FindGraphByName(FName InGraphName) const override;
	
	UE_API UOptimusNodeGraph* CreateGraphDirect(
		EOptimusNodeGraphType InType,
		FName InName,
		TOptional<int32> InInsertBefore) override;
	UE_API bool AddGraphDirect(
		UOptimusNodeGraph* InGraph,
		int32 InInsertBefore) override;
	UE_API bool RemoveGraphDirect(
		UOptimusNodeGraph* InGraph,
		bool bInDeleteGraph) override;

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool MoveGraphDirect(
		UOptimusNodeGraph* InGraph,
		int32 InInsertBefore) override;

	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RenameGraphDirect(
		UOptimusNodeGraph* InGraph,
		const FString& InNewName) override;
	
	UFUNCTION(BlueprintCallable, Category = OptimusNodeGraph)
	UE_API bool RenameGraph(
		UOptimusNodeGraph* InGraph,
		const FString& InNewName) override;


#if WITH_EDITOR
	// Set the graph view location and zoom, to ensure that the view location is stored between sessions and
	// graph switching.
	UE_API void SetViewLocationAndZoom(const FVector2D& InViewLocation, float InViewZoom);

	// Returns the graph view location and zoom, if set. If the location has never been set, this function returns
	// false and the resulting out values are left undefined.
	UE_API bool GetViewLocationAndZoom(FVector2D& OutViewLocation, float& OutViewZoom) const;
#endif
	
protected:
	friend class UOptimusDeformer;
	friend class UOptimusNode;
	friend class UOptimusNodePin;
	friend class UOptimusNode_ConstantValue;
	friend class UOptimusNode_SubGraphReference;
	friend class FOptimusEditorClipboard;
	friend struct FOptimusNodeGraphAction_AddNode;
	friend struct FOptimusNodeGraphAction_DuplicateNode;
	friend struct FOptimusNodeGraphAction_RemoveNode;
	friend struct FOptimusNodeGraphAction_AddRemoveNodePair;
	friend struct FOptimusNodeGraphAction_AddRemoveLink;
	friend struct FOptimusNodeGraphAction_PackageKernelFunction;
	friend struct FOptimusNodeGraphAction_UnpackageKernelFunction;

	// Direct edit functions. Used by the actions.
	UE_API UOptimusNode* CreateNodeDirect(
		const UClass* InNodeClass,
		FName InName,
		TFunction<bool(UOptimusNode*)> InConfigureNodeFunc
		);

	UE_API bool AddNodeDirect(
		UOptimusNode* InNode
	);
	
	UE_API void RemoveGraph(
		UOptimusNodeGraph* InNodeGraph
		);
	
	// Remove a node directly. If a node still has connections this call will fail. 
	UE_API bool RemoveNodeDirect(
		UOptimusNode* InNode,		
		bool bFailIfLinks = true);

	UE_API bool AddNodePairDirect(UOptimusNode* InFirstNode, UOptimusNode* InSecondNode);
	
	UE_API bool RemoveNodePairDirect(UOptimusNode* InFirstNode, UOptimusNode* InSecondNode);
	
	UE_API bool AddLinkDirect(UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin);

	UE_API bool RemoveLinkDirect(UOptimusNodePin* InNodeOutputPin, UOptimusNodePin* InNodeInputPin);

	UE_API bool RemoveAllLinksToPinDirect(UOptimusNodePin* InNodePin);

	UE_API bool RemoveAllLinksToNodeDirect(UOptimusNode* InNode);
	
	// FIXME: Remove this.
	void SetGraphType(EOptimusNodeGraphType InType)
	{
		GraphType = InType;
	}

	UE_API void Notify(EOptimusGraphNotifyType InNotifyType, UObject *InSubject) const;
	UE_API void GlobalNotify(EOptimusGlobalNotifyType InNotifyType, UObject *InSubject) const;

	// The type of graph this represents. 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=Overview)
	EOptimusNodeGraphType GraphType = EOptimusNodeGraphType::Transient;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bViewLocationSet = false;
	
	UPROPERTY()
	FVector2D ViewLocation = {0.0, 0.0};

	UPROPERTY()
	float ViewZoom = 0.0f;
#endif
	
private:
	UE_API IOptimusPathResolver* GetPathResolver() const;
	
	UE_API UOptimusNode* AddNodeInternal(
		const TSubclassOf<UOptimusNode> InNodeClass,
		const FVector2D& InPosition,
		TFunction<void(UOptimusNode*)> InNodeConfigFunc
	);
	
	UE_API TArray<UOptimusNode*> AddNodePairInternal(
		const TSubclassOf<UOptimusNode> InNodeClass,
		const FVector2D& InPosition,
		TFunction<void(UOptimusNode*)> InFirstNodeConfigFunc,
		TFunction<void(UOptimusNode*)> InSecondNodeConfigFunc
	);

	static UE_API void GatherRelatedObjects(
		const TArray<UOptimusNode*>& InNodes,
		TArray<UOptimusNode*>& OutNodes,
		TArray<UOptimusNodePair*>& OutNodePairs,
		TArray<UOptimusNodeSubGraph*>& OutSubGraphs
		);

	static UE_API bool DuplicateSubGraph(
		UOptimusActionStack* InActionStack,
		const FString& InGraphOwnerPath,
		UOptimusNodeSubGraph* InSourceSubGraph,
		FName InNewGraphName
		);

	UE_API void RemoveNodePairByIndex(int32 NodePairIndex);
	
	UE_API void RemoveLinkByIndex(int32 LinkIndex);

	/// Returns the indexes of all links that connect to the node. If a direction is specified
	/// then only links coming into the node for that direction will be added (e.g. if Input
	/// is specified, then only links going into the input pins will be considered).
	/// @param InNode The node to retrieve all link connections for.
	/// @param InDirection The pins the links should be connected into, or Unknown if not 
	/// direction is not important.
	/// @return A list of indexes into the Links array of links to the given node.
	UE_API TArray<int32> GetAllLinkIndexesToNode(
		const UOptimusNode* InNode, 
		EOptimusNodePinDirection InDirection
		) const;


	UE_API TArray<int32> GetAllLinkIndexesToNode(
	    const UOptimusNode* InNode
	    ) const;

		
	UE_API TArray<int32> GetAllLinkIndexesToPin(
		const UOptimusNodePin* InNodePin
		) const;

	UE_API FString ConstructSubGraphPath(const FString& InSubGraphName) const;
	static UE_API FString ConstructSubGraphPath(const FString& InGraphOwnerPath, const FString& InSubGraphName);

	UE_API void PostLoadReplaceAnimAttributeDataInterfaceNodeWithGenericDataInterfaceNode();
	
	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UOptimusNode>> Nodes;

	// FIXME: Use a map.
	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UOptimusNodeLink>> Links;

	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UOptimusNodePair>> NodePairs;
	
	UPROPERTY(NonTransactional)
	TArray<TObjectPtr<UOptimusNodeGraph>> SubGraphs;

	FOptimusGraphNotifyDelegate GraphNotifyDelegate;
};

#undef UE_API
