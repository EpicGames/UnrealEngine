// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "OptimusNodeLink.generated.h"

class UOptimusNodePin;

UCLASS(MinimalAPI)
class UOptimusNodeLink : public UObject
{
	GENERATED_BODY()

public:
	UOptimusNodeLink() = default;

	/** Returns the output pin on the node this link connects from. */
	UOptimusNodePin* GetNodeOutputPin() const { return NodeOutputPin; }

	/** Returns the input pin on the node that this link connects to. */
	UOptimusNodePin* GetNodeInputPin() const { return NodeInputPin; }

protected:
	friend class UOptimusNodeGraph;

	UPROPERTY()
	TObjectPtr<UOptimusNodePin> NodeOutputPin;

	UPROPERTY()
	TObjectPtr<UOptimusNodePin> NodeInputPin;
};
