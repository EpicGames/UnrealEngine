// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "OptimusNodePair.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusNode;

UCLASS(MinimalAPI)
class UOptimusNodePair : public UObject
{
	GENERATED_BODY()

public:
	UOptimusNodePair() = default;

	/** Returns the output pin on the node this link connects from. */
	UOptimusNode* GetFirst() const { return First; }

	/** Returns the input pin on the node that this link connects to. */
	UOptimusNode* GetSecond() const { return Second; }

	UE_API bool Contains(const UOptimusNode* InFirst, const UOptimusNode* InSecond) const;

	UE_API bool Contains(const UOptimusNode* InNode) const;

	UE_API UOptimusNode* GetNodeCounterpart(const UOptimusNode* InNode) const;

	
protected:
	friend class UOptimusNodeGraph;

	UPROPERTY()
	TObjectPtr<UOptimusNode> First;

	UPROPERTY()
	TObjectPtr<UOptimusNode> Second;
};

#undef UE_API
