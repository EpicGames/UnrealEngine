// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusNodePinRouter.h"
#include "OptimusDataType.h"
#include "OptimusDataDomain.h"

#include "OptimusNodePin.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusComponentSourceBinding;
class UOptimusActionStack;
class UOptimusNode;
enum class EOptimusGraphNotifyType;


/** Specifies the storage type of the pin data */
UENUM()
enum class UE_DEPRECATED(5.1, "Replaced with FOptimusDataDomain") EOptimusNodePinStorageType : uint8
{
	Value,				/** Plain value of some type */
	Resource			/** Resource binding of some type */
};


UENUM()
enum class EOptimusNodePinDirection : uint8
{
	Unknown,
	Input, 
	Output
};


UCLASS(MinimalAPI, BlueprintType)
class UOptimusNodePin : public UObject
{
	GENERATED_BODY()

public:
	UOptimusNodePin() = default;

	/// Returns whether this pin is an input or output connection.
	/// @return The direction of this pin.
	EOptimusNodePinDirection GetDirection() const { return Direction; }

	/// Returns \c true if this pin is a grouping pin. 
	bool IsGroupingPin() const { return bIsGroupingPin; }

	/// Returns the parent pin of this pin, or nullptr if it is the top-most pin.
	UE_API UOptimusNodePin* GetParentPin();
	UE_API const UOptimusNodePin* GetParentPin() const;

	/// Returns the next sibling of this pin, or nullptr if it is the last pin of its siblings.
	UE_API UOptimusNodePin* GetNextPin();
	UE_API const UOptimusNodePin* GetNextPin() const;

	/// Returns the root pin of this pin hierarchy.
	UE_API UOptimusNodePin* GetRootPin();
	UE_API const UOptimusNodePin* GetRootPin() const;

	/// Returns the owning node of this pin and all its ancestors and children.
	UE_API UOptimusNode* GetOwningNode() const;

	/// Returns the array of pin names from the root pin to this pin. Can be used to to
	/// easily traverse the pin hierarchy.
	UE_API TArray<FName> GetPinNamePath() const;

	/// Returns a unique name for this pin within the namespace of the owning node.
	/// E.g: Direction.X
	UE_API FName GetUniqueName() const;

	/** Returns a user-friendly display name for this pin */
	UE_API FText GetDisplayName() const;

	/** Returns a tooltip to use when hovering over the pin in the graph */
	UE_API FText GetTooltipText() const;

	/// Returns the path of the pin from the graph collection owner root.
	/// E.g: SetupGraph/LinearBlendSkinning1.Direction.X
	UE_API FString GetPinPath() const;

	/** Set the display name of the node pin */
	UE_API bool SetDisplayName(const FName& InDisplayName);

	/// Returns a pin path from a string. Returns an empty array if string is invalid or empty.
	static UE_API TArray<FName> GetPinNamePathFromString(const FStringView InPinPathString);

	/** Return the registered Optimus data type associated with this pin */
	FOptimusDataTypeHandle GetDataType() const { return DataType.Resolve(); }

	/** Returns the data domain that this pin is expected to cover */
	const FOptimusDataDomain& GetDataDomain() const { return DataDomain; }
	
	/** Return all component source bindings that flow into pin. 
	 */
	UE_API TSet<UOptimusComponentSourceBinding*> GetComponentSourceBindings(const FOptimusPinTraversalContext& InContext) const;

	/** Return all component source bindings that flow into this input pin and its sub-pins.
	 */
	UE_API TSet<UOptimusComponentSourceBinding*> GetComponentSourceBindingsRecursively(const FOptimusPinTraversalContext& InContext) const;

	/** Whether the data presented by the pin can change over time
	 */
	UE_API bool IsMutable(const FOptimusPinTraversalContext& InContext) const;
	
	/** Returns the FProperty object for this pin. This can be used to directly address the
	  * node data represented by this pin. Not all pins have an underlying resource so this can
	  * return nullptr.
	  */
	UE_API FProperty *GetPropertyFromPin() const;

	/// Returns the current value of this pin, including sub-values if necessary, as a string.
	UE_API FString GetValueAsString() const;

	/// Sets the value of this pin from a value string in an undoable fashion.
	UE_API bool SetValueFromString(const FString& InStringValue);

	/// Sets the value of this pin from a value string with no undo (although if a transaction
	/// bracket is open, it will receive the modification).
	UE_API bool SetValueFromStringDirect(const FString &InStringValue);

	/// Returns the sub-pins of this pin. For example for a pin representing the FVector type, 
	/// this will return pins for the X, Y, and Z components of it (as float values).
	TArrayView<const TObjectPtr<UOptimusNodePin>> GetSubPins() const { return MakeArrayView(SubPins); }

	/// Returns all sub-pins of this pin, recursively. In the returned list, the parent pins
	/// are listed before their child pins.
	UE_API TArray<UOptimusNodePin *> GetSubPinsRecursively(bool bInIncludeThisPin = false) const;

	/** Returns all pins that have a _direct_ connection to this pin. If nothing is connected 
	  * to this pin, it returns an empty array.
	  */
	UE_API TArray<UOptimusNodePin *> GetConnectedPins() const;

	/** Returns all pins that are connected to working nodes, traversing fully through any 
	 *  router nodes. This is so that we can get a connection to an actual non-routing,
	 *  working node. If there are no connections to this pin, it returns an empty array.
	 *  Any connections that don't lead to a non-routing node are ignored.
	 *  The routing assumes that we do not go higher up in the graph stack than the level
	 *  at which we started (i.e. if this is called on a node inside of a sub-graph with
	 *  an empty traversal context, and the connection leads to a return node and up into the
	 *  graph that has a reference to that sub-graph, there are no more context levels to peel
	 *  off and the connection will _not_ be included).
	 */
	UE_API TArray<FOptimusRoutedNodePin> GetConnectedPinsWithRouting(
		const FOptimusPinTraversalContext& InContext = {}
		) const;

	/// Ask this pin if it allows a connection from the other pin. 
	/// @param InOtherPin The other pin to connect to/from
	/// @param OutReason An optional string that will contain the reason why the connection
	/// cannot be made if this function returns false.
	/// @return True if the connection can be made, false otherwise.
	UE_API bool CanCannect(const UOptimusNodePin *InOtherPin, FString *OutReason = nullptr) const;

	/** Set the expansion state of this pin. This is purely driven by the UI and is not
	    an undoable operation. No notifications are sent if the state changes. */
	UE_API void SetIsExpanded(bool bInIsExpanded);

	/** Returns the stored expansion state */
	UE_API bool GetIsExpanded() const;

	// UObject overrides
	UE_API void PostLoad() override;
protected:
	friend class UOptimusNode;

	// Initialize the pin data from the given direction and property.
	UE_API void InitializeWithData(
		EOptimusNodePinDirection InDirection,
		FOptimusDataDomain InDataDomain,
		FOptimusDataTypeRef InDataTypeRef
		);

	UE_API void InitializeWithGrouping(
		EOptimusNodePinDirection InDirection 
		);
	
	UE_API void AddSubPin(
		UOptimusNodePin* InSubPin,
		UOptimusNodePin* InBeforePin = nullptr);

	UE_API void ClearSubPins();

	UE_API bool SetDataType(
	    FOptimusDataTypeRef InDataType
		);

	UE_API bool SetName(
	    FName InName);

	UE_API void Notify(EOptimusGraphNotifyType InNotifyType) const;

private:
	UE_API uint8 *GetPropertyValuePtr() const;

	UE_API UOptimusActionStack* GetActionStack() const;

	UE_API bool VerifyValue(const FString& InStringValue) const;

	UPROPERTY()
	bool bIsGroupingPin = false;
	
	UPROPERTY()
	EOptimusNodePinDirection Direction = EOptimusNodePinDirection::Unknown;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "Use DataDomain")
	UPROPERTY()
	EOptimusNodePinStorageType StorageType_DEPRECATED = EOptimusNodePinStorageType::Value;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	UPROPERTY()
	FOptimusDataDomain DataDomain;
	
	UPROPERTY()
	FOptimusDataTypeRef DataType;

	UPROPERTY()
	TArray<TObjectPtr<UOptimusNodePin>> SubPins;

	FName DisplayName;
};

#undef UE_API
