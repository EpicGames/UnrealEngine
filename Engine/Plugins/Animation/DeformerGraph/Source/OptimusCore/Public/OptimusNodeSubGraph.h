// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IOptimusParameterBindingProvider.h"
#include "OptimusNodeGraph.h"
#include "OptimusBindingTypes.h"
#include "OptimusNodeSubGraph.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusNode_GraphTerminal;
enum class EOptimusTerminalType;

UCLASS(MinimalAPI)
class UOptimusNodeSubGraph :
	public UOptimusNodeGraph,
	public IOptimusParameterBindingProvider
{
	
	GENERATED_BODY()

public:

	static UE_API FName GraphDefaultComponentPinName;
	static UE_API FName InputBindingsPropertyName;
	static UE_API FName OutputBindingsPropertyName;

	UE_API UOptimusNodeSubGraph();
	
	// IOptimusParameterBindingProvider 
	UE_API FString GetBindingDeclaration(FName BindingName) const override;
	UE_API bool GetBindingSupportAtomicCheckBoxVisibility(FName BindingName) const override;
	UE_API bool GetBindingSupportReadCheckBoxVisibility(FName BindingName) const override;
	UE_API EOptimusDataTypeUsageFlags GetTypeUsageFlags(const FOptimusDataDomain& InDataDomain) const override;

	UE_API UOptimusComponentSourceBinding* GetDefaultComponentBinding(const FOptimusPinTraversalContext& InTraversalContext) const;

	UE_API UOptimusNode_GraphTerminal* GetTerminalNode(EOptimusTerminalType InTerminalType) const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingArrayPasted, FName);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingValueChanged, FName);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingArrayItemAdded, FName);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingArrayItemRemoved, FName);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingArrayCleared, FName);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingArrayItemMoved, FName);
	
	FOnBindingArrayPasted& GetOnBindingArrayPasted() { return OnBindingArrayPastedDelegate; }
	FOnBindingValueChanged& GetOnBindingValueChanged() { return OnBindingValueChangedDelegate; }
	FOnBindingArrayItemAdded& GetOnBindingArrayItemAdded() { return OnBindingArrayItemAddedDelegate; }
	FOnBindingArrayItemRemoved& GetOnBindingArrayItemRemoved() { return OnBindingArrayItemRemovedDelegate; }
	FOnBindingArrayCleared& GetOnBindingArrayCleared() { return OnBindingArrayClearedDelegate; }
	FOnBindingArrayItemMoved& GetOnBindingArrayItemMoved() { return OnBindingArrayItemMovedDelegate; }
	
#if WITH_EDITOR
	// UObject overrides
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY(EditAnywhere, Category=Bindings, meta=(AllowParameters))
	FOptimusParameterBindingArray InputBindings;

	UPROPERTY(EditAnywhere, Category=Bindings)
	FOptimusParameterBindingArray OutputBindings;



private:

	UE_API void SanitizeBinding(FOptimusParameterBinding& InOutBinding, FName InOldName, bool bInAllowParameter);
	UE_API FName GetSanitizedBindingName(FName InNewName, FName InOldName);
	

#if WITH_EDITOR
	UE_API void PropertyArrayPasted(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemAdded(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemRemoved(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayCleared(const FPropertyChangedEvent& InPropertyChangedEvent);
	UE_API void PropertyArrayItemMoved(const FPropertyChangedEvent& InPropertyChangedEvent);
#endif
	
	FOnBindingArrayPasted OnBindingArrayPastedDelegate;
	FOnBindingValueChanged OnBindingValueChangedDelegate;
	FOnBindingArrayItemAdded OnBindingArrayItemAddedDelegate;
	FOnBindingArrayItemRemoved OnBindingArrayItemRemovedDelegate;
	FOnBindingArrayCleared OnBindingArrayClearedDelegate;
	FOnBindingArrayItemMoved OnBindingArrayItemMovedDelegate;

};

#undef UE_API
