// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusNode.h"
#include "Types/SlateVector2.h"

#include "OptimusNode_Comment.generated.h"

#define UE_API OPTIMUSCORE_API

class UOptimusNode_Comment;

DECLARE_DELEGATE_OneParam(FOnOptimusCommentNodePropertyChanged, UOptimusNode_Comment*);

UCLASS(MinimalAPI, Hidden)
class  UOptimusNode_Comment : 
	public UOptimusNode
{
	GENERATED_BODY()


public:
	
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	FName GetNodeCategory() const override {return NAME_None;}

	UE_API bool SetSize(const UE::Slate::FDeprecateVector2DParameter& InSize);
	const UE::Slate::FDeprecateVector2DResult& GetSize() {return Size;}
	UE_API void SetComment(const FString& InNewComment);
	const FString& GetComment() {return Comment;}
	
	FOnOptimusCommentNodePropertyChanged& GetOnPropertyChanged() {return OnPropertyChangedDelegate; };

	/** Color to style comment with */
	UPROPERTY(EditAnywhere, Category=Comment)
	FLinearColor CommentColor = FLinearColor::White;

	/** Size of the text in the comment box */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(ClampMin=1, ClampMax=1000))
	int32 FontSize = 18;

	/** Comment to show */
	UPROPERTY(EditAnywhere, Category=Comment)
	FString Comment = TEXT("Comment");

	/** Whether to show a zoom-invariant comment bubble when zoomed out (making the comment readable at any distance). */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(DisplayName="Show Bubble When Zoomed"))
	bool bBubbleVisible = false;

	/** Whether to use Comment Color to color the background of the comment bubble shown when zoomed out. */
	UPROPERTY(EditAnywhere, Category=Comment, meta=(DisplayName="Color Bubble", EditCondition=bBubbleVisible))
	bool bColorBubble = false;
	
private:
	friend class UOptimusNodeGraph;
	friend struct FOptimusCommentNodeAction_ResizeNode;
	UE_API bool SetSizeDirect(const FVector2f& InNewSize);	

	UPROPERTY(NonTransactional)
	FDeprecateSlateVector2D Size = FVector2f(400, 100);
	
	FOnOptimusCommentNodePropertyChanged OnPropertyChangedDelegate;
};

#undef UE_API
