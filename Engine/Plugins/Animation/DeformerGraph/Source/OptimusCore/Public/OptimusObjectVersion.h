// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

#define UE_API OPTIMUSCORE_API

struct FOptimusObjectVersion
{
	// Not instantiable.
	FOptimusObjectVersion() = delete;

	enum Type
	{
		InitialVersion,

		SwitchToMeshDeformerBase,
		ReparentResourcesAndVariables,
		SwitchToParameterBindingArrayStruct,
		AddBindingsToGraph,
		ComponentProviderSupport,
		SetPrimaryBindingName,
		DataDomainExpansion,
		KernelDataInterface,
		KernelParameterBindingToggleAtomic,
		PropertyBagValueContainer,
		SkinnedMeshWriteDIColorBufferManualFetchSwizzle,
		PropertyPinSupport,
		KernelThreadIndexUsingTotalThreadCount,
		FunctionGraphUseGuid,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;
};

#undef UE_API
