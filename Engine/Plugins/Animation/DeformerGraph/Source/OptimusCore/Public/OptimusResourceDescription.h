// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"

#include "CoreMinimal.h"
#include "OptimusComponentSource.h"
#include "OptimusDataDomain.h"
#include "UObject/Object.h"

#include "OptimusResourceDescription.generated.h"

#define UE_API OPTIMUSCORE_API


class UOptimusComponentSourceBinding;
class UOptimusDeformer;
class UOptimusPersistentBufferDataInterface;


UCLASS(MinimalAPI, BlueprintType)
class UOptimusResourceDescription :
	public UObject
{
	GENERATED_BODY()
public:

	UOptimusResourceDescription() = default;

	/** Returns the owning deformer to operate on this resource */
	// FIXME: Move to interface-based system.
	UE_API UOptimusDeformer* GetOwningDeformer() const;

	/** Returns the index of the variable within the container */ 
	UE_API int32 GetIndex() const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = ResourceDescription, meta=(NoResetToDefault))
	FName ResourceName;

	/** The the data type of each element of the resource */
	UPROPERTY(EditAnywhere, Category=ResourceDescription, meta=(UseInResource, NoResetToDefault))
	FOptimusDataTypeRef DataType;

	/** The component binding that this resource description is bound to */
	UPROPERTY(VisibleAnywhere, Category=ResourceDescription)
	TWeakObjectPtr<UOptimusComponentSourceBinding> ComponentBinding;
	
	/** The data domain for this resource. */
	UPROPERTY(EditAnywhere, Category=ResourceDescription, meta=(EditCondition="ComponentBinding != nullptr", HideEditConditionToggle))
	FOptimusDataDomain DataDomain;

	UPROPERTY()
	TObjectPtr<UOptimusPersistentBufferDataInterface> DataInterface;

	// UObject overrides
	UE_API void PostLoad() override;
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API void PreEditUndo() override;
	UE_API void PostEditUndo() override;
#endif

	UE_API bool IsValidComponentBinding() const;
	UE_API FOptimusDataDomain GetDataDomainFromComponentBinding() const;

private:
#if WITH_EDITORONLY_DATA
	FName ResourceNameForUndo;
#endif
};

#undef UE_API
