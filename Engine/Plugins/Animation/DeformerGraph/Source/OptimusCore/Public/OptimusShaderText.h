// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OptimusShaderText.generated.h"

USTRUCT()
struct FOptimusShaderText
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=ShaderText)
	FString Declarations;
	
	UPROPERTY(EditAnywhere, Category=ShaderText)
	FString ShaderText;
};
