// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComputeFramework/ComputeSource.h"
#include "IOptimusShaderTextProvider.h"
#include "OptimusSource.generated.h"

#define UE_API OPTIMUSCORE_API

UCLASS(MinimalAPI)
class UOptimusSource
	: public UComputeSource
	, public IOptimusShaderTextProvider
{
	GENERATED_BODY()

public:
	UE_API void SetSource(const FString& InText);
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	// Begin UComputeSource interface.
	FString GetSource() const override { return SourceText; }
	UE_API FString GetVirtualPath() const override;
	// End UComputeSource interface.

	// Begin IOptimusShaderTextProvider interface.
#if WITH_EDITOR	
	UE_API FString GetNameForShaderTextEditor() const override;
	FString GetDeclarations() const override { return FString(); }
	FString GetShaderText() const override { return SourceText; }
	void SetShaderText(const FString& InText) override { SetSource(InText); }
	UE_API bool IsShaderTextReadOnly() const override;
#endif
	// End IOptimusShaderTextProvider interface.

protected:
	/** HLSL Source. */
	UPROPERTY(EditAnywhere, Category = Source, meta = (DisplayAfter = "AdditionalSources"))
	FString SourceText;
};

#undef UE_API
