// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OptimusValidatedName.generated.h"

#define UE_API OPTIMUSCORE_API

/** 
 * Structure containing a FName that has been validated for use as a variable or function name in HLSL code. 
 * When using this as a UPROPERTY, the UI will use a property customization that validates text entry.
 */
USTRUCT()
struct FOptimusValidatedName
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Name)
	FName Name;

	/** 
	 * Our default validation function. 
	 * Returns true if InName is valid. 
	 */
	static UE_API bool IsValid(FString const& InName, FText* OutReason, FText const* InErrorCtx);

	/** 
	 * Allow serialization from a legacy FName or FString. 
	 * Makes it easy to convert FName -> FOptimusValidateName.
	 */
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	// Functions for compatibility with clients expecting a simple FName.
	operator FName () const { return Name; }
	FOptimusValidatedName& operator=(FName const& RHS) { Name = RHS; return *this; }
	bool operator==(FName const& RHS) const { return Name == RHS; }
	bool operator!=(FName const& RHS) const { return Name != RHS; }
	FString ToString() const { return Name.ToString(); }
};

template<>
struct TStructOpsTypeTraits<FOptimusValidatedName> : public TStructOpsTypeTraitsBase2<FOptimusValidatedName>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

#undef UE_API
