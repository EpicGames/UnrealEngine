// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "OptimusDataType.h"
#include "OptimusValueContainerStruct.h"

#include "OptimusValue.generated.h"

UENUM()
enum class EOptimusValueUsage: uint8
{
	None = 0,
	CPU = 1 << 0,
	GPU = 1 << 1
};
ENUM_CLASS_FLAGS(EOptimusValueUsage);

UENUM()
enum class EOptimusValueType : uint8
{
	Invalid,
	Constant,
	Variable
};

USTRUCT()
struct FOptimusValueIdentifier
{
	GENERATED_BODY()

	UPROPERTY()
	EOptimusValueType Type = EOptimusValueType::Invalid;

	UPROPERTY()
	FName Name = NAME_None;
	
	friend uint32 GetTypeHash(const FOptimusValueIdentifier& InIdentifier)
	{
		return HashCombineFast(GetTypeHash(InIdentifier.Type), GetTypeHash(InIdentifier.Name));
	}

	bool operator==(FOptimusValueIdentifier const& InOther) const
	{
		return Type == InOther.Type && Name == InOther.Name;
	}
};

USTRUCT()
struct FOptimusValueDescription
{
	GENERATED_BODY()
	
	UPROPERTY()
	FOptimusDataTypeRef DataType;

	UPROPERTY()
	EOptimusValueUsage ValueUsage = EOptimusValueUsage::None;
	
	UPROPERTY()
	FOptimusValueContainerStruct Value;

	UPROPERTY()
	FShaderValueContainer ShaderValue;
};

USTRUCT()
struct FOptimusDataInterfacePropertyOverrideInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FOptimusValueIdentifier> PinNameToValueIdMap;
};

