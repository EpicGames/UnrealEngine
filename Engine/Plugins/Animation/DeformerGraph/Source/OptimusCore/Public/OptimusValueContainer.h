// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OptimusDataType.h"
#include "OptimusValueContainer.generated.h"

#define UE_API OPTIMUSCORE_API

struct FOptimusValueContainerStruct;

// Deprecated
UCLASS(MinimalAPI)
class UOptimusValueContainerGeneratorClass : public UClass
{
public:
	GENERATED_BODY()
	
	// DECLARE_WITHIN(UObject) is only kept for back-compat, please don't parent the class
	// to the asset object.
	// This class should be parented to the package, instead of the asset object
	// because the engine no longer asset object as UClass outer
	// however, since in the past we have parented the class to the asset object
	// this flag has to be kept such that we can load the old asset in the first place and
	// re-parent it back to the package in post load
	DECLARE_WITHIN(UObject)
private:
	friend class UOptimusValueContainer;
	
	static UE_API FName ValuePropertyName;
	
	// UClass overrides
	UE_API void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	
	static UE_API UClass *GetClassForType(
		UPackage*InPackage,
		FOptimusDataTypeRef InDataType
		);

	static UE_API UClass *RefreshClassForType(
		UPackage*InPackage,
		FOptimusDataTypeRef InDataType
		);

	UPROPERTY()
	FOptimusDataTypeRef DataType;
};

// Deprecated
UCLASS(MinimalAPI)
class UOptimusValueContainer : public UObject
{
public:
	GENERATED_BODY()
	
	// Convert to the newer container type
	UE_API FOptimusValueContainerStruct MakeValueContainerStruct();
	
private:
	UE_API void PostLoad() override;
	
	static UOptimusValueContainer* MakeValueContainer(UObject* InOwner, FOptimusDataTypeRef InDataTypeRef);

	FOptimusDataTypeRef GetValueType() const;
	FShaderValueContainer GetShaderValue() const;
};

#undef UE_API
