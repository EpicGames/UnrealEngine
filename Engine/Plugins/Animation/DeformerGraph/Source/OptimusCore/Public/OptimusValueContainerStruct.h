// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"
#include "StructUtils/PropertyBag.h"
#include "OptimusValueContainerStruct.generated.h"

USTRUCT()
struct FOptimusValueContainerStruct
{
	GENERATED_BODY()

	static OPTIMUSCORE_API const TCHAR* ValuePropertyName;
	
	UPROPERTY(EditAnywhere, Category="Value", meta=(FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag Value;

	OPTIMUSCORE_API bool IsInitialized() const;
	OPTIMUSCORE_API void SetType(FOptimusDataTypeRef InDataType);
	OPTIMUSCORE_API void SetValue(FOptimusDataTypeRef InDataType, TArrayView<const uint8> InValue);
	OPTIMUSCORE_API FShaderValueContainer GetShaderValue(FOptimusDataTypeRef InDataType) const;
	OPTIMUSCORE_API FString GetValueAsString() const;
	
private:
	const FProperty* GetValueProperty() const;
	const uint8* GetValueMemory() const;

};
