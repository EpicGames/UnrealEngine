// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"

#include "CoreMinimal.h"
#include "OptimusValueContainerStruct.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "OptimusVariableDescription.generated.h"

#define UE_API OPTIMUSCORE_API


class UOptimusDeformer;
class UOptimusValueContainer;


USTRUCT()
struct FOptimusVariableMetaDataEntry
{
	GENERATED_BODY()

	FOptimusVariableMetaDataEntry() {}
	FOptimusVariableMetaDataEntry(FName InKey, FString&& InValue)
	    : Key(InKey), Value(MoveTemp(InValue))
	{}

	/** Name of metadata key */
	UPROPERTY(EditAnywhere, Category = VariableMetaDataEntry)
	FName Key;

	/** Name of metadata value */
	UPROPERTY(EditAnywhere, Category = VariableMetaDataEntry)
	FString Value;
};


UCLASS(MinimalAPI, BlueprintType)
class UOptimusVariableDescription : 
	public UObject
{
	GENERATED_BODY()
public:
	/** 
	 * Set the data type, and recreate the backing data storage as well
	 */
	UE_API void SetDataType(FOptimusDataTypeRef InDataType);

	/** Returns the owning deformer to operate on this variable */
	// FIXME: Move to interface-based system.
	UE_API UOptimusDeformer* GetOwningDeformer() const;

	/** Returns the index of the variable within the container */ 
	UE_API int32 GetIndex() const;

	/** An identifier that uniquely identifies this variable */
	UPROPERTY()
	FGuid Guid;

	/** Name of the variable */
	UPROPERTY(EditAnywhere, Category = VariableDefinition,  meta=(NoResetToDefault))
	FName VariableName;

	/** The data type of the variable */
	UPROPERTY(EditAnywhere, Category = VariableDefinition, meta=(UseInVariable, UseInProperty, NoResetToDefault))
	FOptimusDataTypeRef DataType;

	/** The default value for the variable. */
	UPROPERTY(EditAnywhere, Category = VariableDefinition)
	FOptimusValueContainerStruct DefaultValueStruct;
	
	/**	Runtime container for variable values in a deformer instance*/
	UPROPERTY(Transient)
	FShaderValueContainer CachedShaderValue;

	UE_API void PostLoad();
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API void PreEditUndo() override;
	UE_API void PostEditUndo() override;
#endif

private:

	// Deprecated 
    UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use CachedShaderValue instead"))
    TArray<uint8> ValueData_DEPRECATED;

	// Deprecated 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "use DefaultValueStruct instead"))
	TObjectPtr<UOptimusValueContainer> DefaultValue_DEPRECATED = nullptr;
	
#if WITH_EDITORONLY_DATA
	FName VariableNameForUndo;
#endif
};

#undef UE_API
