// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeveloperModule.h"

#include "Modules/ModuleManager.h"

void FOptimusDeveloperModule::StartupModule()
{
}

void FOptimusDeveloperModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FOptimusDeveloperModule, OptimusDeveloper)

DEFINE_LOG_CATEGORY(LogOptimusDeveloper);
