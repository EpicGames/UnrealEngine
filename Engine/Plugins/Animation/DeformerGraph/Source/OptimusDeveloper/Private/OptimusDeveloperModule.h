// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusDeveloperModule.h"

class FOptimusDeveloperModule : public IOptimusDeveloperModule
{
public:
	/** IModuleInterface implementation */
	void StartupModule() override;
	void ShutdownModule() override;
};

DECLARE_LOG_CATEGORY_EXTERN(LogOptimusDeveloper, Log, All);
