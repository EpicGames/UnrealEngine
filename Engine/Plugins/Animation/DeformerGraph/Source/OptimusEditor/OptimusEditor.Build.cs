// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class OptimusEditor : ModuleRules
    {
        public OptimusEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Core",
					"CoreUObject",
					"ApplicationCore",
					"SlateCore",
					"Slate",
					"GraphEditor",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"AssetTools",
					"AdvancedPreviewScene",
					"InputCore",
					"RHI", 
					"ToolMenus",
					"ToolWidgets",
					"ComputeFramework",
					"OptimusCore",
					"OptimusDeveloper",
					"BlueprintGraph",		// For the graph pin colors
					"Persona",
					"MessageLog",
					"PropertyEditor",
					"KismetCompiler",
					"SourceCodeAccess", 
					"StructUtilsEditor"
                }
			);

        }
    }
}
