// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInDeformer.h"
#include "OptimusEditor.h"
#include "GraphEditor.h"

#define LOCTEXT_NAMESPACE "FindInDeformer"

//////////////////////////////////////////////////////////////////////////
// FFindInDeformerResult

FFindInDeformerResult::FFindInDeformerResult(const FFindInGraphResult::FCreateParams& InCreateParams)
	: FFindInGraphResult(InCreateParams)
{
}

void FFindInDeformerResult::JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const
{
	if (TSharedPtr<FOptimusEditor> OptimusEditor = StaticCastSharedPtr<FOptimusEditor>(AssetEditorToolkit.Pin()))
	{
		if (TSharedPtr<SGraphEditor> GraphEditor = OptimusEditor->GetGraphEditorWidget())
		{
			GraphEditor->JumpToNode(InNode, false);
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// SFindInDeformer

TSharedPtr<FFindInGraphResult> SFindInDeformer::MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams)
{
	return MakeShared<FFindInDeformerResult>(InParams);
}

const UEdGraph* SFindInDeformer::GetGraph()
{
	if (TSharedPtr<FOptimusEditor> OptimusEditor = StaticCastSharedPtr<FOptimusEditor>(AssetEditorToolkitPtr.Pin()))
	{
		if (TSharedPtr<SGraphEditor> GraphEditor = OptimusEditor->GetGraphEditorWidget())
		{
			return GraphEditor->GetCurrentGraph();
		}
	}

	return nullptr;
}

bool SFindInDeformer::MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens)
{
	// For now no type-specific node matching
	return false;
}


/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
