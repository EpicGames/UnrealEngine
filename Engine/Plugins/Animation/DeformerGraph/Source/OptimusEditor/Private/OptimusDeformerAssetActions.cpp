// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeformerAssetActions.h"

#include "IOptimusEditor.h"
#include "IOptimusEditorModule.h"

#include "OptimusDeformer.h"

#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"


FOptimusDeformerAssetActions::FOptimusDeformerAssetActions(EAssetTypeCategories::Type InAssetCategoryBit) :
	AssetCategoryBit(InAssetCategoryBit)
{
}


FText FOptimusDeformerAssetActions::GetName() const
{ 
	return NSLOCTEXT("AssetTypeActions", "OptimusDeformerActions", "Deformer Graph"); 
}


FColor FOptimusDeformerAssetActions::GetTypeColor() const
{ 
	return FColor::Blue;
}


UClass* FOptimusDeformerAssetActions::GetSupportedClass() const
{ 
	return UOptimusDeformer::StaticClass(); 
}


void FOptimusDeformerAssetActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor /*= TSharedPtr<IToolkitHost>() */)
{
	EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UOptimusDeformer* OptimusDeformer = Cast<UOptimusDeformer>(*ObjIt))
		{
			const bool bBringToFrontIfOpen = true;
			if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(OptimusDeformer, bBringToFrontIfOpen))
			{
				EditorInstance->FocusWindow(OptimusDeformer);
			}
			else
			{
				IOptimusEditorModule& OptimusEditorModule = FModuleManager::LoadModuleChecked<IOptimusEditorModule>("OptimusEditor");
				OptimusEditorModule.CreateEditor(Mode, EditWithinLevelEditor, OptimusDeformer);
			}
		}
	}
}


uint32 FOptimusDeformerAssetActions::GetCategories()
{ 
	return AssetCategoryBit;
}


const TArray<FText>& FOptimusDeformerAssetActions::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("FOptimusDeformerAssetActions", "AnimDeformersSubMenu", "Deformers")
	};
	return SubMenus;
}


TSharedPtr<SWidget> FOptimusDeformerAssetActions::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	const FSlateBrush* Icon = FSlateIconFinder::FindIconBrushForClass(UOptimusDeformer::StaticClass());;

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetNoBrush())
		.Visibility(EVisibility::HitTestInvisible)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 3.0f))
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		[
			SNew(SImage)
			.Image(Icon)
		];
}
