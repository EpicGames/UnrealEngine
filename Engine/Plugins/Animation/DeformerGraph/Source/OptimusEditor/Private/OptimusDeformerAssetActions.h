// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"

#include "OptimusDeformer.h"

class FOptimusDeformerAssetActions
	: public FAssetTypeActions_Base
{
public:
	FOptimusDeformerAssetActions(EAssetTypeCategories::Type InAssetCategoryBit = EAssetTypeCategories::Animation);
	
protected:
	// IAssetTypeActions overrides
	FText GetName() const override;
	FColor GetTypeColor() const override;
	UClass* GetSupportedClass() const override;
	void OpenAssetEditor(
		const TArray<UObject*>& InObjects, 
		TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()
	) override;
	uint32 GetCategories() override;
	const TArray<FText>& GetSubMenus() const override;
	TSharedPtr<SWidget> GetThumbnailOverlay(
		const FAssetData& AssetData
	) const override;

private:
	EAssetTypeCategories::Type AssetCategoryBit;
};
