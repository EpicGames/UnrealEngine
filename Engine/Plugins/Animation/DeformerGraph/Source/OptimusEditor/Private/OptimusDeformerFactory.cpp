// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDeformerFactory.h"

#include "AssetTypeCategories.h"
#include "OptimusComponentSource.h"
#include "OptimusDeformer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDeformerFactory)


UOptimusDeformerFactory::UOptimusDeformerFactory()
{
	SupportedClass = UOptimusDeformer::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UOptimusDeformerFactory::FactoryCreateNew(
	UClass* InClass, 
	UObject* InParent, 
	FName InName, 
	EObjectFlags InFlags, 
	UObject* InContext, 
	FFeedbackContext* OutWarn
	)
{
	UOptimusDeformer* Deformer = NewObject<UOptimusDeformer>(InParent, InClass, InName, InFlags);
	
	// Create a default primary binding.
	Deformer->AddComponentBinding(nullptr, UOptimusComponentSourceBinding::GetPrimaryBindingName());

	return Deformer;
}


uint32 UOptimusDeformerFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}


bool UOptimusDeformerFactory::ShouldShowInNewMenu() const
{
	return true;
}
