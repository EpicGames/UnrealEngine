// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "OptimusDeformerFactory.generated.h"


/**
 * Implements a factory for UOptimusDeformer objects.
 */
UCLASS(hidecategories = Object)
class UOptimusDeformerFactory : public UFactory
{
	GENERATED_BODY()

public:
	UOptimusDeformerFactory();

	/// UFactory overrides
	FString GetDefaultNewAssetName() const override
	{
		return TEXT("DeformerGraph");
	}

	UObject* FactoryCreateNew(
		UClass* InClass, 
		UObject* InParent, 
		FName InName, 
		EObjectFlags InFlags, 
		UObject* InContext, 
		FFeedbackContext* OutWarn
	) override;

	/// Which category this object belongs to in the "New Asset" menu.
	uint32 GetMenuCategories() const override;

	/// Whether the object should be shown in the "New Asset" menu in the asset browser.
	bool ShouldShowInNewMenu() const override;
};
