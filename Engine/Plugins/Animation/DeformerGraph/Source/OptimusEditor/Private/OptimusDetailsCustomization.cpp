// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDetailsCustomization.h"

#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/SCS_Node.h"
#include "StructUtils/UserDefinedStruct.h"
#include "IDetailChildrenBuilder.h"
#include "IOptimusExecutionDomainProvider.h"
#include "IOptimusParameterBindingProvider.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "OptimusBindingTypes.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformer.h"
#include "OptimusDeformerInstance.h"
#include "OptimusEditorStyle.h"
#include "OptimusExpressionEvaluator.h"
#include "OptimusHLSLSyntaxHighlighter.h"
#include "OptimusHelpers.h"
#include "OptimusNode.h"
#include "OptimusResourceDescription.h"
#include "OptimusShaderText.h"
#include "OptimusSource.h"
#include "OptimusValidatedName.h"
#include "OptimusValueContainer.h"
#include "OptimusExecutionDomain.h"
#include "OptimusValueContainerStruct.h"
#include "PropertyBagDetails.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SButton.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOptimusDataTypeSelector.h"
#include "Widgets/SOptimusShaderTextDocumentTextBox.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "OptimusDetailCustomization"

static const FName ExpressionMarkerName = TEXT("~");

TSharedRef<IPropertyTypeCustomization> FOptimusDataTypeRefCustomization::MakeInstance()
{
	return MakeShared<FOptimusDataTypeRefCustomization>();
}


void FOptimusDataTypeRefCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle, 
	FDetailWidgetRow& InHeaderRow, 
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	// Usage mask can change on a per-instance basis when the multi-level data domain field changes in a shader parameter binding
	auto GetUsageMask = [this, InPropertyHandle]()
	{
		if (UsageMaskOverride != EOptimusDataTypeUsageFlags::None)
		{
			return UsageMaskOverride;
		}
		
		EOptimusDataTypeUsageFlags UsageMask = EOptimusDataTypeUsageFlags::None;
	
		if (InPropertyHandle->HasMetaData(FName(TEXT("UseInResource"))))
		{
			UsageMask |= EOptimusDataTypeUsageFlags::Resource;
		}
		if (InPropertyHandle->HasMetaData(FName(TEXT("UseInVariable"))))
		{
			UsageMask |= EOptimusDataTypeUsageFlags::Variable;
		}
		if (InPropertyHandle->HasMetaData(FName(TEXT("UseInAnimAttribute"))))
		{
			UsageMask |= EOptimusDataTypeUsageFlags::AnimAttributes;
		}
		if (InPropertyHandle->HasMetaData(FName(TEXT("UseInPerBoneAnimAttribute"))))
		{
			UsageMask |= EOptimusDataTypeUsageFlags::PerBoneAnimAttribute;
		}
		if (InPropertyHandle->HasMetaData(FName(TEXT("UseInProperty"))))
		{
			UsageMask |= EOptimusDataTypeUsageFlags::Property;
		}

		return UsageMask;
	};


	TypeNameProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDataTypeRef, TypeName));
	TypeObjectProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDataTypeRef, TypeObject));

	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SOptimusDataTypeSelector)
		.IsEnabled_Lambda([InPropertyHandle]() -> bool
		{
			return InPropertyHandle->IsEditable();
		})
		.CurrentDataType(this, &FOptimusDataTypeRefCustomization::GetCurrentDataType)
		.UsageMask_Lambda(GetUsageMask)
		.Font(InCustomizationUtils.GetRegularFont())
		.OnDataTypeChanged(this, &FOptimusDataTypeRefCustomization::OnDataTypeChanged)
	];
}


void FOptimusDataTypeRefCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	// FIXME: This doesn't update quite properly. Need a better approach.
	FDetailWidgetRow& DeclarationRow = InChildBuilder.AddCustomRow(LOCTEXT("Declaration", "Declaration"));

	DeclarationRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget(LOCTEXT("Declaration", "Declaration"))
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SBox)
		.MinDesiredWidth(180.0f)
		[
			SNew(SMultiLineEditableTextBox)
			.Text(this, &FOptimusDataTypeRefCustomization::GetDeclarationText)
			.Font(FCoreStyle::GetDefaultFontStyle("Mono", InCustomizationUtils.GetRegularFont().Size))
			.IsReadOnly(true)
		]
	];
}


FOptimusDataTypeHandle FOptimusDataTypeRefCustomization::GetCurrentDataType() const
{
	FName TypeName;
	TypeNameProperty->GetValue(TypeName);
	return FOptimusDataTypeRegistry::Get().FindType(TypeName);
}



void FOptimusDataTypeRefCustomization::OnDataTypeChanged(FOptimusDataTypeHandle InDataType)
{
	if (!InDataType.IsValid())
	{
		// Do not accept invalid input
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("SetDataType", "Set Data Type"));
	CurrentDataType = InDataType;
	
	// We have to change the object property first
	// because by the time we change the type name,
	// owner of the property might use the data type ref to construct the default value container, 
	// at which point we have to make sure the type ref is complete
	TypeObjectProperty->SetValue(InDataType.IsValid() ? InDataType->TypeObject.Get() : nullptr);
	TypeNameProperty->SetValue(InDataType.IsValid() ? InDataType->TypeName : NAME_None);
}


FText FOptimusDataTypeRefCustomization::GetDeclarationText() const
{
	FOptimusDataTypeHandle DataType = GetCurrentDataType();

	if (DataType.IsValid() && DataType->ShaderValueType.IsValid())
	{
		const FShaderValueTypeHandle& ValueType = DataType->ShaderValueType;
		FText Declaration;
		if (ValueType->Type == EShaderFundamentalType::Struct)
		{
			TArray<FShaderValueTypeHandle> StructTypes = ValueType->GetMemberStructTypes();
			StructTypes.Add(ValueType);

			// Collect all friendly names
			TMap<FName, FName> FriendlyNameMap;
			for (const FShaderValueTypeHandle& TypeHandle : StructTypes )
			{
				const FOptimusDataTypeHandle DataTypeHandle = FOptimusDataTypeRegistry::Get().FindType(TypeHandle->Name);
				if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(DataTypeHandle->TypeObject))
				{
					FriendlyNameMap.Add(TypeHandle->Name) = Optimus::GetTypeName(Struct, false);
				}
			}
	
			return FText::FromString(ValueType->GetTypeDeclaration(FriendlyNameMap, true));
		}
		else
		{
			return FText::FromString(ValueType->ToString());
		}
	}
	else
	{
		return FText::GetEmpty();
	}
}

// =============================================================================================

TSharedRef<IPropertyTypeCustomization> FOptimusExecutionDomainCustomization::MakeInstance()
{
	return MakeShared<FOptimusExecutionDomainCustomization>();
}


FOptimusExecutionDomainCustomization::FOptimusExecutionDomainCustomization()
{
}


void FOptimusExecutionDomainCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	TArray<UObject*> OwningObjects;
	InPropertyHandle->GetOuterObjects(OwningObjects);
	
	WeakOwningObjects = TArray<TWeakObjectPtr<UObject>>(OwningObjects);
	
	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([InPropertyHandle]()
		{
			// Show the expression box if the domain is an expression (and all match), otherwise show the domain name drop down.
			if (TOptional<FOptimusExecutionDomain> DataDomain = TryGetSingleExecutionDomain(InPropertyHandle, DomainType|DomainExpression))
			{
				return DataDomain->Type == EOptimusExecutionDomainType::Expression ? 1 : 0;
			}
			return 0;
		})
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(ComboBox, SComboBox<FName>)
			.OptionsSource(&ContextNames)
			.IsEnabled_Lambda([InPropertyHandle]() -> bool
			{
				return InPropertyHandle->IsEditable();
			})
			.OnGenerateWidget_Lambda([this](FName InName)
			{
				return SNew(STextBlock)
					.Text(FormatContextName(InName))
					.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
			})
			.OnSelectionChanged_Lambda([InPropertyHandle, this](FName InName, ESelectInfo::Type InSelectType)
			{
				// This is for the initial selection.
				if (InSelectType == ESelectInfo::Direct)
				{
					return;
				}

				FOptimusExecutionDomain ExecutionDomain;
				if (InName == ExpressionMarkerName)
				{
					if (TOptional<FOptimusExecutionDomain> ExecutionDomainOpt = TryGetSingleExecutionDomain(InPropertyHandle))
					{
						ExecutionDomain = *ExecutionDomainOpt;
						
						ExecutionDomain.Expression = ExecutionDomain.Name.ToString();
					}
					else
					{
						ExecutionDomain.Expression = FString();
					}
					ExecutionDomain.Type = EOptimusExecutionDomainType::Expression;
				}
				else
				{
					ExecutionDomain.Type = EOptimusExecutionDomainType::DomainName;
					ExecutionDomain.Name = InName;
				}

				SetExecutionDomain(InPropertyHandle, ExecutionDomain);
			})
			.OnComboBoxOpening(this, &FOptimusExecutionDomainCustomization::UpdateContextNames)
			[
				SNew(STextBlock)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text_Lambda([InPropertyHandle, this]()
				{
					if (TOptional<FOptimusExecutionDomain> ExecutionDomain = TryGetSingleExecutionDomain(InPropertyHandle))
					{
						return FormatContextName(ExecutionDomain->Name);
					}

					return LOCTEXT("MultipleValues", "Multiple Values");
				})
				.ColorAndOpacity_Lambda([InPropertyHandle, this]()
				{
					if (TOptional<FOptimusExecutionDomain> ExecutionDomain = TryGetSingleExecutionDomain(InPropertyHandle))
					{
						if (!ExecutionDomain->IsDefined())
						{
							return FSlateColor(FLinearColor::Red);
						}
					}

					return FSlateColor::UseStyle();
				})
			]
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ExpressionTextBox, SEditableTextBox)
				.IsEnabled_Lambda([InPropertyHandle]() -> bool
				{
					return InPropertyHandle->IsEditable();
				})
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.HintText(LOCTEXT("ExpressionHint", "Expression..."))
				.Text_Lambda([InPropertyHandle, this]() -> FText
				{
					if (TOptional<FOptimusExecutionDomain> ExecutionDomain = TryGetSingleExecutionDomain(InPropertyHandle, DomainType|DomainExpression))
					{
						return FText::FromString(ExecutionDomain->Expression);
					}
					return FText::GetEmpty();
				})
				.OnTextChanged_Lambda([this](const FText& InExpressionText)
				{
					using namespace Optimus::Expression;
					
					// Verify that the expression is correct.
					const FString Expression = InExpressionText.ToString();
					if (TOptional<FParseError> ParseError = FEngine().Verify(Expression))
					{
						ExpressionTextBox->SetError(ParseError->Message);
					}
					else
					{
						ExpressionTextBox->SetError(FString());
					}
				})
				.OnTextCommitted_Lambda([InPropertyHandle, this](const FText& InExpressionText, ETextCommit::Type)
				{
					using namespace Optimus::Expression;
					
					// Only commit the text if the expression parses.
					const FString Expression = InExpressionText.ToString();
					if (!FEngine().Verify(Expression).IsSet())
					{
						const FOptimusExecutionDomain ExpressionDomain(Expression);
						SetExecutionDomain(InPropertyHandle, ExpressionDomain, DomainType|DomainExpression);
					}
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0, 0.0, 0.0, 0.0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
				.ContentPadding(FMargin(2, 2))
				.OnClicked_Lambda([InPropertyHandle, this]() -> FReply
				{
					FOptimusExecutionDomain NamedDomain;
					NamedDomain.Type = EOptimusExecutionDomainType::DomainName;
					SetExecutionDomain(InPropertyHandle, NamedDomain, DomainType);

					// Making sure Expression option can be selected again
					ComboBox->ClearSelection();
					
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("ClearExpression", "Clear Expression"))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]
		]
	];
}

FText FOptimusExecutionDomainCustomization::FormatContextName(FName InName) const
{
	if (InName.IsNone())
	{
		return LOCTEXT("NoneName", "<None>");
	}
	if (InName == ExpressionMarkerName)
	{
		return LOCTEXT("ExpressionName", "Expression...");
	}

	return  FText::FromName(InName);
}

void FOptimusExecutionDomainCustomization::UpdateContextNames()
{
	TSet<FName> CollectedContextNames;
	for (int32 Index = 0; Index < WeakOwningObjects.Num(); Index++)
	{
		TSet<FName> ProviderContextNames;
		if (const IOptimusExecutionDomainProvider* ExecutionDomainProvider = Cast<IOptimusExecutionDomainProvider>(WeakOwningObjects[Index]))
		{
			if (Index == 0)
			{
				CollectedContextNames.Append(ExecutionDomainProvider->GetExecutionDomains());
			}
			else
			{
				CollectedContextNames = CollectedContextNames.Intersect(TSet<FName>{ExecutionDomainProvider->GetExecutionDomains()});
			}
		}
	}
	
	ContextNames = CollectedContextNames.Array();
	if (ContextNames.IsEmpty())
	{
		ContextNames.Add(NAME_None);
	}
	else
	{
		ContextNames.Sort(FNameLexicalLess{});
		ContextNames.Add(ExpressionMarkerName);
	}
	
	ComboBox->RefreshOptions();
}

void FOptimusExecutionDomainCustomization::SetExecutionDomain(TSharedRef<IPropertyHandle> InPropertyHandle,
	const FOptimusExecutionDomain& InExecutionDomain, DomainFlags InSetFlags)
{
	FScopedTransaction Transaction(LOCTEXT("SetExecutionDomains", "Set Execution Domain"));

	// Ideally we'd like to match up the raw data with the outers, but I'm not
	// convinced that there's always 1-to-1 relation.
	TArray<UObject *> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	for (UObject *OuterObject: OuterObjects)
	{
		// Notify the object that is has been modified so that undo/redo works.
		OuterObject->Modify();
	}
				
	InPropertyHandle->NotifyPreChange();
	TArray<void*> RawDataPtrs;
	InPropertyHandle->AccessRawData(RawDataPtrs);

	for (void* RawPtr: RawDataPtrs)
	{
		FOptimusExecutionDomain* DstExecutionDomain = static_cast<FOptimusExecutionDomain*>(RawPtr);
		if (InSetFlags & DomainType)		DstExecutionDomain->Type = InExecutionDomain.Type;
		if (InSetFlags & DomainName)		DstExecutionDomain->Name = InExecutionDomain.Name;
		if (InSetFlags & DomainExpression)	DstExecutionDomain->Expression = InExecutionDomain.Expression;
	}

	InPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

TOptional<FOptimusExecutionDomain> FOptimusExecutionDomainCustomization::TryGetSingleExecutionDomain(
	TSharedRef<IPropertyHandle> InPropertyHandle, DomainFlags InCompareFlags, bool bInCheckMultiples)
{
	TArray<const void *> RawDataPtrs;
	InPropertyHandle->AccessRawData(RawDataPtrs);

	bool bItemsAreAllSame = false;
	const FOptimusExecutionDomain* ComparatorExecutionDomain = nullptr;
	for (const void* RawPtr: RawDataPtrs)
	{
		// During drag & reorder, invalid binding can be created temporarily
		if (const FOptimusExecutionDomain* ExecutionDomain = static_cast<const FOptimusExecutionDomain*>(RawPtr))
		{
			if (!ComparatorExecutionDomain)
			{
				ComparatorExecutionDomain = ExecutionDomain;
				bItemsAreAllSame = true;
			}
			else 
			{
				if (((InCompareFlags & DomainType) && ComparatorExecutionDomain->Type != ExecutionDomain->Type) ||
					((InCompareFlags & DomainName) && ComparatorExecutionDomain->Name != ExecutionDomain->Name) ||
					((InCompareFlags & DomainExpression) && ComparatorExecutionDomain->Expression != ExecutionDomain->Expression))
				{
					bItemsAreAllSame = false;
					break;
				}
			}
		}
	}

	if (ComparatorExecutionDomain && (!bInCheckMultiples || bItemsAreAllSame))
	{
		return *ComparatorExecutionDomain;
	}
	
	return {};
}


TSharedRef<IPropertyTypeCustomization> FOptimusDataDomainCustomization::MakeInstance()
{
	return MakeShared<FOptimusDataDomainCustomization>();
}

FOptimusDataDomainCustomization::FOptimusDataDomainCustomization() :
	ParameterMarker(MakeShared<TArray<FName>>()),
	ExpressionMarker(MakeShared<TArray<FName>>(TArray<FName>{ExpressionMarkerName}))
{
}


FText FOptimusDataDomainCustomization::FormatDomainDimensionNames(
	TSharedRef<TArray<FName>> InDimensionNames
	) const
{
	if (InDimensionNames == ParameterMarker)
	{
		return LOCTEXT("ParameterEntry", "Parameter");
	}
	if (InDimensionNames == ExpressionMarker)
	{
		return LOCTEXT("ExpressionEntry", "Expression...");
	}

	return FText::FromString(Optimus::FormatDimensionNames(*InDimensionNames));
	
}


void FOptimusDataDomainCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	TArray<UObject*> OwningObjects;
	InPropertyHandle->GetOuterObjects(OwningObjects);
	GenerateDimensionNames(OwningObjects);

	// Pre-defined list of multipliers. If required multiplier is not in this list, then the user should thunk to
	// using expressions.
	static TArray<TSharedRef<int32>> Multipliers;
	if (Multipliers.IsEmpty())
	{
		for (int32 Multiplier: {1,2,3,4,8})
		{
			Multipliers.Add(MakeShared<int32>(Multiplier));
		}
	}
	TSharedRef<int32> InitialMultiplierSelection = MakeShared<int32>(INDEX_NONE);
	if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainMultiplier))
	{
		for (TSharedRef<int32> Multiplier: Multipliers)
		{
			if (DataDomain->Multiplier == *Multiplier)
			{
				InitialMultiplierSelection = Multiplier;
			}
		}
	}
	
	TSharedRef<TArray<FName>> InitialDimensionSelection(MakeShared<TArray<FName>>());
	if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainDimensions))
	{
		for (TSharedRef<TArray<FName>> Dimension: DomainDimensionNames)
		{
			if (DataDomain->DimensionNames == *Dimension)
			{
				InitialDimensionSelection = Dimension;
			}
		}
	}

	if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainAll, /*CheckMultiples=*/false))
	{
		// Broadcast for the initial value, so that outer detail customization can adjust the usage flags accordingly
		OnDataDomainChangedDelegate.Broadcast(*DataDomain);
	}

	FText DimensionSelectorTooltipText;
	if (bAllowParameters)
	{
		DimensionSelectorTooltipText = LOCTEXT("DimensionNamesParameterListerToolTip", "Select a data domain dimension from the list of available dimensions.\nSelect Parameter to set this as a parameter pin.\nSelect Expression... to write a custom data domain dimension expression.");
	}
	else
	{
		DimensionSelectorTooltipText = LOCTEXT("DimensionNamesListerToolTip", "Select a data domain dimension from the list of available dimensions.\nSelect Expression... to write a custom data domain dimension expression.");
	}

	InHeaderRow.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([InPropertyHandle]()
		{
			// Show the expression box if the domain is an expression (and all match), otherwise show the dimension names combo.
			if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainExpression))
			{
				return DataDomain->Type == EOptimusDataDomainType::Expression ? 1 : 0;
			}
			return 0;
		})
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(0.9)
			[
				SAssignNew(DimensionalComboBox, SComboBox<TSharedRef<TArray<FName>>>)
				.ToolTipText(DimensionSelectorTooltipText)
				.OptionsSource(&DomainDimensionNames)
				.InitiallySelectedItem(InitialDimensionSelection)
				.IsEnabled_Lambda([InPropertyHandle]() -> bool
				{
					return InPropertyHandle->IsEditable();
				})
				.OnGenerateWidget_Lambda([this](TSharedRef<TArray<FName>> InNames)
				{
					return SNew(STextBlock)
						.Text(FormatDomainDimensionNames(InNames))
						.Font(InNames->IsEmpty() ? IPropertyTypeCustomizationUtils::GetBoldFont() : IPropertyTypeCustomizationUtils::GetRegularFont());
				})
				.OnSelectionChanged_Lambda([InPropertyHandle, this](TSharedPtr<TArray<FName>> InNames, ESelectInfo::Type InSelectType)
				{
					// This is for the initial selection.
					if (InSelectType == ESelectInfo::Direct)
					{
						return;
					}

					FOptimusDataDomain DataDomain;
					if (InNames == ExpressionMarker)
					{
						FString Expression;
						if (TOptional<FOptimusDataDomain> DataDomainOpt = TryGetSingleDataDomain(InPropertyHandle))
						{
							DataDomain = *DataDomainOpt;
							// FIXME: Should probably be a utility function.
							if (DataDomain.IsSingleton())
							{
								DataDomain.Expression = "1";
							}
							else if (DataDomain.IsMultiDimensional())
							{
								DataDomain.Expression = DataDomain.DimensionNames[0].ToString();
							}
							else
							{
								DataDomain.Expression = DataDomain.AsExpression().Get({});
							}
						}
						else
						{
							DataDomain.Expression = FString();
						}
						DataDomain.Type = EOptimusDataDomainType::Expression;
					}
					else
					{
						DataDomain.Type = EOptimusDataDomainType::Dimensional;
						DataDomain.DimensionNames = *InNames;
					}

					SetDataDomain(InPropertyHandle, DataDomain);
				})
				[
					SNew(STextBlock)
					.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
					.Text_Lambda([InPropertyHandle, this]()
					{
						if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainDimensions))
						{
							switch(DataDomain->Type)
							{
							case EOptimusDataDomainType::Dimensional:
								if (DataDomain->DimensionNames.IsEmpty())
								{
									return LOCTEXT("ParameterEntry", "Parameter");
								}
								else
								{
									return FText::FromString(Optimus::FormatDimensionNames(DataDomain->DimensionNames));								
								}
							case EOptimusDataDomainType::Expression:
								return LOCTEXT("ExpressionEntry", "Expression...");;								
							}
							return FText::GetEmpty();
						}
						else
						{
							return LOCTEXT("MultipleValues", "Multiple Values");
						}
					})
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboBox<TSharedRef<int32>>)
				.ToolTipText(LOCTEXT("MultiplierListerToolTip", "Select how many data values per entry for this dimensional resource"))
				.OptionsSource(&Multipliers)
				.InitiallySelectedItem(InitialMultiplierSelection)
				.Visibility_Lambda([InPropertyHandle]() -> EVisibility
				{
					// Only show if we're doing dimensional values, and only if there's a single level of dimension.
					// FIXME: Lift this restriction.
					if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainDimensions))
					{
						return (DataDomain->Type != EOptimusDataDomainType::Dimensional || DataDomain->DimensionNames.Num() != 1) ? EVisibility::Collapsed : EVisibility::Visible;
					}
					return EVisibility::Collapsed;
				})
				.IsEnabled_Lambda([InPropertyHandle]() -> bool
				{
					return InPropertyHandle->IsEditable();
				})
				.OnGenerateWidget_Lambda([this](TSharedRef<int32> Multiplier)
				{
					return SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("MultiplyInteger", "x {0}"), FText::AsNumber(*Multiplier)))
						.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
				})
				.OnSelectionChanged_Lambda([InPropertyHandle, this](TSharedPtr<int32> InMultiplier, ESelectInfo::Type InSelectType)
				{
					// This is for the initial selection.
					if (InSelectType == ESelectInfo::Direct)
					{
						return;
					}

					FOptimusDataDomain DataDomainMultiplier;
					DataDomainMultiplier.Multiplier = *InMultiplier;
					SetDataDomain(InPropertyHandle, DataDomainMultiplier, DomainMultiplier);
				})
				[
					SNew(STextBlock)
					.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
					.Text_Lambda([InPropertyHandle, this]()
					{
						if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainMultiplier))
						{
							return FText::Format(LOCTEXT("MultiplyInteger", "x {0}"), FText::AsNumber(DataDomain->Multiplier));
						}
						else
						{
							return LOCTEXT("MultipleMultiplyValues", "---");
						}
					})
				]
			]
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ExpressionTextBox, SEditableTextBox)
				.IsEnabled_Lambda([InPropertyHandle]() -> bool
				{
					return InPropertyHandle->IsEditable();
				})
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.HintText(LOCTEXT("ExpressionHint", "Expression..."))
				.Text_Lambda([InPropertyHandle, this]() -> FText
				{
					if (TOptional<FOptimusDataDomain> DataDomain = TryGetSingleDataDomain(InPropertyHandle, DomainType|DomainExpression))
					{
						return FText::FromString(DataDomain->Expression);
					}
					return FText::GetEmpty();
				})
				.OnTextChanged_Lambda([this](const FText& InExpressionText)
				{
					using namespace Optimus::Expression;
					
					// Verify that the expression is correct.
					const FString Expression = InExpressionText.ToString();
					if (TOptional<FParseError> ParseError = FEngine().Verify(Expression))
					{
						ExpressionTextBox->SetError(ParseError->Message);
					}
					else
					{
						ExpressionTextBox->SetError(FString());
					}
				})
				.OnTextCommitted_Lambda([InPropertyHandle, this](const FText& InExpressionText, ETextCommit::Type)
				{
					using namespace Optimus::Expression;
					
					// Only commit the text if the expression parses.
					const FString Expression = InExpressionText.ToString();
					if (!FEngine().Verify(Expression).IsSet())
					{
						const FOptimusDataDomain ExpressionDomain(Expression);
						SetDataDomain(InPropertyHandle, ExpressionDomain, DomainType|DomainExpression);
					}
				})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0, 0.0, 0.0, 0.0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
				.ContentPadding(FMargin(2, 2))
				.OnClicked_Lambda([InPropertyHandle, this]() -> FReply
				{
					const FOptimusDataDomain DimensionalDomain;
					SetDataDomain(InPropertyHandle, DimensionalDomain, DomainType);

					// Making sure Expression option can be selected again
					DimensionalComboBox->ClearSelection();
					
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("ClearExpression", "Clear Expression"))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]
		]
	];
}


void FOptimusDataDomainCustomization::SetAllowParameters(const bool bInAllowParameters)
{
	if (bInAllowParameters != bAllowParameters)
	{
		bAllowParameters = bInAllowParameters;
		if (bInAllowParameters)
		{
			DomainDimensionNames.Insert(ParameterMarker, 0);
		}
		else
		{
			DomainDimensionNames.RemoveAt(0);
		}
	}
}


void FOptimusDataDomainCustomization::GenerateDimensionNames(
	const TArray<UObject*>& InOwningObjects 
	)
{
	// FIXME: Use the owning objects to indicate best candidates.
	DomainDimensionNames.Reset();

	if (bAllowParameters)
	{
		DomainDimensionNames.Add(ParameterMarker);
	}
	
	for (TArray<FName> Names: UOptimusComputeDataInterface::GetUniqueDomainDimensions())
	{
		DomainDimensionNames.Add(MakeShared<TArray<FName>>(Names));
	}
	DomainDimensionNames.Sort([](const TSharedRef<TArray<FName>>& A, const TSharedRef<TArray<FName>> &B)
	{
		// Compare up to the point that we have same number of members to compare.
		for (int32 Index = 0; Index < FMath::Min(A->Num(), B->Num()); Index++)
		{
			if ((*A)[Index] != (*B)[Index])
			{
				return FNameLexicalLess()((*A)[Index], (*B)[Index]);
			}
		}
		// Otherwise the entry with fewer members goes first.
		return A->Num() < B->Num();
	});

	DomainDimensionNames.Add(ExpressionMarker);
}


TOptional<FOptimusDataDomain> FOptimusDataDomainCustomization::TryGetSingleDataDomain(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	DomainFlags InCompareFlags,
	bool bInCheckMultiples
	)
{
	TArray<const void *> RawDataPtrs;
	InPropertyHandle->AccessRawData(RawDataPtrs);

	bool bItemsAreAllSame = false;
	const FOptimusDataDomain* ComparatorDataDomain = nullptr;
	for (const void* RawPtr: RawDataPtrs)
	{
		// During drag & reorder, invalid binding can be created temporarily
		if (const FOptimusDataDomain* DataDomain = static_cast<const FOptimusDataDomain*>(RawPtr))
		{
			if (!ComparatorDataDomain)
			{
				ComparatorDataDomain = DataDomain;
				bItemsAreAllSame = true;
			}
			else 
			{
				if (((InCompareFlags & DomainType) && ComparatorDataDomain->Type != DataDomain->Type) ||
					((InCompareFlags & DomainDimensions) && ComparatorDataDomain->DimensionNames != DataDomain->DimensionNames) ||
					((InCompareFlags & DomainMultiplier) && ComparatorDataDomain->Multiplier != DataDomain->Multiplier) ||
					((InCompareFlags & DomainExpression) && ComparatorDataDomain->Expression != DataDomain->Expression))
				{
					bItemsAreAllSame = false;
					break;
				}
			}
		}
	}

	if (ComparatorDataDomain && (!bInCheckMultiples || bItemsAreAllSame))
	{
		return *ComparatorDataDomain;
	}
	
	return {};
}


void FOptimusDataDomainCustomization::SetDataDomain(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	const FOptimusDataDomain& InDataDomain,
	DomainFlags InSetFlags
	)
{
	FScopedTransaction Transaction(LOCTEXT("SetDataDomains", "Set Data Domain"));

	// Ideally we'd like to match up the raw data with the outers, but I'm not
	// convinced that there's always 1-to-1 relation.
	TArray<UObject *> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	for (UObject *OuterObject: OuterObjects)
	{
		// Notify the object that is has been modified so that undo/redo works.
		OuterObject->Modify();
	}
				
	InPropertyHandle->NotifyPreChange();
	TArray<void*> RawDataPtrs;
	InPropertyHandle->AccessRawData(RawDataPtrs);

	for (void* RawPtr: RawDataPtrs)
	{
		FOptimusDataDomain* DstDataDomain = static_cast<FOptimusDataDomain*>(RawPtr);
		if (InSetFlags & DomainType)		DstDataDomain->Type = InDataDomain.Type;
		if (InSetFlags & DomainDimensions)	DstDataDomain->DimensionNames = InDataDomain.DimensionNames;
		if (InSetFlags & DomainMultiplier)	DstDataDomain->Multiplier = InDataDomain.Multiplier;
		if (InSetFlags & DomainExpression)	DstDataDomain->Expression = InDataDomain.Expression;
	}

	InPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	OnDataDomainChangedDelegate.Broadcast(InDataDomain);
}


// =============================================================================================

TSharedRef<IPropertyTypeCustomization> FOptimusShaderTextCustomization::MakeInstance()
{
	return MakeShared<FOptimusShaderTextCustomization>();
}

FOptimusShaderTextCustomization::FOptimusShaderTextCustomization() :
	SyntaxHighlighter(FOptimusHLSLSyntaxHighlighter::Create())
{
	
}

void FOptimusShaderTextCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle, 
	FDetailWidgetRow& InHeaderRow, 
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	DeclarationsProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusShaderText, Declarations));
	ShaderTextProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusShaderText, ShaderText));

	HorizontalScrollbar =
	    SNew(SScrollBar)
	        .AlwaysShowScrollbar(true)
	        .Orientation(Orient_Horizontal);

	VerticalScrollbar =
	    SNew(SScrollBar)
			.AlwaysShowScrollbar(true)
	        .Orientation(Orient_Vertical);

	const FTextBlockStyle &TextStyle = FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("TextEditor.NormalText");
	const FSlateFontInfo &Font = TextStyle.Font;

	const FText ShaderTextTitle = LOCTEXT("OptimusShaderTextTitle", "Shader Text");

	InHeaderRow
	.WholeRowContent()
	[
		SAssignNew(ExpandableArea, SExpandableArea)
		.AreaTitle(ShaderTextTitle)
		.InitiallyCollapsed(true)
		.AllowAnimatedTransition(false)
		.BodyContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FOptimusEditorStyle::Get().GetBrush("TextEditor.Border"))
				.BorderBackgroundColor(FLinearColor::Black)
				[
					SNew(SGridPanel)
					.FillColumn(0, 1.0f)
					.FillRow(0, 1.0f)
					+SGridPanel::Slot(0, 0)
					[
						SAssignNew(ShaderEditor, SMultiLineEditableText)
						.Font(Font)
						.TextStyle(&TextStyle)
						.Text(this, &FOptimusShaderTextCustomization::GetShaderText)
						.AutoWrapText(false)
						.IsReadOnly(true)
						.Marshaller(SyntaxHighlighter)
						.HScrollBar(HorizontalScrollbar)
						.VScrollBar(VerticalScrollbar)
					]
					+SGridPanel::Slot(1, 0)
					[
						VerticalScrollbar.ToSharedRef()
					]
					+SGridPanel::Slot(0, 1)
					[
						HorizontalScrollbar.ToSharedRef()
					]
				]
			]
		]
	];
}

FText FOptimusShaderTextCustomization::GetShaderText() const
{
	FString ShaderText;
	ShaderTextProperty->GetValue(ShaderText);
	return FText::FromString(ShaderText);
}


class SOptimusParameterBindingValueWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOptimusParameterBindingValueWidget) {}
	SLATE_END_ARGS()

	SOptimusParameterBindingValueWidget()
		: CustomizationUtils (nullptr)
	{
		
	}
	virtual void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InBindingPropertyHandle,  IPropertyTypeCustomizationUtils& InCustomizationUtils)
	{
		BindingPropertyHandle = InBindingPropertyHandle;
		CustomizationUtils = &InCustomizationUtils;
		
		TSharedPtr<IPropertyHandle> DataTypeProperty = BindingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBinding, DataType));
		const TSharedPtr<IPropertyHandle> DataDomainProperty = BindingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBinding, DataDomain));

		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = InCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();

		FDetailWidgetRow DataTypeHeaderRow;
		DataTypeRefCustomizationInstance = FOptimusDataTypeRefCustomization::MakeInstance();
		DataTypeRefCustomizationInstance->CustomizeHeader(DataTypeProperty.ToSharedRef(), DataTypeHeaderRow, InCustomizationUtils);

		FDetailWidgetRow DataDomainHeaderRow;
		DataDomainCustomizationInstance = FOptimusDataDomainCustomization::MakeInstance();
		StaticCastSharedPtr<FOptimusDataDomainCustomization>(DataDomainCustomizationInstance)
			->OnDataDomainChangedDelegate.AddLambda([this, SelectedObjects](const FOptimusDataDomain& InDataDomain)
			{
				EOptimusDataTypeUsageFlags AllowedFlags = EOptimusDataTypeUsageFlags::None;

				for (TWeakObjectPtr<UObject> Object : SelectedObjects)
				{
					if (const IOptimusParameterBindingProvider* BindingProvider = Cast<const IOptimusParameterBindingProvider>(Object))
					{
						if (AllowedFlags == EOptimusDataTypeUsageFlags::None)
						{
							AllowedFlags = BindingProvider->GetTypeUsageFlags(InDataDomain);
						}
						else
						{
							AllowedFlags &= BindingProvider->GetTypeUsageFlags(InDataDomain);
						}
					}
				}

				StaticCastSharedPtr<FOptimusDataTypeRefCustomization>(DataTypeRefCustomizationInstance)->SetUsageMaskOverride(AllowedFlags);
			});
		DataDomainCustomizationInstance->CustomizeHeader(DataDomainProperty.ToSharedRef(), DataDomainHeaderRow, InCustomizationUtils);
		
		ColumnSizeData = MakeShared<FOptimusParameterBindingCustomization::FColumnSizeData>();

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(SSplitter)
				.Style(FAppStyle::Get(), "DetailsView.Splitter")
				.PhysicalSplitterHandleSize(1.0f)
				.HitDetectionSplitterHandleSize(5.0f)
				+SSplitter::Slot()
				.Value(this, &SOptimusParameterBindingValueWidget::GetDataTypeColumnSize)
				.OnSlotResized(this, &SOptimusParameterBindingValueWidget::OnDataTypeColumnResized)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					// padding values grabbed from DetailWidgetConstants
					.Padding(0,0,10,0)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							DataTypeHeaderRow.ValueContent().Widget
						]
					]
				]
				+SSplitter::Slot()
				.Value(this, &SOptimusParameterBindingValueWidget::GetDataDomainColumnSize)
				.OnSlotResized(this, &SOptimusParameterBindingValueWidget::OnDataDomainColumnResized)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					// padding values grabbed from DetailWidgetConstants
					.Padding(12,0,10,0)
					[
						DataDomainHeaderRow.ValueContent().Widget
					]
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(2.0f,0.f)
			[
				PropertyCustomizationHelpers::MakeEmptyButton(
					FSimpleDelegate::CreateLambda([this]()
					{
						// This is copied from FPropertyEditor::DeleteItem()
						// This action must be deferred until next tick so that we avoid accessing invalid data before we have a chance to tick
						CustomizationUtils->GetPropertyUtilities()->EnqueueDeferredAction(
									FSimpleDelegate::CreateSP(this, &SOptimusParameterBindingValueWidget::OnDeleteItem)
						);
					}),
					LOCTEXT("OptimusParameterBindingRemoveButton", "Remove this Binding"),
					TAttribute<bool>::CreateLambda([this]()
					{
						return BindingPropertyHandle->IsEditable();
					}))
			]
		];
	}

	void SetColumnSizeData(TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> InData)
	{
		ColumnSizeData = InData;
	};

	void SetAllowParameters(const bool bInAllowParameters)
	{
		TSharedPtr<FOptimusDataDomainCustomization> DataDomainCustomization = StaticCastSharedPtr<FOptimusDataDomainCustomization>(DataDomainCustomizationInstance);
		DataDomainCustomization->SetAllowParameters(bInAllowParameters);
	}
	

private:
	void OnDeleteItem() const
	{
		TSharedPtr<IPropertyHandleArray> ArrayHandle = BindingPropertyHandle->GetParentHandle()->AsArray();

		check(ArrayHandle.IsValid());

		int32 Index = BindingPropertyHandle->GetArrayIndex();

		if (ArrayHandle.IsValid())
		{
			ArrayHandle->DeleteItem(Index);
		}

		//In case the property is show in the favorite category refresh the whole tree
		if (BindingPropertyHandle->IsFavorite() || (BindingPropertyHandle->GetParentHandle() != nullptr && BindingPropertyHandle->GetParentHandle()->IsFavorite()))
		{
			CustomizationUtils->GetPropertyUtilities()->ForceRefresh();
		}
	};

	float GetDataTypeColumnSize() const {return ColumnSizeData->GetDataTypeColumnSize();}
	void OnDataTypeColumnResized(float InSize) const {ColumnSizeData->OnDataDomainColumnResized(InSize);}
	float GetDataDomainColumnSize() const {return ColumnSizeData->GetDataDomainColumnSize();}
	void OnDataDomainColumnResized(float InSize) const {ColumnSizeData ->OnDataDomainColumnResized(InSize);}
	
	TSharedPtr<IPropertyHandle> BindingPropertyHandle;
	IPropertyTypeCustomizationUtils* CustomizationUtils;

	TSharedPtr<IPropertyTypeCustomization> DataTypeRefCustomizationInstance;
	TSharedPtr<IPropertyTypeCustomization> DataDomainCustomizationInstance;
	
	TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> ColumnSizeData;
};


TSharedRef<IPropertyTypeCustomization> FOptimusParameterBindingCustomization::MakeInstance()
{
	return MakeShared<FOptimusParameterBindingCustomization>();
}

EVisibility FOptimusParameterBindingCustomization::IsAtomicCheckBoxVisible(const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects,  TSharedRef<IPropertyHandle> InPropertyHandle)
{
	for (TWeakObjectPtr<UObject> Object : InSelectedObjects)
	{
		if (IOptimusParameterBindingProvider* BindingProvider = Cast<IOptimusParameterBindingProvider>(Object))
		{
			TArray<const void *> RawData;

			InPropertyHandle->AccessRawData(RawData);
			if (RawData.Num() > 0)
			{
				const FOptimusParameterBinding* Binding = static_cast<const FOptimusParameterBinding*>(RawData[0]);
				// During drag & reorder, we can have invalid bindings in the property
				if (Binding && Binding->Name != NAME_None)
				{
					return BindingProvider->GetBindingSupportAtomicCheckBoxVisibility(Binding->Name) ? EVisibility::Visible : EVisibility::Collapsed;
				}
			}
			break;
		}
	}
		
	return EVisibility::Collapsed;
}

EVisibility FOptimusParameterBindingCustomization::IsSupportReadCheckBoxVisible(
	const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	for (TWeakObjectPtr<UObject> Object : InSelectedObjects)
    	{
    		if (IOptimusParameterBindingProvider* BindingProvider = Cast<IOptimusParameterBindingProvider>(Object))
    		{
    			TArray<const void *> RawData;
    
    			InPropertyHandle->AccessRawData(RawData);
    			if (RawData.Num() > 0)
    			{
    				const FOptimusParameterBinding* Binding = static_cast<const FOptimusParameterBinding*>(RawData[0]);
    				// During drag & reorder, we can have invalid bindings in the property
    				if (Binding && Binding->Name != NAME_None)
    				{
    					return BindingProvider->GetBindingSupportReadCheckBoxVisibility(Binding->Name) ? EVisibility::Visible : EVisibility::Collapsed;
    				}
    			}
    			break;
    		}
    	}
    		
    	return EVisibility::Collapsed;
}

FOptimusParameterBindingCustomization::FOptimusParameterBindingCustomization()
{
	
}

void FOptimusParameterBindingCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedRef<IPropertyHandle>& BindingPropertyHandle = InPropertyHandle;
	const TSharedPtr<IPropertyHandle> ValidatedNameProperty = BindingPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBinding, Name));
	const TSharedPtr<IPropertyHandle> NameProperty = ValidatedNameProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusValidatedName, Name));
	
	InHeaderRow
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0,0,10,0)
		[
			SNew(SEditableTextBox)
			.IsEnabled_Lambda([NameProperty]() -> bool
			{
				return NameProperty->IsEditable();
			})
			.Font(InCustomizationUtils.GetRegularFont())
			.Text_Lambda([NameProperty]()
			{
				FName Value;
				NameProperty->GetValue(Value);
				return FText::FromName(Value);
			})
			.OnTextCommitted_Lambda([NameProperty](const FText& InText, ETextCommit::Type InTextCommit)
			{
				NameProperty->SetValue(FName(InText.ToString()));
			})
			.OnVerifyTextChanged_Lambda([NameProperty](const FText& InNewText, FText& OutErrorMessage) -> bool
			{
				if (InNewText.IsEmpty())
				{
					OutErrorMessage = LOCTEXT("NameEmpty", "Name can't be empty.");
					return false;
				}
					
				FText FailureContext = LOCTEXT("NameFailure", "Name");
				if (!FOptimusValidatedName::IsValid(InNewText.ToString(), &OutErrorMessage, &FailureContext))
				{
					return false;
				}
					
				return true;
			})
		]
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		SNew(SOptimusParameterBindingValueWidget, BindingPropertyHandle, InCustomizationUtils)
	];	
}

void FOptimusParameterBindingCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = InCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();

	auto GetDeclarationText = [SelectedObjects, InPropertyHandle]()
	{
		FString Declaration;
		
		for (TWeakObjectPtr<UObject> Object : SelectedObjects)
		{
			if (IOptimusParameterBindingProvider* BindingProvider = Cast<IOptimusParameterBindingProvider>(Object))
			{
				TArray<const void *> RawData;

				InPropertyHandle->AccessRawData(RawData);
				if (ensure(RawData.Num() > 0))
				{
					const FOptimusParameterBinding* Binding = static_cast<const FOptimusParameterBinding*>(RawData[0]);
					// During drag & reorder, we can have invalid bindings in the property
					if (Binding && Binding->Name != NAME_None)
					{
						if (Binding->DataType->ShaderValueType.IsValid())
						{
							Declaration = BindingProvider->GetBindingDeclaration(Binding->Name);
						}
						else
						{
							Declaration = FString::Printf(TEXT("Type is not supported"));
						}
					}
				}
				break;
			}
		}

		return FText::FromString(Declaration);
	};
	
	if (!GetDeclarationText().IsEmpty())
	{
		FDetailWidgetRow& DeclarationRow = InChildBuilder.AddCustomRow(FText::GetEmpty());
		DeclarationRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget(LOCTEXT("Declaration", "Declaration"))
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SBox)
			.MinDesiredWidth(180.0f)
			[
				SNew(SMultiLineEditableTextBox)
				.Text_Lambda(GetDeclarationText)
				.Font(FCoreStyle::GetDefaultFontStyle("Mono",InCustomizationUtils.GetRegularFont().Size))
				.IsReadOnly(true)
			]
		];		
	}

	auto IsAtomicRowVisible = [SelectedObjects, InPropertyHandle]()
	{
		return IsAtomicCheckBoxVisible(SelectedObjects, InPropertyHandle);
	};
	

	TSharedPtr<IPropertyHandle> SupportAtomicHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBinding, bSupportAtomicIfCompatibleDataType), false);
	IDetailPropertyRow& AtomicRow = InChildBuilder.AddProperty(SupportAtomicHandle.ToSharedRef());
	AtomicRow.Visibility(TAttribute<EVisibility>::CreateLambda(IsAtomicRowVisible));

	auto IsSupportReadRowVisible = [SelectedObjects, InPropertyHandle]()
	{
		return IsSupportReadCheckBoxVisible(SelectedObjects, InPropertyHandle);
	};
	
	TSharedPtr<IPropertyHandle> SupportReadHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBinding, bSupportRead), false);
	IDetailPropertyRow& SupportReadRow = InChildBuilder.AddProperty(SupportReadHandle.ToSharedRef());
	SupportReadRow.Visibility(TAttribute<EVisibility>::CreateLambda(IsSupportReadRowVisible));
}

TSharedRef<FOptimusParameterBindingArrayBuilder> FOptimusParameterBindingArrayBuilder::MakeInstance(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> InColumnSizeData,
	const bool bInAllowParameters
	)
{
	TSharedRef<FOptimusParameterBindingArrayBuilder> Builder = MakeShared<FOptimusParameterBindingArrayBuilder>(InPropertyHandle, InColumnSizeData, bInAllowParameters);
	
	Builder->OnGenerateArrayElementWidget(
		FOnGenerateArrayElementWidget::CreateSP(Builder, &FOptimusParameterBindingArrayBuilder::OnGenerateEntry));
	return Builder;
}

FOptimusParameterBindingArrayBuilder::FOptimusParameterBindingArrayBuilder(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> InColumnSizeData,
	const bool bInAllowParameters
	) : FDetailArrayBuilder(InPropertyHandle, true, false, true),
		ArrayProperty(InPropertyHandle->AsArray()),
		ColumnSizeData(InColumnSizeData),
		bAllowParameters(bInAllowParameters)
{
	if (!ColumnSizeData.IsValid())
	{
		ColumnSizeData = MakeShared<FOptimusParameterBindingCustomization::FColumnSizeData>();
	}
}

void FOptimusParameterBindingArrayBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	// Do nothing since we don't want to show the "InnerArray" row, see FOptimusParameterBindingArrayCustomization::CustomizeHeader
}

void FOptimusParameterBindingArrayBuilder::GenerateWrapperStructHeaderRowContent(FDetailWidgetRow& NodeRow, TSharedRef<SWidget> NameContent)
{
	FDetailArrayBuilder::GenerateHeaderRowContent(NodeRow);
	NodeRow.ValueContent()
	.HAlign( HAlign_Left )
	.VAlign( VAlign_Center )
	// Value grabbed from SPropertyEditorArray::GetDesiredWidth
	.MinDesiredWidth(170.f)
	.MaxDesiredWidth(170.f);

	NodeRow.NameContent()
	[
		NameContent
	];
}


void FOptimusParameterBindingArrayBuilder::OnGenerateEntry(
	TSharedRef<IPropertyHandle> ElementProperty,
	int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder) const
{
	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(ElementProperty);
	PropertyRow.ShowPropertyButtons(false);
	PropertyRow.ShouldAutoExpand(false);

	// Hide the reset to default button since it provides little value
	const FResetToDefaultOverride	ResetDefaultOverride = FResetToDefaultOverride::Create(TAttribute<bool>(false));
	PropertyRow.OverrideResetToDefault(ResetDefaultOverride);
	
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	PropertyRow.GetDefaultWidgets( NameWidget, ValueWidget);
	PropertyRow.CustomWidget(true)
	.NameContent()
	.HAlign(HAlign_Fill)
	[
		NameWidget.ToSharedRef()
	]
	.ValueContent()
	.HAlign(HAlign_Fill)
	[
		ValueWidget.ToSharedRef()
	];
	
	const TSharedPtr<SHorizontalBox> HBox = StaticCastSharedPtr<SHorizontalBox>(ValueWidget);
	const TSharedPtr<SWidget> InnerValueWidget = HBox->GetSlot(0).GetWidget();
	const TSharedPtr<SOptimusParameterBindingValueWidget> OptimusValueWidget = StaticCastSharedPtr<SOptimusParameterBindingValueWidget>(InnerValueWidget);
	OptimusValueWidget->SetColumnSizeData(ColumnSizeData);
	OptimusValueWidget->SetAllowParameters(bAllowParameters);
}

FOptimusParameterBindingArrayCustomization::FOptimusParameterBindingArrayCustomization()
	: ColumnSizeData(MakeShared<FOptimusParameterBindingCustomization::FColumnSizeData>())
{
}

void FOptimusParameterBindingArrayCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
                                                                 FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const bool bAllowParameters = InPropertyHandle->HasMetaData(UOptimusNode::PropertyMeta::AllowParameters);
	const TSharedPtr<IPropertyHandle> ArrayHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusParameterBindingArray, InnerArray), false);

	ArrayBuilder = FOptimusParameterBindingArrayBuilder::MakeInstance(ArrayHandle.ToSharedRef(), ColumnSizeData, bAllowParameters);
	// use the top level property instead of "InnerArray"
	ArrayBuilder->GenerateWrapperStructHeaderRowContent(InHeaderRow,InPropertyHandle->CreatePropertyNameWidget());
}

void FOptimusParameterBindingArrayCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
                                                                   IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InChildBuilder.AddCustomBuilder(ArrayBuilder.ToSharedRef());
}

FOptimusValueContainerStructCustomization::FOptimusValueContainerStructCustomization()
{
	PropertyBagCustomization = FPropertyBagDetails::MakeInstance();
}

void FOptimusValueContainerStructCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	uint32 NumChildren = 0;
	InPropertyHandle->GetNumChildren(NumChildren);
	
	// During reordering, we may have zero children temporarily
	if (NumChildren > 0)
	{
		InnerPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusValueContainerStruct, Value));

		PropertyBagCustomization->CustomizeHeader(InnerPropertyHandle.ToSharedRef(), InHeaderRow, InCustomizationUtils);
	}
}

void FOptimusValueContainerStructCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	if (InnerPropertyHandle)
	{
		PropertyBagCustomization->CustomizeChildren(InnerPropertyHandle.ToSharedRef(), InChildBuilder, InCustomizationUtils);
	}
}


FOptimusValidatedNameCustomization::FOptimusValidatedNameCustomization()
{
}

TSharedRef<IPropertyTypeCustomization> FOptimusValidatedNameCustomization::MakeInstance()
{
	return MakeShared<FOptimusValidatedNameCustomization>();
}

void FOptimusValidatedNameCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> NameProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusValidatedName, Name));
	
	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SEditableTextBox)
		.Font(InCustomizationUtils.GetRegularFont())
		.Text_Lambda([NameProperty]()
		{
			FName Value;
			NameProperty->GetValue(Value);
			return FText::FromName(Value);
		})
		.OnTextCommitted_Lambda([NameProperty](const FText& InText, ETextCommit::Type InTextCommit)
		{
			NameProperty->SetValue(FName(InText.ToString()));
		})
		.OnVerifyTextChanged_Lambda([NameProperty](const FText& InNewText, FText& OutErrorMessage) -> bool
		{
			if (InNewText.IsEmpty())
			{
				OutErrorMessage = LOCTEXT("NameEmpty", "Name can't be empty.");
				return false;
			}
				
			FText FailureContext = LOCTEXT("NameFailure", "Name");
			if (!FOptimusValidatedName::IsValid(InNewText.ToString(), &OutErrorMessage, &FailureContext))
			{
				return false;
			}
				
			return true;
		})
	];
}


FOptimusSourceDetailsCustomization::FOptimusSourceDetailsCustomization()
	: SyntaxHighlighter(FOptimusHLSLSyntaxHighlighter::Create())
{
}

TSharedRef<IDetailCustomization> FOptimusSourceDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FOptimusSourceDetailsCustomization);
}

void FOptimusSourceDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	OptimusSource = Cast<UOptimusSource>(ObjectsBeingCustomized[0].Get());
	if (OptimusSource == nullptr)
	{
		return;
	}

	TSharedRef<IPropertyHandle> SourcePropertyHandle = DetailBuilder.GetProperty(TEXT("SourceText"));
	if (TSharedPtr<IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr())
	{
		DetailsView->HideFilterArea(true);
	}
	DetailBuilder.EditDefaultProperty(SourcePropertyHandle)->CustomWidget()
	[
		SNew(SBox)
		.HeightOverride(500)
		[
			SAssignNew(SourceTextBox, SOptimusShaderTextDocumentTextBox)
			.Text(this, &FOptimusSourceDetailsCustomization::GetText)
			.IsReadOnly(false)
			.Marshaller(SyntaxHighlighter)
			.OnTextChanged(this, &FOptimusSourceDetailsCustomization::OnTextChanged)
		]
	];
}

FText FOptimusSourceDetailsCustomization::GetText() const
{
	return FText::FromString(OptimusSource->GetShaderText());
}

void FOptimusSourceDetailsCustomization::OnTextChanged(const FText& InValue)
{
	OptimusSource->SetSource(InValue.ToString());
}


FOptimusComponentSourceBindingDetailsCustomization::FOptimusComponentSourceBindingDetailsCustomization()

{
}

TSharedRef<IDetailCustomization> FOptimusComponentSourceBindingDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FOptimusComponentSourceBindingDetailsCustomization);
}
void FOptimusComponentSourceBindingDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	OptimusSourceBinding = Cast<UOptimusComponentSourceBinding>(ObjectsBeingCustomized[0].Get());
	if (OptimusSourceBinding == nullptr)
	{
		return;
	}

	// Collect and sort ComponentSources for combo box.
	UOptimusComponentSource* CurrentSource = OptimusSourceBinding->ComponentType->GetDefaultObject<UOptimusComponentSource>();
	TSharedPtr<FString> CurrentSelection;
	for (const UOptimusComponentSource* Source : UOptimusComponentSource::GetAllSources())
	{
		if (!OptimusSourceBinding->IsPrimaryBinding() || Source->IsUsableAsPrimarySource())
		{
			TSharedPtr<FString> SourceName = MakeShared<FString>(Source->GetDisplayName().ToString());
			if (Source == CurrentSource)
			{
				CurrentSelection = SourceName;
			}
			ComponentSources.Add(SourceName);
		}
	}
	Algo::Sort(ComponentSources, [](TSharedPtr<FString> ItemA, TSharedPtr<FString> ItemB)
	{
		return ItemA->Compare(*ItemB) < 0;
	});

	TSharedRef<IPropertyHandle> SourcePropertyHandle = DetailBuilder.GetProperty(TEXT("ComponentType"));
	DetailBuilder.EditDefaultProperty(SourcePropertyHandle)->ShowPropertyButtons(false)
	.CustomWidget()
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.NameContent()
	[
		SourcePropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(STextComboBox)
			.OptionsSource(&ComponentSources)
			.InitiallySelectedItem(CurrentSelection)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OnSelectionChanged(this, &FOptimusComponentSourceBindingDetailsCustomization::ComponentSourceChanged)
	];
}

void FOptimusComponentSourceBindingDetailsCustomization::ComponentSourceChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	for (const UOptimusComponentSource* Source : UOptimusComponentSource::GetAllSources())
	{
		if (*Selection == Source->GetDisplayName().ToString())
		{
			UOptimusDeformer* Deformer = OptimusSourceBinding->GetOwningDeformer();
			Deformer->SetComponentBindingSource(OptimusSourceBinding, Source);
			return;
		}
	}
}


TSharedRef<IDetailCustomization> FOptimusResourceDescriptionDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FOptimusResourceDescriptionDetailsCustomization);
}


void FOptimusResourceDescriptionDetailsCustomization::CustomizeDetails(
	IDetailLayoutBuilder& InDetailBuilder
	)
{
	TArray<TWeakObjectPtr<UObject>> ResourceDescriptionObjects;
	InDetailBuilder.GetObjectsBeingCustomized(ResourceDescriptionObjects);

	const UOptimusResourceDescription* ResourceDescription = Cast<UOptimusResourceDescription>(ResourceDescriptionObjects[0].Get());
	if (ResourceDescription && ResourceDescription->GetOwningDeformer())
	{
		ComponentBindings = ResourceDescription->GetOwningDeformer()->GetComponentBindings();
	}

	TSharedPtr<IPropertyHandle> ComponentBindingProperty = InDetailBuilder.GetProperty(GET_MEMBER_NAME_STRING_CHECKED(UOptimusResourceDescription, ComponentBinding));
	IDetailPropertyRow* ComponentBindingRow = InDetailBuilder.EditDefaultProperty(ComponentBindingProperty);

	UObject* SelectedBinding = nullptr;
	ComponentBindingProperty->GetValue(SelectedBinding);

	ComponentBindingRow->CustomWidget()
		.NameContent()
		[
			ComponentBindingProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboBox<UOptimusComponentSourceBinding*>)
			.OptionsSource(&ComponentBindings)
			.InitiallySelectedItem(Cast<UOptimusComponentSourceBinding>(SelectedBinding))
			.OnGenerateWidget_Lambda([](const UOptimusComponentSourceBinding* InBinding)
			{
				return SNew(STextBlock).Text(FText::FromName(InBinding->BindingName));
			})
			.OnSelectionChanged_Lambda([ResourceDescriptionObjects, ComponentBindingProperty](UOptimusComponentSourceBinding* InBinding, ESelectInfo::Type InInfo)
			{
				if (InInfo != ESelectInfo::Direct)
				{
					ComponentBindingProperty->NotifyPreChange();
					for (TWeakObjectPtr<UObject> ResourceDescriptionObject: ResourceDescriptionObjects)
					{
						UOptimusResourceDescription* ResourceDescription = Cast<UOptimusResourceDescription>(ResourceDescriptionObject.Get());
						if (ResourceDescription)
						{
							ResourceDescription->ComponentBinding = InBinding;
						}
					}
					ComponentBindingProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
				}
			})
			[
				SNew(STextBlock)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text_Lambda([ComponentBindingProperty]()
				{
					UObject* BindingObject = nullptr;
					FPropertyAccess::Result Result = ComponentBindingProperty->GetValue(BindingObject);
					if (Result == FPropertyAccess::MultipleValues)
					{
						return LOCTEXT("MultipleValues", "Multiple Values");
					}
					else if (UOptimusComponentSourceBinding* Binding = Cast<UOptimusComponentSourceBinding>(BindingObject))
					{
						return FText::FromName(Binding->BindingName);
					}
					else
					{
						return FText::GetEmpty();
					}
				})
			]
		];
}


TSharedRef<IPropertyTypeCustomization> FOptimusDeformerInstanceComponentBindingCustomization::MakeInstance()
{
	return MakeShareable(new FOptimusDeformerInstanceComponentBindingCustomization);
}


void FOptimusDeformerInstanceComponentBindingCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	const TSharedPtr<IPropertyHandle> BindingPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDeformerInstanceComponentBinding, ProviderName));
	const TSharedPtr<IPropertyHandle> ComponentPropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDeformerInstanceComponentBinding, ComponentName));
	
	FName BindingName;
	BindingPropertyHandle->GetValue(BindingName);

	FName SelectedComponentName;
	ComponentPropertyHandle->GetValue(SelectedComponentName);
	
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	

	const UOptimusComponentSourceBinding* Binding = nullptr;
	FComponentHandle SelectedComponentHandle;

	if (UOptimusDeformerInstanceSettings const* BindingProvider = Cast<UOptimusDeformerInstanceSettings>(OuterObjects[0]))
	{
		Binding = BindingProvider->GetComponentBindingByName(BindingName);
		
		if (Binding != nullptr)
		{
			TArray<UActorComponent*> FilteredComponents;

			// Add unset "auto" setting.
			FilteredComponents.Add(nullptr);

			// Add actor components.
			if (AActor const* OwningActor = BindingProvider->GetTypedOuter<AActor>())
			{
				OwningActor->GetComponents(Binding->GetComponentSource()->GetComponentClass(), FilteredComponents);
			}

			// Add blueprint components.
			if (UBlueprintGeneratedClass const* OwningBlueprint = BindingProvider->GetTypedOuter<UBlueprintGeneratedClass>())
			{
				const TArray<USCS_Node*>& ActorBlueprintNodes = OwningBlueprint->SimpleConstructionScript->GetAllNodes();
				for (USCS_Node* Node : ActorBlueprintNodes)
				{
					if (Node->ComponentTemplate->IsA(Binding->GetComponentSource()->GetComponentClass()))
					{
						FilteredComponents.Add(Node->ComponentTemplate);
					}
				}
			}

			// Add filtered components to drop down
			for (const UActorComponent* Component : FilteredComponents)
			{
				// Store ComponentNames for reverse lookup from Name->Handle used for getting component icon.
				FName Name = Component ? FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(Component) : FName();
				ComponentNames.Add(Name);

				FSoftObjectPath Path = Component ? FSoftObjectPath::GetOrCreateIDForObject(Component) : FSoftObjectPath();
				ComponentHandles.Add(MakeShared<FSoftObjectPath>(Path));

				if (Name.ToString() == SelectedComponentName.ToString())
				{
					SelectedComponentHandle = ComponentHandles.Last();
				}
			}
		}
	}

	InHeaderRow
	.NameContent()
	[
		ComponentPropertyHandle->CreatePropertyNameWidget(FText::FromName(BindingName))
	]
	.ValueContent()
	[
		SNew(SComboBox<FComponentHandle>)
		.IsEnabled(Binding && !Binding->IsPrimaryBinding())
		.OptionsSource(&ComponentHandles)
		.InitiallySelectedItem(SelectedComponentHandle)
		.OnGenerateWidget_Lambda([](const FComponentHandle InComponentHandle)
		{
			const UActorComponent* Component = Cast<UActorComponent>(InComponentHandle->ResolveObject());

			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(Component ? FSlateIconFinder::FindIconBrushForClass(Component->GetClass(), TEXT("SCS.Component")) : nullptr)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0)
				.Padding(2.0f, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(Component ? FText::FromName(FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(Component)) : LOCTEXT("AutoName", "Auto"))
				];
		})
		.OnSelectionChanged_Lambda([ComponentPropertyHandle](const FComponentHandle InComponentHandle, ESelectInfo::Type InInfo)
		{
			if (InInfo != ESelectInfo::Direct)
			{
				const UActorComponent* Component = Cast<UActorComponent>(InComponentHandle->ResolveObject());
				ComponentPropertyHandle->SetValue(FOptimusDeformerInstanceComponentBinding::GetSanitizedComponentName(Component));
			}
		})
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image_Lambda([this, ComponentPropertyHandle]()-> const FSlateBrush*
				{
					FName Name;
					ComponentPropertyHandle->GetValue(Name);

					int32 Index = ComponentNames.Find(Name);
					const UActorComponent* Component = (Index == INDEX_NONE) ? nullptr : Cast<UActorComponent>(ComponentHandles[Index]->ResolveObject());
					return Component ? FSlateIconFinder::FindIconBrushForClass(Component->GetClass(), TEXT("SCS.Component")) : nullptr;
				})
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0)
			.Padding(2.0f, 0, 0, 0)
			[
				SNew(STextBlock)
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.Text_Lambda([ComponentPropertyHandle]()
				{
					FName Name;
					ComponentPropertyHandle->GetValue(Name);
					return Name.IsNone() ? LOCTEXT("AutoName", "Auto") : FText::FromName(Name);
				})
			]
		]
	]
 	.OverrideResetToDefault(FResetToDefaultOverride::Create(
 		FIsResetToDefaultVisible::CreateLambda([](TSharedPtr<IPropertyHandle> PropertyHandle) 
		{
			const TSharedPtr<IPropertyHandle> ComponentPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDeformerInstanceComponentBinding, ComponentName));
			return PropertyHandle->GetIndexInArray() != 0 && ComponentPropertyHandle->CanResetToDefault();
		}),
 		FResetToDefaultHandler::CreateLambda([](TSharedPtr<IPropertyHandle> PropertyHandle) 
		{
			TSharedPtr<IPropertyHandle> ComponentPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FOptimusDeformerInstanceComponentBinding, ComponentName));
			ComponentPropertyHandle->ResetToDefault();
		})
	));
}

#undef LOCTEXT_NAMESPACE
