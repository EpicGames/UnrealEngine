// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "OptimusDataType.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboBox.h"

struct FOptimusExecutionDomain;
class IOptimusExecutionDomainProvider;
class FOptimusHLSLSyntaxHighlighter;
class SEditableTextBox;
class SExpandableArea;
class SMultiLineEditableText;
class SOptimusShaderTextDocumentTextBox;
class SScrollBar;
class UOptimusComponentSourceBinding;
class UOptimusSource;
struct FOptimusDataDomain;


class FOptimusDataTypeRefCustomization : 
	public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader( 
		TSharedRef<IPropertyHandle> InPropertyHandle, 
		FDetailWidgetRow& InHeaderRow, 
		IPropertyTypeCustomizationUtils& InCustomizationUtils 
		) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle, 
		IDetailChildrenBuilder& InChildBuilder, 
		IPropertyTypeCustomizationUtils& InCustomizationUtils
		) override;

	void SetUsageMaskOverride(EOptimusDataTypeUsageFlags InOverride)
	{
		UsageMaskOverride = InOverride;
	}

private:
	
	FOptimusDataTypeHandle GetCurrentDataType() const;
	void OnDataTypeChanged(FOptimusDataTypeHandle InDataType);

	FText GetDeclarationText() const;

	TSharedPtr<IPropertyHandle> TypeNameProperty;
	TSharedPtr<IPropertyHandle> TypeObjectProperty;
	TAttribute<FOptimusDataTypeHandle> CurrentDataType;

	EOptimusDataTypeUsageFlags UsageMaskOverride = EOptimusDataTypeUsageFlags::None;
	
};


class FOptimusExecutionDomainCustomization :
	public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FOptimusExecutionDomainCustomization();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override
	{ }

	enum DomainFlags
	{
		DomainType			= 0x1,
		DomainName			= 0x2,
		DomainExpression	= 0x4,

		DomainAll			= DomainType + DomainName + DomainExpression     
	};
	
	
private:

	FText FormatContextName(
		FName InName
		) const;
	
	void UpdateContextNames();

	static void SetExecutionDomain(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		const FOptimusExecutionDomain& InExecutionDomain,
		DomainFlags InSetFlags = DomainAll
		);
	
	static TOptional<FOptimusExecutionDomain> TryGetSingleExecutionDomain(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		DomainFlags InCompareFlags = DomainAll,
		bool bInCheckMultiples = true
	);
	
	TSharedPtr<SComboBox<FName>> ComboBox;
	TSharedPtr<SEditableTextBox> ExpressionTextBox;
	
	TArray<FName> ContextNames;

	TArray<TWeakObjectPtr<UObject>> WeakOwningObjects;
};

ENUM_CLASS_FLAGS(FOptimusExecutionDomainCustomization::DomainFlags);


class FOptimusDataDomainCustomization :
	public IPropertyTypeCustomization
{
public:
	DECLARE_EVENT_OneParam(FOptimusDataDomainCustomization, FOnDataDomainChanged, const FOptimusDataDomain& );
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FOptimusDataDomainCustomization();
	
	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override
	{ }

	void SetAllowParameters(const bool bInAllowParameters);

	FOnDataDomainChanged OnDataDomainChangedDelegate; 

	enum DomainFlags
	{
		DomainType			= 0x1,
		DomainDimensions	= 0x2,
		DomainMultiplier	= 0x4,
		DomainExpression	= 0x8,

		DomainAll			= DomainType + DomainDimensions + DomainMultiplier + DomainExpression     
	};
	
private:
	
	FText FormatDomainDimensionNames(
		TSharedRef<TArray<FName>> InDimensionNames
		) const;
	
	void GenerateDimensionNames(
		const TArray<UObject*>& InOwningObjects
		);

	static TOptional<FOptimusDataDomain> TryGetSingleDataDomain(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		DomainFlags InCompareFlags = DomainAll,
		bool bInCheckMultiples = true
		);

	void SetDataDomain(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		const FOptimusDataDomain& InDataDomain,
		DomainFlags InSetFlags = DomainAll
		);

	TSharedRef<TArray<FName>> ParameterMarker;
	TSharedRef<TArray<FName>> ExpressionMarker;
	
	TSharedPtr<SComboBox<TSharedRef<TArray<FName>>>> DimensionalComboBox;
	TSharedPtr<SEditableTextBox> ExpressionTextBox;
	
	TArray<TSharedRef<TArray<FName>>> DomainDimensionNames;
	bool bAllowParameters = false;
};

ENUM_CLASS_FLAGS(FOptimusDataDomainCustomization::DomainFlags);

class FOptimusShaderTextCustomization : 
	public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FOptimusShaderTextCustomization();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
	    TSharedRef<IPropertyHandle> InPropertyHandle,
	    FDetailWidgetRow& InHeaderRow,
	    IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
	    TSharedRef<IPropertyHandle> InPropertyHandle,
	    IDetailChildrenBuilder& InChildBuilder,
	    IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}

private:
	TSharedRef<FOptimusHLSLSyntaxHighlighter> SyntaxHighlighter;

	TSharedPtr<IPropertyHandle> DeclarationsProperty;
	TSharedPtr<IPropertyHandle> ShaderTextProperty;

	TSharedPtr<SExpandableArea> ExpandableArea;
	TSharedPtr<SScrollBar> HorizontalScrollbar;
	TSharedPtr<SScrollBar> VerticalScrollbar;

	TSharedPtr<SMultiLineEditableText> ShaderEditor;

	FText GetShaderText() const;
};


class FOptimusParameterBindingCustomization : public IPropertyTypeCustomization
{
public:
	struct FColumnSizeData
	{
		FColumnSizeData() : DataTypeColumnSize(0.5f) , DataDomainColumnSize(0.5f) {}
			
		float GetDataTypeColumnSize() const {return DataTypeColumnSize;}
		void OnDataTypeColumnResized(float InSize) {DataTypeColumnSize = InSize;}
		float GetDataDomainColumnSize() const {return DataDomainColumnSize;}
		void OnDataDomainColumnResized(float InSize) {DataDomainColumnSize = InSize;}
		
		float DataTypeColumnSize;
		float DataDomainColumnSize;	
	};
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static EVisibility IsAtomicCheckBoxVisible(const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects,  TSharedRef<IPropertyHandle> InPropertyHandle);
	static EVisibility IsSupportReadCheckBoxVisible(const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects,  TSharedRef<IPropertyHandle> InPropertyHandle);

	FOptimusParameterBindingCustomization();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
};


class FOptimusParameterBindingArrayBuilder
	: public FDetailArrayBuilder
	, public TSharedFromThis<FOptimusParameterBindingArrayBuilder>
{
public:
	static TSharedRef<FOptimusParameterBindingArrayBuilder> MakeInstance(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> InColumnSizeData,
		const bool bInAllowParameters);
	
	FOptimusParameterBindingArrayBuilder(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> InColumnSizeData,
		const bool bInAllowParameters);
	
	// FDetailArrayBuilder Interface
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

	// Used by FOptimusParameterBindingArrayCustomization
	void GenerateWrapperStructHeaderRowContent(FDetailWidgetRow& NodeRow, TSharedRef<SWidget> NameContent);

private:
	void OnGenerateEntry(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder) const;

	TSharedPtr<IPropertyHandleArray> ArrayProperty;

	TSharedPtr<FOptimusParameterBindingCustomization::FColumnSizeData> ColumnSizeData;

	bool bAllowParameters = false;
};


class FOptimusParameterBindingArrayCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FOptimusParameterBindingArrayCustomization>();
	}

	FOptimusParameterBindingArrayCustomization();
	
	// IPropertyTypeCustomization overrides
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	
private:
	TSharedPtr<FOptimusParameterBindingArrayBuilder> ArrayBuilder;
	TSharedRef<FOptimusParameterBindingCustomization::FColumnSizeData> ColumnSizeData;	
};


class FOptimusValueContainerStructCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FOptimusValueContainerStructCustomization>();
	}

	FOptimusValueContainerStructCustomization();
	// IPropertyTypeCustomization overrides
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
private:
	TSharedPtr<IPropertyHandle> InnerPropertyHandle;

	TSharedPtr<IPropertyTypeCustomization> PropertyBagCustomization;
};


class FOptimusValidatedNameCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	FOptimusValidatedNameCustomization();

	// IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override
	{ }
};


/** UI customization for UOptimusSource */
class FOptimusSourceDetailsCustomization :
	public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FOptimusSourceDetailsCustomization();

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	UOptimusSource* OptimusSource = nullptr;

 	TSharedRef<FOptimusHLSLSyntaxHighlighter> SyntaxHighlighter;
 	TSharedPtr<SOptimusShaderTextDocumentTextBox> SourceTextBox;

	FText GetText() const;
	void OnTextChanged(const FText& InValue);
};


/** UI customization for UOptimusComponentSourceBinding */
class FOptimusComponentSourceBindingDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FOptimusComponentSourceBindingDetailsCustomization();

	//~ Begin IDetailCustomization Interface.
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

	/** Handle combo box selection. */
	void ComponentSourceChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

private:
	/** SourceBinding being edited. */
	UOptimusComponentSourceBinding* OptimusSourceBinding = nullptr;
	/** Array of combo box entries. */
	TArray<TSharedPtr<FString>> ComponentSources;
};


/** UI customization for UOptimusResourceDescription */
class FOptimusResourceDescriptionDetailsCustomization :
	public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization Interface.
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface.

private:
	TArray<UOptimusComponentSourceBinding*> ComponentBindings; 
};



/** UI customization for FOptimusDeformerInstanceComponentBinding */
class FOptimusDeformerInstanceComponentBindingCustomization :
	public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	FOptimusDeformerInstanceComponentBindingCustomization() = default;

	// -- IPropertyTypeCustomization overrides
	void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override
	{ }

	void ComponentsReplaced(const TMap<UObject*, UObject*>& InReplacementMap);
	
private:
	TArray<FName> ComponentNames;
	using FComponentHandle = TSharedPtr<FSoftObjectPath>;
	TArray<FComponentHandle> ComponentHandles;
};
