// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditor.h"

#include "Components/StaticMeshComponent.h"
#include "IDetailsView.h"
#include "OptimusEditorTabSummoners.h"
#include "OptimusEditorClipboard.h"
#include "OptimusEditorCommands.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphNode.h"
#include "OptimusEditorGraphSchema.h"
#include "OptimusEditorModule.h"
#include "Widgets/SOptimusGraphTitleBar.h"

#include "OptimusActionStack.h"
#include "OptimusCoreNotify.h"
#include "OptimusDeformer.h"
#include "OptimusNode.h"
#include "OptimusNode_Comment.h"
#include "OptimusNodeGraph.h"
#include "OptimusFunctionNodeGraph.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationEditorPreviewActor.h"
#include "Engine/StaticMeshActor.h"
#include "FindInDeformer.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditor.h"
#include "IAssetFamily.h"
#include "IMessageLogListing.h"
#include "IOptimusAlternativeSelectedObjectProvider.h"
#include "IOptimusNodeGraphProvider.h"
#include "IOptimusShaderTextProvider.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "IPersonaViewport.h"
#include "ISkeletonEditorModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MessageLogModule.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusEditorGraphCommands.h"
#include "Misc/UObjectToken.h"
#include "Modules/ModuleManager.h"
#include "OptimusEditorMode.h"
#include "PersonaModule.h"
#include "PersonaTabs.h"
#include "SkeletalRenderPublic.h"
#include "ToolMenus.h"
#include "OptimusShaderText.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"
#include "OptimusEditorStyle.h"
#include "Toolkits/ToolkitManager.h"
#include "EditorViewportClient.h"
#include "GraphEditorActions.h"
#include "OptimusEditorGraphNode_Comment.h"
#include "OptimusEditorGraphSchemaActions.h"
#include "OptimusEditorHelpers.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "OptimusEditor"

const FName OptimusEditorAppName(TEXT("OptimusEditorApp"));


FOptimusEditor::FOptimusEditor()
{

}


FOptimusEditor::~FOptimusEditor()
{
	if (DeformerObject)
	{
		DeformerObject->GetCompileBeginDelegate().RemoveAll(this);
		DeformerObject->GetCompileEndDelegate().RemoveAll(this);
		DeformerObject->GetCompileMessageDelegate().RemoveAll(this);
		DeformerObject->GetNotifyDelegate().RemoveAll(this);
	}

	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}


void FOptimusEditor::Construct(
	const EToolkitMode::Type InMode,
	const TSharedPtr< class IToolkitHost >& InToolkitHost,
	UOptimusDeformer* InDeformerObject)
{
	DeformerObject = InDeformerObject;

	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	
	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(this, &FOptimusEditor::HandlePreviewSceneCreated);
	PersonaToolkitArgs.bPreviewMeshCanUseDifferentSkeleton = true;
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InDeformerObject, PersonaToolkitArgs);

	PersonaToolkit->GetPreviewScene()->SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode::Animation);

	// Make sure we get told when a new preview scene is set so that we can update the compute
	// graph component's scene component bindings.
	PersonaToolkit->GetPreviewScene()->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &FOptimusEditor::HandlePreviewMeshChanged));

	// Construct a new graph with a default name
	// TODO: Use a document manager like blueprints.
	// FIXME: The deformer asset shouldn't really be the owner.

	// We recreate the editor graph all the time based on the active UOptimusNodeGraph that the user is viewing,
	// so there is no need for editor graph be to Transactional to keep track of its state
	EditorGraph = NewObject<UOptimusEditorGraph>(
		DeformerObject, UOptimusEditorGraph::StaticClass(), NAME_None, 
		RF_Transient);
	EditorGraph->Schema = UOptimusEditorGraphSchema::StaticClass();
	
	// Used to query the size of slate node widgets
	EditorGraph->WeakEditor = SharedThis(this);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;

	// This call relies on virtual functions, so cannot be called from the constructor, hence
	// the dual-construction style.
	InitAssetEditor(InMode, InToolkitHost, OptimusEditorAppName, FTabManager::FLayout::NullLayout,
		bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InDeformerObject);

	BindCommands();

	// Set the default editor mode. This creates the editor layout and tabs.
	AddApplicationMode(FOptimusEditorMode::ModeId,MakeShareable(new FOptimusEditorMode(SharedThis(this))));

	// Create the compiler output log. This is used by our compilation message receiver
	// and the output widget so needs to exist before we set the mode.
	CreateMessageLog();

	CreateWidgets();

	SetCurrentMode(FOptimusEditorMode::ModeId);
	
	RegisterToolbar();

	
	// Find the update graph and set that as the startup graph.
	for (UOptimusNodeGraph* Graph: DeformerObject->GetGraphs())
	{
		if (Graph->GetGraphType() == EOptimusNodeGraphType::Update)
		{
			PreviousEditedNodeGraph = UpdateGraph = Graph;
			break;
		}
	}
	SetEditGraph(UpdateGraph);


	// Ensure that the action stack creates undoable transactions when actions are run.
	DeformerObject->GetActionStack()->SetTransactionScopeFunctions(
		[](UObject* InTransactObject, const FString& Title)->int32 {
			int32 TransactionId = INDEX_NONE;
			if (GEditor && GEditor->Trans)
			{
				InTransactObject->SetFlags(RF_Transactional);
				TransactionId = GEditor->BeginTransaction(TEXT(""), FText::FromString(Title), InTransactObject);
				InTransactObject->Modify();
			}
			return TransactionId;
		},
		[](int32 InTransactionId) {
			if (GEditor && GEditor->Trans && InTransactionId >= 0)
			{
				// For reasons I cannot fathom, EndTransaction returns the active index upon
				// entry, rather than the active index on exit. Which makes it one higher than
				// the index we got from BeginTransaction ¯\_(ツ)_/¯
				int32 TransactionId = GEditor->EndTransaction();
				check(InTransactionId == (TransactionId - 1));
			}
		}
		);

	// Make sure we get told when the deformer changes.
	DeformerObject->GetNotifyDelegate().AddSP(this, &FOptimusEditor::OnDeformerModified);

	DeformerObject->GetCompileBeginDelegate().AddSP(this, &FOptimusEditor::CompileBegin);
	DeformerObject->GetCompileEndDelegate().AddSP(this, &FOptimusEditor::CompileEnd);
	DeformerObject->GetCompileMessageDelegate().AddSP(this, &FOptimusEditor::OnCompileMessage);

	if (PersonaToolkit->GetPreviewMesh())
	{
		HandlePreviewMeshChanged(nullptr, PersonaToolkit->GetPreviewMesh());
	}
}


UOptimusDeformer* FOptimusEditor::GetDeformer() const
{
	return DeformerObject;
}


TSharedRef<IMessageLogListing> FOptimusEditor::GetMessageLog() const
{
	return CompilerResultsListing.ToSharedRef();
}


FText FOptimusEditor::GetGraphCollectionRootName() const
{
	return FText::FromName(DeformerObject->GetFName());
}


UOptimusActionStack* FOptimusEditor::GetActionStack() const
{
	return DeformerObject->GetActionStack();
}


void FOptimusEditor::InspectObject(UObject* InObject)
{
	// FIXME: This should take us to the correct graph as well.
	PropertyDetailsWidget->SetObject(InObject, /*bForceRefresh=*/true);

	// Bring the node details tab into the open.
	GetTabManager()->TryInvokeTab(FPersonaTabs::DetailsID);
}


void FOptimusEditor::InspectObjects(const TArray<UObject*>& InObjects)
{
	PropertyDetailsWidget->SetObjects(InObjects, /*bForceRefresh=*/true);

	// Bring the node details tab into the open.
	GetTabManager()->TryInvokeTab(FPersonaTabs::DetailsID);
}


FName FOptimusEditor::GetToolkitFName() const
{
	return FName("DeformerGraphEditor");
}


FText FOptimusEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Deformer Graph Editor");
}


FString FOptimusEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Deformer Graph Editor ").ToString();
}


FLinearColor FOptimusEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.2f, 0.2f, 0.6f, 0.5f);
}


void FOptimusEditor::OnClose()
{
	StoreCurrentViewLocation();
	
	IOptimusEditor::OnClose();
}


bool FOptimusEditor::SetEditGraph(UOptimusNodeGraph* InNodeGraph)
{
	if (ensure(InNodeGraph))
	{
		PreviousEditedNodeGraph = EditorGraph->NodeGraph;

		// Store the location/zoom for the graph we're leaving
		StoreCurrentViewLocation();

		GraphEditorWidget->ClearSelectionSet();

		EditorGraph->Reset();
		EditorGraph->InitFromNodeGraph(InNodeGraph);

		// If there was a stored view location in the node graph, use that, otherwise just frame
		// all the nodes.
		FVector2D Location;
		float Zoom;
		if (InNodeGraph->GetViewLocationAndZoom(Location, Zoom))
		{
			GraphEditorWidget->SetViewLocation(UE::Slate::FDeprecateVector2DParameter(Location), Zoom);
		}
		else
		{
			GraphEditorWidget->ZoomToFit(false);
		}

		RefreshEvent.Broadcast();
		return true;
	}
	else
	{
		return false;
	}
}


void FOptimusEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	// 
}


void FOptimusEditor::Compile()
{
	if (DeformerObject->Compile())
	{
		// Ensure we do a redraw.
		// FIXMNE: 
		// EditorViewportWidget->GetViewportClient()->Invalidate();
	}
}

void FOptimusEditor::ZoomToFitGraph()
{
	if(GraphEditorWidget)
	{
		// This will focus on the selected nodes, if any. If no nodes are selected, it will focus the whole graph.
		constexpr bool bOnlySelection = true;	
		GraphEditorWidget->ZoomToFit(bOnlySelection);
	}
}

bool FOptimusEditor::CanCompile() const
{
	return true;
}


void FOptimusEditor::CompileBegin(UOptimusDeformer* InDeformer)
{
	RemoveDataProviders();
	
	Diagnostics.Reset();
	OnDiagnosticsUpdated().Broadcast();

	CompilerResultsListing->ClearMessages();
}


void FOptimusEditor::CompileEnd(UOptimusDeformer* InDeformer)
{
	InstallDataProviders();
}


void FOptimusEditor::OnCompileMessage(FOptimusCompilerDiagnostic const& Diagnostic)
{
	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	if (Diagnostic.Level == EOptimusDiagnosticLevel::Warning)
	{
		Severity = EMessageSeverity::Warning;
	}
	else if (Diagnostic.Level == EOptimusDiagnosticLevel::Error)
	{
		Severity = EMessageSeverity::Error;
	}

	if (Diagnostic.Line != INDEX_NONE)
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(Severity,	Diagnostic.Message);

		if (UObject const* TokenObject = Diagnostic.Object.Get())
		{
			// Add activation which opens in text editor tab.
			// Could improve by putting cursor at correct location.
			Message->AddToken(FActionToken::Create(
				LOCTEXT("OpenInShaderTextEditor", "Open in shader text editor."),
				FText::FromString(TokenObject->GetName()),
				FOnActionTokenExecuted::CreateLambda([Event = OnSelectedNodesChanged(), TabManagerPtr = GetTabManager(), TokenObject]
					{
						TabManagerPtr->TryInvokeTab(FOptimusEditorShaderTextEditorTabSummoner::TabId);
						UObject* MutableTokenObject = const_cast<UObject*>(TokenObject);
						Event.Broadcast({ MutableTokenObject });
					})
			));
		}
		else if (!Diagnostic.AbsoluteFilePath.IsEmpty())
		{
			// Add activation which opens an external text editor.
			if (ISourceCodeAccessModule* SourceCodeAccessModule = FModuleManager::LoadModulePtr<ISourceCodeAccessModule>("SourceCodeAccess"))
			{
				Message->AddToken(FActionToken::Create(
					LOCTEXT("OpenInExternalEditor", "Open in external text editor."),
					FText::FromString(Diagnostic.AbsoluteFilePath),
					FOnActionTokenExecuted::CreateLambda([SourceCodeAccessModule, Path = Diagnostic.AbsoluteFilePath, Line = Diagnostic.Line, Column = Diagnostic.ColumnStart]
						{
							SourceCodeAccessModule->GetAccessor().OpenFileAtLine(Path, Line, Column);
						})
				));
			}
		}

		CompilerResultsListing->AddMessage(Message, false);
	}
	else
	{
		TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(Severity, Diagnostic.Message);

		if (UObject const* TokenObject = Diagnostic.Object.Get())
		{
			// Add a dummy activation since default behavior opens content browser.
			// Could improve this to select node in deformer graph view?
			static auto DummyActivation = [](const TSharedRef<class IMessageToken>&) {};
			Message->AddToken(FUObjectToken::Create(TokenObject)->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda(DummyActivation)));
		}
	
		CompilerResultsListing->AddMessage(Message, false);
	}

	// Add to diagnostic list which is referenced by syntax highlighters.
	Diagnostics.Add(Diagnostic);
	// Broadcast change to UI. For efficiency we could do this once at end of compilation.
	OnDiagnosticsUpdated().Broadcast();
}


void FOptimusEditor::InstallDataProviders()
{
	// SkeletalMeshComponent->SetMeshDeformer(MeshDeformer);
}


void FOptimusEditor::RemoveDataProviders()
{
	// SkeletalMeshComponent->SetMeshDeformer(nullptr);
}


void FOptimusEditor::SelectAllNodes()
{
	GraphEditorWidget->SelectAllNodes();
}


bool FOptimusEditor::CanSelectAllNodes() const
{
	return GraphEditorWidget.IsValid();
}


void FOptimusEditor::DeleteSelectedNodes()
{
	TArray<UOptimusNode*> NodesToDelete;
	for (UOptimusNode* ModelNode : GetSelectedModelNodes())
	{
		if (ModelNode->CanUserDeleteNode())
		{
			NodesToDelete.Add(ModelNode);
		}
	}

	if (NodesToDelete.IsEmpty())
	{
		return;
	}

	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(GraphEditorWidget->GetCurrentGraph());
	if (Graph) 
	{
		Graph->GetModelGraph()->RemoveNodes(NodesToDelete);
	}

	GraphEditorWidget->ClearSelectionSet();
}


bool FOptimusEditor::CanDeleteSelectedNodes() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	if (GraphEditorWidget->GetSelectedNodes().IsEmpty())
	{
		return false;
	}
	
	for (UObject* Object : GraphEditorWidget->GetSelectedNodes())
	{
		UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object);
		if (GraphNode && !GraphNode->CanUserDeleteNode())
		{
			return false;
		}
	}
	
	return true;
}


void FOptimusEditor::CopySelectedNodes() const
{
	FOptimusEditorClipboard& Clipboard = FOptimusEditorModule::Get().GetClipboard();

	Clipboard.SetClipboardFromNodes(GetSelectedModelNodes());
}


bool FOptimusEditor::CanCopyNodes() const
{
	if (GraphEditorWidget->GetSelectedNodes().IsEmpty())
	{
		return false;
	}
	
	for (UObject* Object : GraphEditorWidget->GetSelectedNodes())
	{
		UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object);
		if (GraphNode && !GraphNode->CanDuplicateNode())
		{
			return false;
		}
	}
	
	return true;
}


void FOptimusEditor::CutSelectedNodes() const
{
	FOptimusEditorClipboard& Clipboard = FOptimusEditorModule::Get().GetClipboard();

	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();
	Clipboard.SetClipboardFromNodes(ModelNodes);

	FOptimusActionScope ActionScope(*GetActionStack(), {});
	int32 NumCut = ModelGraph->RemoveNodesAndCount(ModelNodes);
	const FString& ActionTitle = NumCut == 1 ? TEXT("Cut Node") : FString::Printf(TEXT("Cut %d Nodes"), NumCut);
	ActionScope.SetTitle(ActionTitle);
}


bool FOptimusEditor::CanCutNodes() const
{
	return CanDeleteSelectedNodes() && CanCopyNodes();
}


void FOptimusEditor::PasteNodes() const
{
	const FOptimusEditorClipboard& Clipboard = FOptimusEditorModule::Get().GetClipboard();
	const UOptimusNodeGraph* TransientGraph = Clipboard.GetGraphFromClipboardContent(DeformerObject->GetPackage());
	if (!TransientGraph)
	{
		return;
	}

	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	ModelGraph->DuplicateNodes(
		TransientGraph->GetAllNodes(), GraphEditorWidget->GetPasteLocation2f(), TEXT("Paste"));
}


bool FOptimusEditor::CanPasteNodes()
{
	if (IsGraphReadOnly())
	{
		return false;	
	}
	
	return FOptimusEditorModule::Get().GetClipboard().HasValidClipboardContent();
}


void FOptimusEditor::DuplicateNodes() const
{
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();

	ModelGraph->DuplicateNodes(ModelNodes, GraphEditorWidget->GetPasteLocation2f());
}


bool FOptimusEditor::CanDuplicateNodes() const
{
	return CanCopyNodes() && !IsGraphReadOnly();
}


void FOptimusEditor::PackageNodes()
{
	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Package Nodes"));
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	TArray<UObject*> NewNodes;
	for (UOptimusNode* ModelNode: GetSelectedModelNodes())
	{
		UOptimusNode* NewNode = ModelGraph->ConvertCustomKernelToFunction(ModelNode);
		if (NewNode)
		{
			NewNodes.Add(NewNode);
		}
	}
	InspectObjects(NewNodes);
}


bool FOptimusEditor::CanPackageNodes() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();
	const UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();	
	for (UOptimusNode* ModelNode: ModelNodes)
	{
		if (!ModelGraph->IsCustomKernel(ModelNode))
		{
			return false;
		}
	}

	return !ModelNodes.IsEmpty();
}


void FOptimusEditor::UnpackageNodes()
{
	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Unpackage Nodes"));
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	TArray<UObject*> NewNodes;
	for (UOptimusNode* ModelNode: GetSelectedModelNodes())
	{
		UOptimusNode* NewNode = ModelGraph->ConvertFunctionToCustomKernel(ModelNode);
		if (NewNode)
		{
			NewNodes.Add(NewNode);
		}
	}
	InspectObjects(NewNodes);
}


bool FOptimusEditor::CanUnpackageNodes() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();	
	const UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();	
	for (UOptimusNode* ModelNode: ModelNodes)
	{
		if (!ModelGraph->IsKernelFunction(ModelNode))
		{
			return false;
		}
	}

	return !ModelNodes.IsEmpty();
}


void FOptimusEditor::CollapseNodesToFunction()
{
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	UOptimusNode* CollapseNode = ModelGraph->CollapseNodesToFunction(GetSelectedModelNodes());
	InspectObject(CollapseNode);
}


void FOptimusEditor::CollapseNodesToSubGraph()
{
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	UOptimusNode* CollapseNode = ModelGraph->CollapseNodesToSubGraph(GetSelectedModelNodes());
	InspectObject(CollapseNode);
}


bool FOptimusEditor::CanCollapseNodes() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	return CanCopyNodes();
}


void FOptimusEditor::ExpandCollapsedNode()
{
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	TArray<UObject*> NewNodes;
	for (UOptimusNode* ModelNode: GetSelectedModelNodes())
	{
		// Note: currently each expand is one undo step due to limitation on move node not able to move a
		// node that were removed as part of a previous expansion
		ModelGraph->ExpandCollapsedNodes(ModelNode);
	}
}


bool FOptimusEditor::CanExpandCollapsedNode() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();

	// Note: currently we only allow one node expansion at a time due to limitation on move node not able to move a
	// node that were removed as part of a previous expansion 
	if (ModelNodes.Num() != 1)
	{
		return false;
	}
	
	const UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();	
	for (UOptimusNode* ModelNode: ModelNodes)
	{
		if (!ModelGraph->IsFunctionReference(ModelNode) &&
			!ModelGraph->IsSubGraphReference(ModelNode))
		{
			return false;
		}
	}

	return !ModelNodes.IsEmpty();
}

void FOptimusEditor::ConvertToFunction()
{
	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Convert to Functions"));
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	for (UOptimusNode* ModelNode: GetSelectedModelNodes())
	{
		ModelGraph->ConvertToFunction(ModelNode);
	}
}

bool FOptimusEditor::CanConvertToFunction() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();	
	const UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();	
	for (UOptimusNode* ModelNode: ModelNodes)
	{
		if(!ModelGraph->IsSubGraphReference(ModelNode))
		{
			return false;
		}
	}

	return !ModelNodes.IsEmpty();
}

void FOptimusEditor::ConvertToSubGraph()
{
	FOptimusActionScope ActionScope(*GetActionStack(), TEXT("Convert to SubGraphs"));
	UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();
	for (UOptimusNode* ModelNode: GetSelectedModelNodes())
	{
		ModelGraph->ConvertToSubGraph(ModelNode);
	}
}

bool FOptimusEditor::CanConvertToSubGraph() const
{
	if (IsGraphReadOnly())
	{
		return false;
	}
	
	const TArray<UOptimusNode*> ModelNodes = GetSelectedModelNodes();	
	const UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();	
	for (UOptimusNode* ModelNode: ModelNodes)
	{
		if (!ModelGraph->IsFunctionReference(ModelNode))
		{
			return false;
		}
	}

	return !ModelNodes.IsEmpty();
}

void FOptimusEditor::OnCreateComment()
{
	// In case there aren't any nodes selected, create a comment node at current mouse position
	const FVector2f CurrentMouseGraphPosition = GraphEditorWidget->GetPasteLocation2f();
	FOptimusGraphSchemaAction_NewCommentNode NewCommentNodeAction;
	NewCommentNodeAction.PerformAction(EditorGraph, nullptr, CurrentMouseGraphPosition, true);
}

void FOptimusEditor::OnFindInDeformer()
{
	TabManager->TryInvokeTab(FOptimusEditorFindTabSummoner::TabId);
	FindResults->FocusForUse();
}

void FOptimusEditor::SubscribeToGraphNotifies(UOptimusNodeGraph* Graph, const FOnGraphNotified& OnGraphNotifiedDelegate)
{
	if (IsValid(Graph))
	{
		TSet<TWeakObjectPtr<UOptimusNodeGraph>>& Graphs = GraphSubscribers.FindOrAdd(OnGraphNotifiedDelegate.GetHandle());
		if (!Graphs.Contains(Graph))
		{
			Graphs.Add(Graph);
			Graph->GetNotifyDelegate().Add(OnGraphNotifiedDelegate);
		}
	}
}

void FOptimusEditor::UnsubscribeToGraphNotifies(FDelegateHandle DelegateHandle)
{
	if (TSet<TWeakObjectPtr<UOptimusNodeGraph>>* Graphs = GraphSubscribers.Find(DelegateHandle))
	{
		for (TWeakObjectPtr<UOptimusNodeGraph> Graph : *Graphs)
		{
			if (Graph.IsValid())
			{
				Graph->GetNotifyDelegate().Remove(DelegateHandle);
			}
		}
			
		GraphSubscribers.Remove(DelegateHandle);
	}
}

void FOptimusEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	TSet<UEdGraphNode*> SelectedNodes;

	for (UObject* Object : NewSelection)
	{
		UObject* ObjectToShow;
		if (UEdGraphNode* GraphNode = Cast<UEdGraphNode>(Object))
		{
			ObjectToShow = OptimusEditor::FindModelNodeFromGraphNode(GraphNode);
			SelectedNodes.Add(GraphNode);
		}
		else
		{
			ObjectToShow = Object;
		}

		if (const IOptimusAlternativeSelectedObjectProvider* AlternativeObjectProvider = Cast<IOptimusAlternativeSelectedObjectProvider>(ObjectToShow))
		{
			ObjectToShow = AlternativeObjectProvider->GetObjectToShowWhenSelected();
		}
		
		SelectedObjects.AddUnique(ObjectToShow);
	}

	// Make sure the graph knows too.
	EditorGraph->SetSelectedNodes(SelectedNodes);

	if (SelectedObjects.IsEmpty())
	{
		// If nothing was selected, default to the deformer object.
		SelectedObjects.Add(DeformerObject);
	}

	PropertyDetailsWidget->SetObjects(SelectedObjects, /*bForceRefresh=*/true);

	// Bring the node details tab into the open.
	GetTabManager()->TryInvokeTab(FPersonaTabs::DetailsID);

	if (SelectedObjects.ContainsByPredicate(
		[](const TWeakObjectPtr<UObject>& InObject)
		{
			if (IOptimusShaderTextProvider* ShaderTextProvider = Cast<IOptimusShaderTextProvider>(InObject.Get()))
			{
				return true;
			}
			return false;
		}))
	{
		GetTabManager()->TryInvokeTab(FOptimusEditorShaderTextEditorTabSummoner::TabId);
	}

	SelectedNodesChangedEvent.Broadcast(SelectedObjects);
}


void FOptimusEditor::OnNodeDoubleClicked(class UEdGraphNode* Node)
{
	UOptimusNode* ModelNode = OptimusEditor::FindModelNodeFromGraphNode(Node);
	if (IOptimusNodeGraphProvider* GraphProvider = Cast<IOptimusNodeGraphProvider>(ModelNode))
	{
		UOptimusNodeGraph* GraphToShow = GraphProvider->GetNodeGraphToShow();
		UOptimusNodeGraph* CurrentGraph = ModelNode->GetOwningGraph();

		if (GraphToShow)
		{
			if (CurrentGraph->GetCollectionRoot() == GraphToShow->GetCollectionRoot())
			{
				SetEditGraph(GraphToShow);
			}
			else if (UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(GraphToShow->GetCollectionRoot()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Deformer);

				TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Deformer);
				
				if (FoundAssetEditor.IsValid())
				{
					FoundAssetEditor->BringToolkitToFront();
					
					TSharedPtr<FOptimusEditor> OptimusEditorForExternalGraph = StaticCastSharedPtr<FOptimusEditor>(FoundAssetEditor);

					OptimusEditorForExternalGraph->SetEditGraph(GraphToShow);
				}	
			}
		}
	}
}


void FOptimusEditor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	if (UOptimusNode_Comment* CommentNode = Cast<UOptimusNode_Comment>(OptimusEditor::FindModelNodeFromGraphNode(NodeBeingChanged)))
	{
		FString NewComment = NewText.ToString();
		if (CommentNode->GetComment() != NewComment)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("OptimusEditorChangeCommentNodeComment_Transaction", "Change Comment Text"));
			CommentNode->SetComment(NewText.ToString());
		}
	}
}


bool FOptimusEditor::OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	if (UOptimusEditorGraphNode_Comment* CommentGraphNode = Cast<UOptimusEditorGraphNode_Comment>(NodeBeingChanged))
	{
		return true;
	}
	return false;
}


FReply FOptimusEditor::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph)
{
	return FReply::Handled();
}


void FOptimusEditor::RegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* ToolBar;
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);
	if (ToolMenus->IsMenuRegistered(MenuName))
	{
		ToolBar = ToolMenus->ExtendMenu(MenuName);
	}
	else
	{
		ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);
	}

	const FOptimusEditorCommands& Commands = FOptimusEditorCommands::Get();
	FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
	{
		FToolMenuSection& Section = ToolBar->AddSection("Compile", TAttribute<FText>(), InsertAfterAssetSection);
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			Commands.Compile,
			TAttribute<FText>(),
			TAttribute<FText>(this, &FOptimusEditor::GetCompileStatusTooltip),
			TAttribute<FSlateIcon>(this, &FOptimusEditor::GetCompileStatusIcon)));
	}

}


void FOptimusEditor::BindCommands()
{
	const FOptimusEditorCommands& Commands = FOptimusEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Compile,
		FExecuteAction::CreateSP(this, &FOptimusEditor::Compile),
		FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanCompile));

	ToolkitCommands->MapAction(
		Commands.ZoomToFitGraph,
		FExecuteAction::CreateSP(this, &FOptimusEditor::ZoomToFitGraph));
}

void FOptimusEditor::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{

	AAnimationEditorPreviewActor* Actor = InPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	Actor->SetFlags(RF_Transient);
	InPreviewScene->SetActor(Actor);

	SkeletalMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
	if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
	{
		SkeletalMeshComponent->SetMobility(EComponentMobility::Static);
	}
	SkeletalMeshComponent->bSelectable = false;
	SkeletalMeshComponent->MarkRenderStateDirty();

	SkeletalMeshComponent->SetMeshDeformer(DeformerObject);

	InPreviewScene->AddComponent(SkeletalMeshComponent, FTransform::Identity);
	InPreviewScene->SetPreviewMeshComponent(SkeletalMeshComponent);

	InPreviewScene->SetAllowMeshHitProxies(false);
	InPreviewScene->SetAdditionalMeshesSelectable(false);
}


void FOptimusEditor::HandlePreviewMeshChanged(
	USkeletalMesh* InOldPreviewMesh,
	USkeletalMesh* InNewPreviewMesh
	)
{
	InstallDataProviders();

	USkeleton* Skeleton = InNewPreviewMesh ? InNewPreviewMesh->GetSkeleton() : nullptr;

	if (Skeleton)
	{
		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);
	}
	else
	{
		EditableSkeleton.Reset();
	}
	GetPersonaToolkit()->GetPreviewScene()->SetEditableSkeleton(EditableSkeleton);
	
	// Vanilla preview scene relies on the skeleton tree widget to do this step, see SSkeletonTree::OnSelectionChanged
	// But since we don't have that widget, this has to be done manually
	if (InOldPreviewMesh != InNewPreviewMesh || InNewPreviewMesh == nullptr)
	{
		GetPersonaToolkit()->GetPreviewScene()->DeselectAll();
	}
}


void FOptimusEditor::CreateMessageLog()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	const FName LogName("LogOptimusCompiler");
	if (MessageLogModule.IsRegisteredLogListing(LogName))
	{
		CompilerResultsListing = MessageLogModule.GetLogListing(LogName);
	}
	else
	{
		FMessageLogInitializationOptions LogInitOptions;
		LogInitOptions.bShowInLogWindow = false;
		CompilerResultsListing = MessageLogModule.CreateLogListing(LogName, LogInitOptions);
	}

	CompilerResultsListing->OnMessageTokenClicked().AddSP(this, &FOptimusEditor::HandleMessageTokenClicked);
}


void FOptimusEditor::HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken)
{
	if (InMessageToken->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> ObjectToken = StaticCastSharedRef<FUObjectToken>(InMessageToken);

		UOptimusNode *Node = Cast<UOptimusNode>(ObjectToken->GetObject().Get());
		if (Node)
		{
			// Make sure we switch to the right graph too.
			if (Node->GetOwningGraph() != EditorGraph->NodeGraph)
			{
				SetEditGraph(Node->GetOwningGraph());
			}

			// Highlight the selection in the graph.
			UEdGraphNode* GraphNode = EditorGraph->FindGraphNodeFromModelNode(Node);
			EditorGraph->SetSelectedNodes({GraphNode});
			
			InspectObject(Node);
		}
	}
}


void FOptimusEditor::HandleDetailsCreated(
	const TSharedRef<IDetailsView>& InDetailsView
	)
{
	PropertyDetailsWidget = InDetailsView;
	PropertyDetailsWidget->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &FOptimusEditor::IsPropertyReadOnly));
}


void FOptimusEditor::HandleViewportCreated(
	const TSharedRef<IPersonaViewport>& InPersonaViewport
	)
{
	ViewportWidget = InPersonaViewport;
	ViewportWidget->GetViewportClient().EngineShowFlags.ShaderPrint = 1;
}



void FOptimusEditor::CreateWidgets()
{
	// -- Graph editor
	GraphEditorWidget = CreateGraphEditorWidget();
	GraphEditorWidget->SetViewLocation(FVector2f::ZeroVector, 1);

	FindResults =
		SNew(SFindInDeformer, SharedThis(this));
}


TSharedRef<SGraphEditor> FOptimusEditor::CreateGraphEditorWidget()
{
	GraphEditorCommands = MakeShareable(new FUICommandList);
	{
		// Editing commands
		GraphEditorCommands->MapAction(FGenericCommands::Get().SelectAll,
			FExecuteAction::CreateSP(this, &FOptimusEditor::SelectAllNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanSelectAllNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &FOptimusEditor::DeleteSelectedNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanDeleteSelectedNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &FOptimusEditor::CopySelectedNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanCopyNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &FOptimusEditor::CutSelectedNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanCutNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &FOptimusEditor::PasteNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanPasteNodes)
		);

		GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &FOptimusEditor::DuplicateNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanDuplicateNodes)
		);
#if 0
		// Packaging commands
		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().ConvertToKernelFunction,
			FExecuteAction::CreateSP(this, &FOptimusEditor::PackageNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanPackageNodes)
		);

		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().ConvertFromKernelFunction,
			FExecuteAction::CreateSP(this, &FOptimusEditor::UnpackageNodes),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanUnpackageNodes)
		);

		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().CollapseNodesToFunction,
			FExecuteAction::CreateSP(this, &FOptimusEditor::CollapseNodesToFunction),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanCollapseNodes)
		);
#endif
		
		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().CollapseNodesToSubGraph,
			FExecuteAction::CreateSP(this, &FOptimusEditor::CollapseNodesToSubGraph),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanCollapseNodes)
		);
		
		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().ExpandCollapsedNode,
			FExecuteAction::CreateSP(this, &FOptimusEditor::ExpandCollapsedNode),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanExpandCollapsedNode)
		);

		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().ConvertToFunction,
			FExecuteAction::CreateSP(this, &FOptimusEditor::ConvertToFunction),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanConvertToFunction)
		);
		
		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().ConvertToSubGraph,
			FExecuteAction::CreateSP(this, &FOptimusEditor::ConvertToSubGraph),
			FCanExecuteAction::CreateSP(this, &FOptimusEditor::CanConvertToSubGraph)
		);
		
		// Graph Editor Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().CreateComment,
			FExecuteAction::CreateSP(this, &FOptimusEditor::OnCreateComment)
		);

		GraphEditorCommands->MapAction(FOptimusEditorGraphCommands::Get().FindInDeformer,
			FExecuteAction::CreateSP(this, &FOptimusEditor::OnFindInDeformer));
#if 0

		// Alignment Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesTop,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignTop)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesMiddle,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignMiddle)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesBottom,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignBottom)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesLeft,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignLeft)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesCenter,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignCenter)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().AlignNodesRight,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnAlignRight)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().StraightenConnections,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnStraightenConnections)
		);

		// Distribution Commands
		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesHorizontally,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnDistributeNodesH)
		);

		GraphEditorCommands->MapAction(FGraphEditorCommands::Get().DistributeNodesVertically,
			FExecuteAction::CreateSP(this, &FMaterialEditor::OnDistributeNodesV)
		);
#endif
	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FOptimusEditor::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FOptimusEditor::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FOptimusEditor::OnNodeTitleCommitted);
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FOptimusEditor::OnVerifyNodeTextCommit);
	InEvents.OnSpawnNodeByShortcutAtLocation = 
		SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(
			this, &FOptimusEditor::OnSpawnGraphNodeByShortcut, 
			static_cast<UEdGraph*>(EditorGraph));

	// Create the title bar widget
	TSharedPtr<SWidget> TitleBarWidget = SNew(SOptimusGraphTitleBar)
		.OptimusEditor(SharedThis(this))
		.OnGraphCrumbClickedEvent_Lambda([this](UOptimusNodeGraph* InNodeGraph) { SetEditGraph(InNodeGraph);});

	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(this, &FOptimusEditor::IsGraphEditable)
		.DisplayAsReadOnly(this, &FOptimusEditor::IsGraphReadOnly)
		.TitleBar(TitleBarWidget)
		.Appearance(this, &FOptimusEditor::GetGraphAppearance)
		.GraphToEdit(EditorGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false);
}


FGraphAppearanceInfo FOptimusEditor::GetGraphAppearance() const
{
	FGraphAppearanceInfo Appearance;
	Appearance.CornerText = LOCTEXT("AppearanceCornerText_DeformerGraph", "DEFORMER GRAPH");
	return Appearance;
}

bool FOptimusEditor::IsGraphReadOnly() const
{
	return EditorGraph->GetModelGraph()->IsReadOnly();
}

bool FOptimusEditor::IsGraphEditable() const
{
	return !IsGraphReadOnly();
}

bool FOptimusEditor::IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const
{
	const TArray<TWeakObjectPtr<UObject>>& DetailsSelectedObjects = PropertyDetailsWidget->GetSelectedObjects();
	for (const TWeakObjectPtr<UObject>& SelectedObject : DetailsSelectedObjects)
	{
		if (!SelectedObject.IsValid())
		{
			continue;
		}

		// Everything a read-only graph owns should be read only
		if (UOptimusNodeGraph* OwnerGraph = SelectedObject->GetTypedOuter<UOptimusNodeGraph>())
		{
			return OwnerGraph->IsReadOnly();
		}
		
	}

	return false;	
}


TArray<UOptimusNode*> FOptimusEditor::GetSelectedModelNodes() const
{
	TArray<UOptimusNode*> Nodes = EditorGraph->GetSelectedModelNodes();

	return Nodes;
}

bool FOptimusEditor::GetBoundsForSelectedNodes(FSlateRect& Rect)
{
	constexpr float BoundPadding = 50.0f;
	return GraphEditorWidget->GetBoundsForSelectedNodes(Rect, BoundPadding);
}


void FOptimusEditor::StoreCurrentViewLocation()
{
	if (EditorGraph->NodeGraph)
	{
		FVector2f Location;
		float Zoom;
		
		GraphEditorWidget->GetViewLocation(Location, Zoom);
		EditorGraph->NodeGraph->SetViewLocationAndZoom(FDeprecateSlateVector2D(Location), Zoom);
	}
}


void FOptimusEditor::OnDeformerModified(
	EOptimusGlobalNotifyType InNotifyType, 
	UObject* InModifiedObject
	)
{
	switch (InNotifyType)
	{
	case EOptimusGlobalNotifyType::GraphAdded:
		if (UOptimusNodeGraph* Graph = Cast<UOptimusNodeGraph>(InModifiedObject))
		{
			if (Optimus::IsExecutionGraphType(Graph->GetGraphType()))
			{
				SetEditGraph(Graph);	
			}
		}
		RefreshEvent.Broadcast();
		break;

	case EOptimusGlobalNotifyType::GraphIndexChanged:
	case EOptimusGlobalNotifyType::GraphRenamed:
		RefreshEvent.Broadcast();
		break;

	case EOptimusGlobalNotifyType::ComponentBindingAdded:
	case EOptimusGlobalNotifyType::ResourceAdded:
	case EOptimusGlobalNotifyType::VariableAdded:
		InspectObject(InModifiedObject);
		RefreshEvent.Broadcast();
		break;

	case EOptimusGlobalNotifyType::ComponentBindingRemoved:
	case EOptimusGlobalNotifyType::ResourceRemoved:
	case EOptimusGlobalNotifyType::VariableRemoved:
		InspectObject(UpdateGraph);
		RefreshEvent.Broadcast();
		break;

	case EOptimusGlobalNotifyType::ComponentBindingRenamed:
	case EOptimusGlobalNotifyType::ComponentBindingIndexChanged:
	case EOptimusGlobalNotifyType::ComponentBindingSourceChanged:
	case EOptimusGlobalNotifyType::ResourceRenamed:
	case EOptimusGlobalNotifyType::ResourceIndexChanged:
	case EOptimusGlobalNotifyType::VariableRenamed:
	case EOptimusGlobalNotifyType::VariableIndexChanged:
	case EOptimusGlobalNotifyType::NodeTypeAdded: 
	case EOptimusGlobalNotifyType::NodeTypeRemoved: 
		RefreshEvent.Broadcast();
		break;

	case EOptimusGlobalNotifyType::ConstantValueChanged:
		break;
		
	case EOptimusGlobalNotifyType::DataTypeChanged:
		OnDataTypeChanged();
		break;

	case EOptimusGlobalNotifyType::GraphRemoved: 
	{
		// If the currently editing graph is being removed, then switch to the previous graph
		// or the update graph if no previous graph.
		UOptimusNodeGraph* RemovedGraph = Cast<UOptimusNodeGraph>(InModifiedObject);
		if (EditorGraph->NodeGraph == RemovedGraph)
		{
			if (ensure(PreviousEditedNodeGraph))
			{
				SetEditGraph(PreviousEditedNodeGraph);
			}
			PreviousEditedNodeGraph = UpdateGraph;
		}
		else if (PreviousEditedNodeGraph == RemovedGraph)
		{
			PreviousEditedNodeGraph = UpdateGraph;
		}
		RefreshEvent.Broadcast();
		break;
	}
	}
}


void FOptimusEditor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
	{
		FProperty* Property = PropertyChangedEvent.Property;

		for (int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); Index++ )
		{
			if (const UOptimusNode* ModelNode = Cast<const UOptimusNode>(PropertyChangedEvent.GetObjectBeingEdited(Index)))
			{
				UOptimusNodeGraph* ModelGraph = ModelNode->GetOwningGraph();
				if (UpdateGraph && UpdateGraph == ModelGraph)
				{
					const UOptimusNodePin* ModelPin = ModelNode->FindPinFromProperty(
						PropertyChangedEvent.MemberProperty,
						PropertyChangedEvent.Property);

					if (ModelPin)
					{
						if (UOptimusEditorGraphNode* GraphNode = EditorGraph->FindOptimusGraphNodeFromModelNode(ModelNode))
						{
							GraphNode->SynchronizeGraphPinValueWithModelPin(ModelPin);
						}
					}
				}
			}
		}
	}
}

void FOptimusEditor::OnDataTypeChanged()
{
	if (PropertyDetailsWidget)
	{
		PropertyDetailsWidget->ForceRefresh();
	}
}

FSlateIcon FOptimusEditor::GetCompileStatusIcon() const
{
	static const FName CompileStatusBackground("Toolbar.CompileStatus.Background");
	static const FName CompileStatusModified("Toolbar.CompileStatus.Overlay.Modified");
	static const FName CompileStatusHasError("Toolbar.CompileStatus.Overlay.Error");
	static const FName CompileStatusCompiled("Toolbar.CompileStatus.Overlay.Good");
	static const FName CompileStatusCompiledWithWarnings("Toolbar.CompileStatus.Overlay.Warning");

	const FName StyleSetName = FOptimusEditorStyle::Get().GetStyleSetName();

	switch (DeformerObject->GetStatus())
	{
	default:
	case EOptimusDeformerStatus::Modified:
		return FSlateIcon(StyleSetName, CompileStatusBackground, NAME_None, CompileStatusModified);
	case EOptimusDeformerStatus::HasErrors:
		return FSlateIcon(StyleSetName, CompileStatusBackground, NAME_None, CompileStatusHasError);
	case EOptimusDeformerStatus::Compiled:
		return FSlateIcon(StyleSetName, CompileStatusBackground, NAME_None, CompileStatusCompiled);
	case EOptimusDeformerStatus::CompiledWithWarnings:
		return FSlateIcon(StyleSetName, CompileStatusBackground, NAME_None, CompileStatusCompiledWithWarnings);
	}
}

FText FOptimusEditor::GetCompileStatusTooltip() const
{
	switch (DeformerObject->GetStatus())
	{
	default:
	case EOptimusDeformerStatus::Modified:
		return LOCTEXT("GraphStatusTooltip_Modified", "Deformer graph has been modified and needs to be recompiled in order to update the deformation result.");
	case EOptimusDeformerStatus::HasErrors:
		return LOCTEXT("GraphStatusTooltip_HasErrors", "Deformer graph has errors which need to be fixed.");
	case EOptimusDeformerStatus::Compiled:
		return LOCTEXT("GraphStatusTooltip_Compiled", "Deformer graph is compiled and corresponds to the deformation result.");
	case EOptimusDeformerStatus::CompiledWithWarnings:
		return LOCTEXT("GraphStatusTooltip_CompiledWithWarnings", "Deformer graph is compiled with warnings, these warnings will likely not affect the deformation result but should be fixed regardless.");
	}
}

#undef LOCTEXT_NAMESPACE
