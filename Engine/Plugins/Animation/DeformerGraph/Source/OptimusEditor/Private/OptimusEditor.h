// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusEditor.h"
#include "OptimusCoreNotify.h"
#include "OptimusNodeGraph.h"

#include "Misc/NotifyHook.h"


class IEditableSkeleton;
class FUICommandList;
class IMessageLogListing;
class IOptimusPathResolver;
class IPersonaPreviewScene;
class IPersonaToolkit;
class IPersonaViewport;
class SFindInDeformer;
class SGraphEditor;
class SOptimusEditorViewport;
class SOptimusGraphTitleBar;
class SOptimusNodePalette;
class UDebugSkelMeshComponent;
class UEdGraph;
class UOptimusActionStack;
class UOptimusDeformer;
class UOptimusEditorGraph;
class UOptimusMeshDeformer;
class UOptimusNode;
class UOptimusNodeGraph;
class USkeletalMesh;
enum class EOptimusGlobalNotifyType;
struct FGraphAppearanceInfo;
struct FOptimusCompilerDiagnostic;
enum class EOptimusGraphNotifyType;
class FSlateRect;


class FOptimusEditor :
	public IOptimusEditor,
	public FGCObject,
	public FNotifyHook
{
public:
	FOptimusEditor();
	~FOptimusEditor() override;

	void Construct(
		const EToolkitMode::Type InMode,
		const TSharedPtr< class IToolkitHost >& InToolkitHost,
		UOptimusDeformer* InDeformerObject
	);

	/// @brief Returns the graph that this editor operates on.
	/// @return The graph that this editor operates on.
	UOptimusEditorGraph* GetGraph() const
	{
		return EditorGraph;
	}

	template<typename Interface>
	Interface* GetDeformerInterface()
	{
		return Cast<Interface>(DeformerObject);
	}
	UOptimusDeformer* GetDeformer() const;

	TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override
	{
		return PersonaToolkit.ToSharedRef();
	}

	TSharedRef<IMessageLogListing> GetMessageLog() const;
	
	FText GetGraphCollectionRootName() const;

	UOptimusActionStack* GetActionStack() const;

	/// @brief Set object to view in the details panel.
	/// @param InObject  The object to view and edit in the details panel.
	void InspectObject(UObject* InObject);

	/// @brief Set a group of object to view in the details panel.
	/// @param InObject  The objects to view and edit in the details panel.
	void InspectObjects(const TArray<UObject *> &InObjects);

	// IToolkit overrides
	FName GetToolkitFName() const override;				
	FText GetBaseToolkitName() const override;			
	FString GetWorldCentricTabPrefix() const override;	
	FLinearColor GetWorldCentricTabColorScale() const override;

	//~ Begin FAssetEditorToolkit Interface.
	void OnClose() override;
	//~ End FAssetEditorToolkit Interface.
	
	// --
	bool SetEditGraph(UOptimusNodeGraph *InNodeGraph);

	DECLARE_EVENT( FOptimusEditor, FOnRefreshEvent );
	FOnRefreshEvent& OnRefresh() { return RefreshEvent; }

	// FGCObject overrides
	void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FOptimusEditor");
	}

private:
	// ----------------------------------------------------------------------------------------
	// Editor commands
	void Compile();
	void ZoomToFitGraph();

	bool CanCompile() const;

	void CompileBegin(UOptimusDeformer* InDeformer);
	void CompileEnd(UOptimusDeformer* InDeformer);

	// Called for every compilation result. There may be multiple calls for each Compile.
	void OnCompileMessage(FOptimusCompilerDiagnostic const& Diagnostic);
	
	void InstallDataProviders();
	void RemoveDataProviders();

	// ----------------------------------------------------------------------------------------
	// Graph commands

	/// Select all nodes in the visible graph
	void SelectAllNodes();

	/// Returns \c true if all the nodes can be selected.
	bool CanSelectAllNodes() const;

	/// Delete all selected nodes in the graph
	void DeleteSelectedNodes();

	/// Returns \c true if all the nodes can be selected.
	bool CanDeleteSelectedNodes() const;

	void CopySelectedNodes() const;
	bool CanCopyNodes() const;

	void CutSelectedNodes() const;
	bool CanCutNodes() const;
	
	void PasteNodes() const;
	bool CanPasteNodes();
	
	void DuplicateNodes() const;
	bool CanDuplicateNodes() const;

	void PackageNodes();
	bool CanPackageNodes() const;

	void UnpackageNodes();
	bool CanUnpackageNodes() const;

	void CollapseNodesToFunction();
	void CollapseNodesToSubGraph();
	bool CanCollapseNodes() const;

	void ExpandCollapsedNode();
	bool CanExpandCollapsedNode() const;

	void ConvertToFunction();
	bool CanConvertToFunction() const;
	
	void ConvertToSubGraph();
	bool CanConvertToSubGraph() const;

	void OnCreateComment();

	void OnFindInDeformer();
	
	// ----------------------------------------------------------------------------------------
	// Graph event listeners
	void OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);
	void OnNodeDoubleClicked(class UEdGraphNode* Node);
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);
	bool OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage);
	FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph);

public:
	// ----------------------------------------------------------------------------------------
	// Utility for other widgets to subscribe to Graph Notifies 
	DECLARE_DELEGATE_ThreeParams(FOnGraphNotified,  EOptimusGraphNotifyType /* type */, UOptimusNodeGraph*/*graph */, UObject* /* subject */);
	void SubscribeToGraphNotifies(UOptimusNodeGraph* Graph, const FOnGraphNotified& OnGraphNotifiedDelegate);
	void UnsubscribeToGraphNotifies(FDelegateHandle DelegateHandle);

	// ----------------------------------------------------------------------------------------
	// Broadcast Graph Event
	DECLARE_EVENT_OneParam(FOptimusEditor, FOnSelectedNodesChanged, const TArray<TWeakObjectPtr<UObject>>&);
	FOnSelectedNodesChanged& OnSelectedNodesChanged() { return SelectedNodesChangedEvent; }

	// Broadcast Diagnostic Event
	DECLARE_EVENT(FOptimusEditor, FOnDiagnosticsUpdated);
	FOnDiagnosticsUpdated& OnDiagnosticsUpdated() { return DiagnosticsUpdatedEvent; }
	
	virtual void AddCompilationDiagnostic(const FOptimusCompilerDiagnostic& InDiagnostic) {}
	virtual const TArray<FOptimusCompilerDiagnostic>& GetCompilationDiagnostics() const { return Diagnostics; }

private:
	// Toolbar and command helpers
	void RegisterToolbar();

	void BindCommands();

public:
	// Handlers for created tabs
	void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPreviewScene);
	void HandlePreviewMeshChanged(USkeletalMesh* InOldPreviewMesh, USkeletalMesh* InNewPreviewMesh);
	void HandleDetailsCreated(const TSharedRef<IDetailsView>& InDetailsView);
	void HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport);
	
	// KILL ME
	TSharedPtr<SGraphEditor> GetGraphEditorWidget() const { return GraphEditorWidget; }

	// @TODO: Remove below if above gets removed
	TSharedPtr<SFindInDeformer> GetFindResultsWidget() const { return FindResults; }

	bool GetBoundsForSelectedNodes(FSlateRect& Rect);
	
private:
	void CreateMessageLog();
	void HandleMessageTokenClicked(const TSharedRef<class IMessageToken>& InMessageToken);
	void CreateWidgets();
	TSharedRef<SGraphEditor> CreateGraphEditorWidget();
	FGraphAppearanceInfo GetGraphAppearance() const;
	bool IsGraphReadOnly() const;
	bool IsGraphEditable() const;
	bool IsPropertyReadOnly(const FPropertyAndParent& InPropertyAndParent) const;

	TArray<UOptimusNode*> GetSelectedModelNodes() const;

	void StoreCurrentViewLocation();

	void OnDeformerModified(
		EOptimusGlobalNotifyType InNotifyType, 
		UObject *InModifiedObject
		);

	// Called when the inspector has changed a value.
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	void OnDataTypeChanged();


	FSlateIcon GetCompileStatusIcon() const;
	FText GetCompileStatusTooltip() const;
	
private:
	// Persona toolkit for the skelmesh preview
	TSharedPtr<IPersonaToolkit> PersonaToolkit;
	
	// -- Widgets
	TSharedPtr<IPersonaViewport> ViewportWidget;
	TSharedPtr<SFindInDeformer> FindResults;
	TSharedPtr<SGraphEditor> GraphEditorWidget;
	TSharedPtr<IDetailsView> PropertyDetailsWidget;
	TSharedPtr<IDetailsView> PreviewDetailsWidget;
	TSharedPtr<SWidget> CompilerResultsWidget;
	TSharedPtr<IMessageLogListing> CompilerResultsListing;

	UOptimusDeformer* DeformerObject = nullptr;
	UOptimusEditorGraph* EditorGraph = nullptr;
	UOptimusNodeGraph* PreviousEditedNodeGraph = nullptr;
	UOptimusNodeGraph* UpdateGraph = nullptr;
	TSharedPtr<FUICommandList> GraphEditorCommands;

	// Compute Graph Component and data providers.
	UDebugSkelMeshComponent* SkeletalMeshComponent = nullptr;
	UOptimusMeshDeformer* MeshDeformer = nullptr;

	// An editability wrapper around USkeleton. Used by the Persona viewport for picking
	// and manipulation.
	TSharedPtr<IEditableSkeleton> EditableSkeleton;
	
	// Bone selection data
	struct FCapsuleInfo
	{
		FName Name;
		int32 Index;
	};

	TArray<FCapsuleInfo> CapsuleInfos;
	
	TArray<FOptimusCompilerDiagnostic> Diagnostics;

	FOnRefreshEvent RefreshEvent;
	FOnSelectedNodesChanged SelectedNodesChangedEvent;
	FOnDiagnosticsUpdated DiagnosticsUpdatedEvent;
	
	TMap<FDelegateHandle, TSet<TWeakObjectPtr<UOptimusNodeGraph>>> GraphSubscribers;
};
