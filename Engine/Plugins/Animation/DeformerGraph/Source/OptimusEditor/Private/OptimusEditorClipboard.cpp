// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorClipboard.h"

#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodeSubGraph.h"
#include "OptimusNodePair.h"
#include "OptimusNodeLink.h"
#include "OptimusNodePin.h"

#include "Editor.h"
#include "IOptimusGeneratedClassDefiner.h"
#include "OptimusHelpers.h"
#include "Exporters/Exporter.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UnrealExporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/EngineVersion.h"
#include "Misc/StringOutputDevice.h"
#include "OptimusCore/Private/Nodes/OptimusNode_SubGraphReference.h"

static TArray<const IOptimusGeneratedClassDefiner*> CollectClassDefiners(const TArray<UOptimusNode*>& InNodes , const TArray<UOptimusNodeSubGraph*> InSubGraphs)
{
	TArray<const IOptimusGeneratedClassDefiner*> ClassDefiners;
	
	TArray<UOptimusNode*> NodesToScanForClassDefiner = InNodes;

	TQueue<UOptimusNodeGraph*> SubGraphQueue;
	for (UOptimusNodeSubGraph* SubGraph : InSubGraphs)
	{
		SubGraphQueue.Enqueue(SubGraph);
	}

	UOptimusNodeGraph* WorkingSubGraph;
	while (SubGraphQueue.Dequeue(WorkingSubGraph))
	{
		NodesToScanForClassDefiner.Append(WorkingSubGraph->GetAllNodes());
		for (UOptimusNodeGraph* const SubGraph : WorkingSubGraph->GetGraphs())
		{
			SubGraphQueue.Enqueue(SubGraph);
		}
	}
	
	for (UOptimusNode* Node: NodesToScanForClassDefiner)
	{
		// Export any generated classes first. If the node implements the IOptimusGeneratedClassDefiner interface,
		// it means that it uses a generated class as a base. We need to ensure that we have all the relevant information
		// to construct the generated class on the receiving side.
		if (const IOptimusGeneratedClassDefiner* ClassDefiner = Cast<IOptimusGeneratedClassDefiner>(Node))
		{
			ClassDefiners.Add(ClassDefiner);
		}
	}

	return ClassDefiners;
}

void FOptimusEditorClipboard::SetClipboardFromNodes(const TArray<UOptimusNode*>& InNodes)
{
	if (InNodes.IsEmpty())
	{
		return;
	}

	TArray<UOptimusNode*> NodesToConsider;
	TArray<UOptimusNodePair*> NodePairsToConsider;
	TArray<UOptimusNodeSubGraph*> SubGraphsToConsider;
	
	UOptimusNodeGraph* Graph = InNodes[0]->GetOwningGraph();

	Graph->GatherRelatedObjects(InNodes, NodesToConsider, NodePairsToConsider, SubGraphsToConsider);

	TArray<const IOptimusGeneratedClassDefiner*> ClassDefiners = CollectClassDefiners(NodesToConsider, SubGraphsToConsider);
	
	// Export the clipboard to text.
	const FExportObjectInnerContext Context;
	FStringOutputDevice Archive;

	// Clear all stray export marks to ensure all the objects get properly exported. 
	UnMarkAllObjects(OBJECTMARK_TagExp);

	// Export any generated classes first. If the node implements the IOptimusGeneratedClassDefiner interface,
	// it means that it uses a generated class as a base. We need to ensure that we have all the relevant information
	// to construct the generated class on the receiving side.
	for (const IOptimusGeneratedClassDefiner* ClassDefiner : ClassDefiners)
	{
		const UOptimusNode* Node = Cast<UOptimusNode>(ClassDefiner);
		if (ensure(Node))
		{
			Archive.Logf(TEXT("GeneratedClass Class=%s GeneratorPath=%s %s\n"),
						*Node->GetClass()->GetPathName(), *ClassDefiner->GetAssetPathForClassDefiner().ToString(),
						*ClassDefiner->GetClassCreationString());	
		}
	}
	
	
	// Export the sub graphs.
	for (UOptimusNodeSubGraph* SubGraph : SubGraphsToConsider)
	{
		UExporter::ExportToOutputDevice(
			&Context,
			SubGraph,
			nullptr,
			Archive,
			TEXT("copy"),
			0,
			PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited,
			false,
			Graph);	
	}

	// Export the nodes.
	TSet<UOptimusNode*> NodesToCopy;
	NodesToCopy.Reserve(NodesToConsider.Num());
	for (UOptimusNode* Node: NodesToConsider)
	{
		if (!ensure(Node->GetOwningGraph() == Graph))
		{
			continue;
		}
		
		UExporter::ExportToOutputDevice(
			&Context,
			Node,
			nullptr,
			Archive,
			TEXT("copy"),
			0,
			PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited,
			false,
			Graph);
		
		NodesToCopy.Add(Node);
	}

	for (UOptimusNodePair* NodePair : NodePairsToConsider)
	{
		UExporter::ExportToOutputDevice(
			&Context,
			NodePair,
			nullptr,
			Archive,
			TEXT("copy"),
			0,
			PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited,
			false,
			Graph);	
	}

	// Export all internal links as well.
	for (UOptimusNodeLink* Link: Graph->GetAllLinks())
	{
		if (ensure(Link->GetNodeOutputPin() != nullptr))
		{
			const UOptimusNode *OutputNode = Link->GetNodeOutputPin()->GetOwningNode();
			const UOptimusNode *InputNode = Link->GetNodeInputPin()->GetOwningNode();

			// Only include the link if the both end points are connected to nodes in the list of nodes
			// we're interested in copying.
			if (NodesToCopy.Contains(OutputNode) && NodesToCopy.Contains(InputNode))
			{
				UExporter::ExportToOutputDevice(
					&Context,
					Link,
					nullptr,
					Archive,
					TEXT("copy"),
					0,
					PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited,
					false,
					Graph);
			}
		}
	}
	
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}


UOptimusNodeGraph* FOptimusEditorClipboard::GetGraphFromClipboardContent(
	UPackage* InTargetPackage
	)
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	// Try to create optimus clipboard content from that. We're not using a custom version of FCustomizableTextObjectFactory
	// since the original does not handle sub-objects at all.
	if (CanCreateObjectsFromText(*ClipboardText))
	{
		// Avoid multiple paste pasting into the same transient graph causing name collision
		const FName UniqueClipboardGraphName = MakeUniqueObjectName(GetTransientPackage(), UOptimusNodeGraph::StaticClass(), FName("ClipboardGraph"));
		UOptimusNodeGraph* Graph = NewObject<UOptimusNodeGraph>(GetTransientPackage(), UniqueClipboardGraphName, RF_Transient);
		
		ProcessObjectBuffer(InTargetPackage, Graph, *ClipboardText);
		return Graph;
	}

	return nullptr;
}


bool FOptimusEditorClipboard::HasValidClipboardContent()
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	return CanCreateObjectsFromText(*ClipboardText);
}


bool FOptimusEditorClipboard::CanCreateObjectsFromText(
	const TCHAR* InBuffer
	)
{
	FParse::Next( &InBuffer );
	
	// Find at least one object we can create.
	FString StrLine;
	while (FParse::Line(&InBuffer,StrLine))
	{
		const TCHAR* Str = *StrLine;
		if (FParse::Command(&Str, TEXT("Begin")) && FParse::Command(&Str,TEXT("Object")))
		{
			UClass* ObjClass = nullptr;
			if (ParseObject<UClass>(Str, TEXT("Class="), ObjClass, nullptr))
			{
				if (CanCreateClass(ObjClass))
				{
					return true;
				}
			}
		}
	}
	return false;
}


bool FOptimusEditorClipboard::ProcessObjectBuffer(
	UPackage* InPackage, 
	UObject* InRootOuter,
	const TCHAR* InBuffer
	)
{
	FParse::Next(&InBuffer);
	
	FObjectInstancingGraph InstancingGraph;
	TMap<TPair<FName /*ObjectName*/, UObject* /*Outer*/>, UObject*> ObjectMap;
	TArray<UObject*> CurrentObjectStack;
	TArray<UObject*> CreatedObjects;
	TMap<FName, UClass*> GeneratedClasses;
	CurrentObjectStack.Push(InRootOuter);

	auto ParseObjectCommand = [](const TCHAR** Str, const TCHAR* Match) -> bool
	{
		const TCHAR* Tmp = *Str;
		if (FParse::Command(Str, Match) && FParse::Command(Str, TEXT("Object")))
		{
			return true;
		}
		*Str = Tmp;
		return false;
	};
	auto ParseCommand = [](const TCHAR** Str, const TCHAR* Match) -> bool
	{
		const TCHAR* Tmp = *Str;
		if (FParse::Command(Str, Match))
		{
			return true;
		}
		*Str = Tmp;
		return false;
	};
	
	// 
	FString LineStr;
	while (FParse::Line(&InBuffer, LineStr))
	{
		const TCHAR* LineBuffer = *LineStr;

		if (ParseObjectCommand(&LineBuffer, TEXT("Begin")))
		{
			// Get the name, it should always be there.
			FName ObjectName;
			if (!FParse::Value(LineBuffer, TEXT("Name="), ObjectName))
			{
				return false;
			}
			
			UClass *ObjectClass = nullptr;
			bool bInvalidObject = false;
			if (!ParseObject<UClass>(LineBuffer, TEXT("Class="), ObjectClass, nullptr, &bInvalidObject) &&
				bInvalidObject)
			{
				// Check if we have it in the list of generated classes. We already know the tag is there.
				FName ClassName;
				(void)FParse::Value(LineBuffer, TEXT("Class="), ClassName);
				UClass **ObjectClassResult = GeneratedClasses.Find(ClassName);
				if (!ObjectClassResult)
				{
					// Class= found but no matching class object found.
					return false;
				}

				ObjectClass = *ObjectClassResult;
			}

			// If a class is given, create the object now and add it to the outer stack.
			if (ObjectClass)
			{
				UObject* CreatedObject = NewObject<UObject>(CurrentObjectStack.Last(), ObjectClass, ObjectName, RF_NoFlags, nullptr, true, &InstancingGraph);

				ObjectMap.Add(MakeTuple(ObjectName, CurrentObjectStack.Last()), CreatedObject);
				CreatedObjects.Add(CreatedObject);

				// Mark the newly created object as the current object on the stack, it will receive any properties found
				// at this level and be the outer for any newly created objects as well.
				CurrentObjectStack.Push(CreatedObject);
			}
			// Else it's a definition containing the properties, set the current object to the named object.
			else if (UObject **FoundObject = ObjectMap.Find(MakeTuple(ObjectName, CurrentObjectStack.Last())))
			{
				CurrentObjectStack.Push(*FoundObject);
			}
			else
			{
				// Something's gone wrong.
				return false;
			}
		}
		else if (ParseObjectCommand(&LineBuffer, TEXT("End")))
		{
			CurrentObjectStack.Pop();
		}
		else if (ParseCommand(&LineBuffer, TEXT("GeneratedClass")))
		{
			FName ClassName;
			if (!FParse::Value(LineBuffer, TEXT("Class="), ClassName))
			{
				return false;
			}

			FString GeneratorPath;
			if (!FParse::Value(LineBuffer, TEXT("GeneratorPath="), GeneratorPath))
			{
				return false;
			}

			// Find the class whose object implements the definer that we need to create the generated class. 
			FTopLevelAssetPath GeneratorAssetPath(GeneratorPath);
			UClass* GeneratorClass = FindObject<UClass>(GeneratorAssetPath);
			if (!GeneratorClass)
			{
				return false;
			}
			IOptimusGeneratedClassDefiner* ClassDefiner = Cast<IOptimusGeneratedClassDefiner>(GeneratorClass->GetDefaultObject(true));
			if (!ClassDefiner)
			{
				return false;
			}

			UClass* GeneratedClass = ClassDefiner->GetClassFromCreationString(InPackage, LineBuffer);
			if (!GeneratedClass)
			{
				return false;
			}
			GeneratedClasses.Add(ClassName, GeneratedClass);
		}
		// Custom properties?
		else if (!CurrentObjectStack.IsEmpty())
		{
			UObject* PropertyObject = CurrentObjectStack.Last();
			FImportObjectParams ImportParams;

			ImportParams.DestData = reinterpret_cast<uint8*>(PropertyObject);
			ImportParams.SourceText = LineBuffer;
			ImportParams.ObjectStruct = PropertyObject->GetClass();
			ImportParams.SubobjectRoot = PropertyObject;
			ImportParams.SubobjectOuter = PropertyObject;
			ImportParams.Warn = GWarn;
			ImportParams.Depth = 0;
			ImportParams.LineNumber = 0;
			ImportParams.InInstanceGraph = &InstancingGraph;
			ImportParams.bShouldCallEditChange = false;
			
			ImportObjectProperties(ImportParams);
		}
	}

	// Process the created objects in the order of creation because 
	// objects created later depend on objects created earlier.
	for (int32 Index = 0; Index < CreatedObjects.Num(); Index++)
	{
		if (!ProcessPostCreateObject(InRootOuter, CreatedObjects[Index]))
		{
			return false;
		}
	}

	return true;
}


bool FOptimusEditorClipboard::CanCreateClass(const UClass* InClass)
{
	return InClass->IsChildOf(UOptimusNode::StaticClass()) ||
		   InClass == UOptimusNodePin::StaticClass() ||
		   InClass == UOptimusNodeLink::StaticClass() ||
		   InClass == UOptimusNodePair::StaticClass() ||
		   InClass == UOptimusNodeSubGraph::StaticClass();
}


bool FOptimusEditorClipboard::ProcessPostCreateObject(UObject* InRootOuter, UObject* InNewObject)
{
	if (InNewObject->GetOuter() == InRootOuter && InRootOuter->IsA<UOptimusNodeGraph>())
	{
		UOptimusNodeGraph* Graph = Cast<UOptimusNodeGraph>(InRootOuter);
		if (InNewObject->IsA<UOptimusNode>())
		{
			UOptimusNode* Node = Cast<UOptimusNode>(InNewObject);

			Node->PostDeserializeNode();
			
			return Graph->AddNodeDirect(Node);
		}
		else if (InNewObject->IsA<UOptimusNodeLink>())
		{
			const UOptimusNodeLink* Link = Cast<UOptimusNodeLink>(InNewObject);

			return Graph->AddLinkDirect(Link->GetNodeOutputPin(), Link->GetNodeInputPin());
		}
		else if (InNewObject->IsA<UOptimusNodePair>())
		{
			const UOptimusNodePair* NodePair = Cast<UOptimusNodePair>(InNewObject);

			return Graph->AddNodePairDirect(NodePair->GetFirst(), NodePair->GetSecond());
		}
		else if(InNewObject->IsA<UOptimusNodeSubGraph>())
		{
			UOptimusNodeSubGraph* SubGraph = Cast<UOptimusNodeSubGraph>(InNewObject);

			// Objects created by the subgraph needs to init their transient data as well
			TQueue<UOptimusNodeGraph*> SubGraphQueue;
			SubGraphQueue.Enqueue(SubGraph);
			UOptimusNodeGraph* WorkingSubGraph;
			while(SubGraphQueue.Dequeue(WorkingSubGraph))
			{
				for (UOptimusNode* Node : WorkingSubGraph->GetAllNodes())
				{
					Node->PostDeserializeNode();
				}

				for (UOptimusNodeGraph* SubGraphToAdd : WorkingSubGraph->GetGraphs())
				{
					SubGraphQueue.Enqueue(SubGraphToAdd);
				}
			}

			
			return Graph->AddGraphDirect(SubGraph, INDEX_NONE);
		}
	}

	return true;
}
