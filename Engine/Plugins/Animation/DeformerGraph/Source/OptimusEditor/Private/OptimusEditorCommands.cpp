// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorCommands.h"

#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "OptimusEditorCommands"


FOptimusEditorCommands::FOptimusEditorCommands() 
	: TCommands<FOptimusEditorCommands>(
		"OptimusEditor", // Context name for fast lookup
		NSLOCTEXT("Contexts", "DeformerGraphEditor", "Deformer Graph Editor"), // Localized context name for displaying
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{

}


void FOptimusEditorCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compile the current deformer graph into a compute kernel graph.", EUserInterfaceActionType::Button, FInputChord(EKeys::F7))
	UI_COMMAND(ZoomToFitGraph, "ZoomToFitGraph", "Fit the graph in the editor viewport.", EUserInterfaceActionType::None, FInputChord(EKeys::F));
}


#undef LOCTEXT_NAMESPACE
