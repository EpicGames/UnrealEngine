// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"


/// Optimus editor command set.
class FOptimusEditorCommands :
	public TCommands<FOptimusEditorCommands>
{
public:
	FOptimusEditorCommands();

	// TCommands<> overrides
	void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> ZoomToFitGraph;
};