// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"


class UOptimusNodeGraph;

// A collection of delegates used by the Optimus editor.

/// A simple delegate that fires when the Optimus when a specific graph should be opened.
DECLARE_DELEGATE_OneParam(FOptimusOpenGraphEvent, UOptimusNodeGraph*);
