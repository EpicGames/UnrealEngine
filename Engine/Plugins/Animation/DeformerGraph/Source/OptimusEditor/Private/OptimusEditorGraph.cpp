// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraph.h"

#include "OptimusEditorGraphNode.h"

#include "OptimusNode.h"
#include "OptimusNode_Comment.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodeLink.h"
#include "OptimusNodePin.h"

#include "EdGraph/EdGraphPin.h"
#include "Styling/AppStyle.h"
#include "GraphEditAction.h"
#include "OptimusActionStack.h"
#include "OptimusEditorHelpers.h"
#include "OptimusEditorStyle.h"
#include "Editor.h"
#include "OptimusEditor.h"
#include "OptimusEditorGraphNode_Comment.h"
#include "Layout/SlateRect.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusEditorGraph)


UOptimusEditorGraph::UOptimusEditorGraph()
{
	AddOnGraphChangedHandler(FOnGraphChanged::FDelegate::CreateUObject(this, &UOptimusEditorGraph::HandleThisGraphModified));
}


void UOptimusEditorGraph::InitFromNodeGraph(UOptimusNodeGraph* InNodeGraph)
{
	NodeGraph = InNodeGraph;

	// Create all the nodes.
	for (UOptimusNode* ModelNode : InNodeGraph->GetAllNodes())
	{
		if (ensure(ModelNode != nullptr))
		{
			UEdGraphNode* GraphNode = AddGraphNodeFromModelNode(ModelNode);
		}
	}

	// Add all the graph links
	for (const UOptimusNodeLink* Link : InNodeGraph->GetAllLinks())
	{
		if (!ensure(Link->GetNodeOutputPin() != nullptr && Link->GetNodeInputPin() != nullptr))
		{
			continue;
		}

		UEdGraphNode* OutputGraphNode = FindGraphNodeFromModelNode(Link->GetNodeOutputPin()->GetOwningNode());
		UEdGraphNode* InputGraphNode = FindGraphNodeFromModelNode(Link->GetNodeInputPin()->GetOwningNode());

		if (OutputGraphNode == nullptr || InputGraphNode == nullptr)
		{
			continue;
		}

		FName OutputPinName = Link->GetNodeOutputPin()->GetUniqueName();
		FName InputPinName = Link->GetNodeInputPin()->GetUniqueName();

		UEdGraphPin* OutputPin = OutputGraphNode->FindPin(OutputPinName);
		UEdGraphPin* InputPin = InputGraphNode->FindPin(InputPinName);

		OutputPin->MakeLinkTo(InputPin);
	}

	// Listen to notifications from the node graph.
	InNodeGraph->GetNotifyDelegate().AddUObject(this, &UOptimusEditorGraph::HandleNodeGraphModified);
}


void UOptimusEditorGraph::Reset()
{
	if (NodeGraph == nullptr)
	{
		return;
	}

	NodeGraph->GetNotifyDelegate().RemoveAll(this);

	SelectedNodes.Reset();
	NodeGraph = nullptr;

	TArray<UEdGraphNode*> NodesToRemove(Nodes);
	for (UEdGraphNode* GraphNode : NodesToRemove)
	{
		RemoveNode(GraphNode, true);
	}
	NotifyGraphChanged();
}


const TArray<UOptimusNode*> UOptimusEditorGraph::GetSelectedModelNodes() const
{
	TArray<UOptimusNode*> SelectedModelNodes;

	for (UEdGraphNode* GraphNode: GetSelectedNodes())
	{
		if (UOptimusNode* ModelNode = OptimusEditor::FindModelNodeFromGraphNode(GraphNode))
		{
			SelectedModelNodes.Add(ModelNode);
		}
	}
	
	return SelectedModelNodes;
}

void UOptimusEditorGraph::HandleGraphNodeMoved()
{
#if WITH_EDITOR
	// Cancel the current transaction created by SNodePanel::OnMouseMove so that the
	// only transaction recorded is the one we place on the action stack.
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	
	if (SelectedNodes.Num() == 0)
	{
		return;
	}
	
	TSet<UEdGraphNode*> MovedNodes;
	for (UEdGraphNode* SelectedNode : SelectedNodes)
	{
		MovedNodes.Add(SelectedNode);
		if (UOptimusEditorGraphNode_Comment* CommentNode = Cast<UOptimusEditorGraphNode_Comment>(SelectedNode))
		{
			for (FCommentNodeSet::TConstIterator NodeIt(CommentNode->GetNodesUnderComment()); NodeIt; ++NodeIt)
			{
				if (UEdGraphNode* NodeUnderComment = Cast<UEdGraphNode>(*NodeIt))
				{
					MovedNodes.Add(NodeUnderComment);
				}
			}
		}
	}
	
	FString ActionTitle;
	if (MovedNodes.Num() == 1)
	{
		ActionTitle = TEXT("Move Node");
	}
	else
	{
		ActionTitle = FString::Printf(TEXT("Move %d Nodes"), MovedNodes.Num());
	}
	
	FOptimusActionScope Scope(*GetModelGraph()->GetActionStack(), ActionTitle);
	for (UEdGraphNode* MovedNode : MovedNodes)
	{
		FVector2D Position(MovedNode->NodePosX, MovedNode->NodePosY);
		// It is possible that the model node is deleted during drag
		UOptimusNode* ModelNode = OptimusEditor::FindModelNodeFromGraphNode(MovedNode);
		if (ModelNode)
		{
			ModelNode->SetGraphPosition(Position);
		}
	}
}

bool UOptimusEditorGraph::GetBoundsForSelectedNodes(FSlateRect& Rect)
{
	if (!WeakEditor.IsValid())
	{
		return false;
	}

	return WeakEditor.Pin()->GetBoundsForSelectedNodes(Rect);
}

void UOptimusEditorGraph::RefreshVisualNode(UOptimusEditorGraphNode* InGraphNode)
{
	// Ensure that SOptimusEditorGraphNode captures the latest.
	InGraphNode->UpdateTopLevelPins();

	// We send an AddNode notif to UEdGraph which magically removes the node
	// if it already exists and recreates it.
	FEdGraphEditAction EditAction;
	EditAction.Graph = this;
	EditAction.Action = GRAPHACTION_AddNode;
	EditAction.bUserInvoked = false;
	EditAction.Nodes.Add(InGraphNode);
	NotifyGraphChanged(EditAction);	
}


const FSlateBrush* UOptimusEditorGraph::GetGraphTypeIcon(
	const UOptimusNodeGraph* InModelGraph
	)
{
	switch(InModelGraph->GetGraphType())
	{
	case EOptimusNodeGraphType::Setup:
		return FOptimusEditorStyle::Get().GetBrush(TEXT("GraphType.Setup"));
	case EOptimusNodeGraphType::Update:
		return FOptimusEditorStyle::Get().GetBrush(TEXT("GraphType.Update"));
	case EOptimusNodeGraphType::ExternalTrigger:
		return FOptimusEditorStyle::Get().GetBrush(TEXT("GraphType.Trigger"));	
	case EOptimusNodeGraphType::SubGraph:
		return FOptimusEditorStyle::Get().GetBrush(TEXT("GraphType.SubGraph"));
	case EOptimusNodeGraphType::Function:
		return FAppStyle::Get().GetBrush(TEXT("GraphEditor.Function_16x"));
	default:
		checkNoEntry();
		break;
	}
	return nullptr;
}


void UOptimusEditorGraph::HandleThisGraphModified(const FEdGraphEditAction& InEditAction)
{
	switch (InEditAction.Action)
	{
		case GRAPHACTION_SelectNode:
		{
			SelectedNodes.Reset();
			for (const UEdGraphNode* Node : InEditAction.Nodes)
			{
				UOptimusEditorGraphNode* GraphNode = Cast<UOptimusEditorGraphNode>(
					const_cast<UEdGraphNode*>(Node));
				if (GraphNode != nullptr)
				{
					SelectedNodes.Add(GraphNode);
				}
			}
			break;
		}
		case GRAPHACTION_RemoveNode:
		{
			for (const UEdGraphNode* Node : InEditAction.Nodes)
			{
				UOptimusEditorGraphNode* GraphNode = Cast<UOptimusEditorGraphNode>(
					const_cast<UEdGraphNode*>(Node));
				if (GraphNode != nullptr)
				{
					SelectedNodes.Remove(GraphNode);
				}
			}
			break;
		}

		default:
			break;
	}
}


void UOptimusEditorGraph::HandleNodeGraphModified(EOptimusGraphNotifyType InNotifyType, UOptimusNodeGraph* InNodeGraph, UObject* InSubject)
{
	switch (InNotifyType)
	{
		case EOptimusGraphNotifyType::NodeAdded:
		{
			UOptimusNode *ModelNode = Cast<UOptimusNode>(InSubject);

			if (ensure(ModelNode))
			{
				AddGraphNodeFromModelNode(ModelNode);
			}
		    break;
		}

		case EOptimusGraphNotifyType::NodeRemoved:
		{
			const UOptimusNode* ModelNode = Cast<UOptimusNode>(InSubject);

			UEdGraphNode* GraphNode = FindGraphNodeFromModelNode(ModelNode);

			if (ensure(GraphNode))
			{
				RemoveGraphNode(GraphNode);
			}
		    break;
		}

		case EOptimusGraphNotifyType::LinkAdded:
		case EOptimusGraphNotifyType::LinkRemoved:
		{
			const UOptimusNodeLink *ModelNodeLink = Cast<UOptimusNodeLink>(InSubject);
			const UOptimusEditorGraphNode* OutputGraphNode = FindOptimusGraphNodeFromModelNode(ModelNodeLink->GetNodeOutputPin()->GetOwningNode());
			const UOptimusEditorGraphNode* InputGraphNode = FindOptimusGraphNodeFromModelNode(ModelNodeLink->GetNodeInputPin()->GetOwningNode());

			if (ensure(OutputGraphNode) && ensure(InputGraphNode))
			{
				UEdGraphPin *OutputGraphPin = OutputGraphNode->FindGraphPinFromModelPin(ModelNodeLink->GetNodeOutputPin());
				UEdGraphPin* InputGraphPin = InputGraphNode->FindGraphPinFromModelPin(ModelNodeLink->GetNodeInputPin());

				if (ensure(OutputGraphPin) && ensure(InputGraphPin))
				{
					if (InNotifyType == EOptimusGraphNotifyType::LinkAdded)
					{
						OutputGraphPin->MakeLinkTo(InputGraphPin);
					}
					else
					{
						OutputGraphPin->BreakLinkTo(InputGraphPin);					
					}
				}
			}
		    break;
		}

		case EOptimusGraphNotifyType::NodeDisplayNameChanged:
		{
			const UOptimusNode* ModelNode = Cast<UOptimusNode>(InSubject);
			UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelNode);

			GraphNode->SyncGraphNodeNameWithModelNodeName();
		}
		break;

		case EOptimusGraphNotifyType::NodePositionChanged:
		{
			const UOptimusNode* ModelNode = Cast<UOptimusNode>(InSubject);
			UEdGraphNode* GraphNode = FindGraphNodeFromModelNode(ModelNode);

			if (ensure(GraphNode))
			{
				GraphNode->NodePosX = FMath::RoundToInt(ModelNode->GetGraphPosition().X);
				GraphNode->NodePosY = FMath::RoundToInt(ModelNode->GetGraphPosition().Y);
			}

			if (UOptimusEditorGraphNode_Comment* CommentGraphNode = Cast<UOptimusEditorGraphNode_Comment>(GraphNode))
			{
				// Notify slate graph node widget
				CommentGraphNode->OnPositionChanged();
			}
		    break;
		}

		case EOptimusGraphNotifyType::NodeDiagnosticLevelChanged:
		{
			const UOptimusNode* ModelNode = Cast<UOptimusNode>(InSubject);
			UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelNode);

			GraphNode->SyncDiagnosticStateWithModelNode();

			break;
		}

	    case EOptimusGraphNotifyType::PinAdded:
		{
		    const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
		    if (ensure(ModelPin))
		    {
			    UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

			    if (ensure(GraphNode))
			    {
					GraphNode->ModelPinAdded(ModelPin);
			    }
			}
			break;
		}

		case EOptimusGraphNotifyType::PinRemoved:
		{
			const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
			if (ensure(ModelPin))
			{
				UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

				if (ensure(GraphNode))
				{
					GraphNode->ModelPinRemoved(ModelPin);
				}
			}
			break;
		}

		case EOptimusGraphNotifyType::PinMoved:
		{
			const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
			if (ensure(ModelPin))
			{
				UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

				if (ensure(GraphNode))
				{
					GraphNode->ModelPinMoved(ModelPin);
				}
			}
			break;
		}
		
		case EOptimusGraphNotifyType::PinRenamed:
		{
		    const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
		    if (ensure(ModelPin))
		    {
			    UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

			    if (ensure(GraphNode))
			    {
				    GraphNode->SynchronizeGraphPinNameWithModelPin(ModelPin);
			    }
		    }
			break;
		}

		case EOptimusGraphNotifyType::PinValueChanged:
		{
			// The pin's value was changed on the model pin itself. The model pin has already
			// updated the stored node value. We just need to ensure that the graph node shows
			// the same value (which may now include clamping and sanitizing).
			const UOptimusNodePin *ModelPin = Cast<UOptimusNodePin>(InSubject);
			if (ensure(ModelPin))
			{
			    UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

				if (ensure(GraphNode))
				{
				    GraphNode->SynchronizeGraphPinValueWithModelPin(ModelPin);
				}
			}
		    break;
		}

		case EOptimusGraphNotifyType::PinTypeChanged:
		// FIXME: For now we just use the same reconstruction as type change for domain change.
		case EOptimusGraphNotifyType::PinDataDomainChanged:
		{
			// The pin type has changed. We may need to reconstruct the pin, especially if it
			// had sub-pins before but doesn't now, or the other way around. 
		    const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
		    if (ensure(ModelPin))
		    {
			    UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

			    if (ensure(GraphNode))
			    {
				    GraphNode->SynchronizeGraphPinTypeWithModelPin(ModelPin);
			    }
			}
			break;
		}

		case EOptimusGraphNotifyType::PinExpansionChanged:
	    {
			const UOptimusNodePin* ModelPin = Cast<UOptimusNodePin>(InSubject);
			if (ensure(ModelPin))
			{
				UOptimusEditorGraphNode* GraphNode = FindOptimusGraphNodeFromModelNode(ModelPin->GetOwningNode());

				if (ensure(GraphNode))
				{
					GraphNode->SynchronizeGraphPinExpansionWithModelPin(ModelPin);
				}
			}
			break;
	    }
	}
}


UEdGraphNode* UOptimusEditorGraph::AddGraphNodeFromModelNode(UOptimusNode* InModelNode)
{
	UEdGraphNode* GraphNode = nullptr;
	
	const bool bIsCreatedFromUI = InModelNode->IsCreatedFromUI();
	
	if (UOptimusNode_Comment* CommentNode = Cast<UOptimusNode_Comment>(InModelNode))
	{
		FGraphNodeCreator<UOptimusEditorGraphNode_Comment> NodeCreator(*this);

		UOptimusEditorGraphNode_Comment* CommentGraphNode = bIsCreatedFromUI ?
			NodeCreator.CreateUserInvokedNode(false) :
			NodeCreator.CreateNode(false);
		
		CommentGraphNode->Construct(CommentNode);
		NodeCreator.Finalize();

		GraphNode = CommentGraphNode;
	}
	else
	{
		FGraphNodeCreator<UOptimusEditorGraphNode> NodeCreator(*this);

		UOptimusEditorGraphNode* OptimusGraphNode = bIsCreatedFromUI ?
			NodeCreator.CreateUserInvokedNode(false) :
			NodeCreator.CreateNode(false);
		
		OptimusGraphNode->Construct(InModelNode);
		NodeCreator.Finalize();

		GraphNode = OptimusGraphNode;
	}

		
	GraphNodeToModelNodeMap.Add(GraphNode, InModelNode);
	ModelNodeToGraphNodeMap.Add(InModelNode, GraphNode);
	
	return GraphNode;
}

bool UOptimusEditorGraph::RemoveGraphNode(UEdGraphNode* NodeToRemove, bool bBreakAllLinks)
{
	UOptimusNode* ModelNode = GraphNodeToModelNodeMap[NodeToRemove].Get(); 

	ModelNodeToGraphNodeMap.Remove(ModelNode);
	GraphNodeToModelNodeMap.Remove(NodeToRemove);
	
	return RemoveNode(NodeToRemove, bBreakAllLinks);
}


UOptimusEditorGraphNode* UOptimusEditorGraph::FindOptimusGraphNodeFromModelNode(const UOptimusNode* InModelNode)
{
	return Cast<UOptimusEditorGraphNode>(FindGraphNodeFromModelNode(InModelNode));
}

UEdGraphNode* UOptimusEditorGraph::FindGraphNodeFromModelNode(const UOptimusNode* InModelNode)
{
	return ModelNodeToGraphNodeMap.FindRef(InModelNode).Get();
}

UOptimusNode* UOptimusEditorGraph::FindModelNodeFromGraphNode(const UEdGraphNode* InGraphNode)
{
	return GraphNodeToModelNodeMap.FindRef(InGraphNode).Get();
}
