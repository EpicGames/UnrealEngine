// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusCoreNotify.h"

#include "EdGraph/EdGraph.h"
#include "Containers/Set.h"

#include "OptimusEditorGraph.generated.h"

struct FSlateBrush;
class UOptimusNode;
class UOptimusDeformer;
class UOptimusEditorGraphNode;
class FSlateRect;
class FOptimusEditor;

UCLASS()
class UOptimusEditorGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	UOptimusEditorGraph();

	void InitFromNodeGraph(UOptimusNodeGraph* InNodeGraph);

	void Reset();

	UOptimusNodeGraph* GetModelGraph() const { return NodeGraph; }

	UEdGraphNode* FindGraphNodeFromModelNode(const UOptimusNode* InModelNode);
	UOptimusNode* FindModelNodeFromGraphNode(const UEdGraphNode* InGraphNode);
	
	UOptimusEditorGraphNode* FindOptimusGraphNodeFromModelNode(const UOptimusNode* InModelNode);

	const TSet<UEdGraphNode*> &GetSelectedNodes() const { return SelectedNodes; }
	
	const TArray<UOptimusNode*> GetSelectedModelNodes() const;

	void HandleGraphNodeMoved();

	bool GetBoundsForSelectedNodes(FSlateRect& Rect);
	
	// Do a visual refresh of the node.
	void RefreshVisualNode(UOptimusEditorGraphNode *InGraphNode);

	///
	static const FSlateBrush* GetGraphTypeIcon(const UOptimusNodeGraph* InModelGraph);

protected:
	friend class FOptimusEditor;

	void SetSelectedNodes(const TSet<UEdGraphNode*>& InSelectedNodes)
	{
		SelectedNodes = InSelectedNodes;
	}

private:
	void HandleThisGraphModified(
		const FEdGraphEditAction &InEditAction
	);

	void HandleNodeGraphModified(
		EOptimusGraphNotifyType InNotifyType, 
		UOptimusNodeGraph *InNodeGraph, 
		UObject *InSubject
		);

	UEdGraphNode* AddGraphNodeFromModelNode(UOptimusNode* InModelNode);

	bool RemoveGraphNode(UEdGraphNode* NodeToRemove, bool bBreakAllLinks = true);

	TObjectPtr<UOptimusNodeGraph> NodeGraph;

	TSet<UEdGraphNode*> SelectedNodes;

	TMap<TWeakObjectPtr<UEdGraphNode>, TWeakObjectPtr<UOptimusNode>> GraphNodeToModelNodeMap;
	TMap<TWeakObjectPtr<UOptimusNode>, TWeakObjectPtr<UEdGraphNode>> ModelNodeToGraphNodeMap;

	TWeakPtr<FOptimusEditor> WeakEditor;
};
