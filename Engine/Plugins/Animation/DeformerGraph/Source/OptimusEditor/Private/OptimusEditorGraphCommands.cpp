// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphCommands.h"

#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "OptimusEditorGraphCommands"


FOptimusEditorGraphCommands::FOptimusEditorGraphCommands() 
	: TCommands<FOptimusEditorGraphCommands>(
		"OptimusEditorGraph", // Context name for fast lookup
		NSLOCTEXT("Contexts", "DeformerGraphEditorGraph", "Deformer Graph Editor Graph"), // Localized context name for displaying
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{
}


void FOptimusEditorGraphCommands::RegisterCommands()
{
	UI_COMMAND(ConvertToKernelFunction, "Convert to Kernel Function", "Convert the selected custom kernel nodes to a shareable kernel function.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertFromKernelFunction, "Convert from Kernel Function", "Convert the selected kernel function nodes to a custom kernel.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CollapseNodesToFunction, "Collapse to a Function", "Collapse the selected nodes to a reusable function graph, replacing the selected nodes with a single node.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CollapseNodesToSubGraph, "Collapse to a Sub-Graph", "Collapse the selected nodes to a non-reusable sub-graph, replacing the selected nodes with a single node.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExpandCollapsedNode, "Expand Collapsed Node", "Replace the selected node with the collapsed nodes from inside the selected function or sub-graph node.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ConvertToFunction, "Convert to a Function", "Convert the selected SubGraph to a reusable Function.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertToSubGraph, "Convert to a SubGraph", "Convert the selected Function to a non-reusable SubGraph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FindInDeformer, "FindInDeformer", "Search the current deformer graph.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
}

#undef LOCTEXT_NAMESPACE
