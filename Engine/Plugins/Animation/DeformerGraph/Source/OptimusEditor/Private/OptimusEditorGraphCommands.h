// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FOptimusEditorGraphCommands :
	public TCommands<FOptimusEditorGraphCommands>
{
public:
	FOptimusEditorGraphCommands();
	
	// TCommands<> overrides
	void RegisterCommands() override;
	
	TSharedPtr<FUICommandInfo> ConvertToKernelFunction;
	TSharedPtr<FUICommandInfo> ConvertFromKernelFunction;

	TSharedPtr<FUICommandInfo> CollapseNodesToFunction;
	TSharedPtr<FUICommandInfo> CollapseNodesToSubGraph;
	TSharedPtr<FUICommandInfo> ExpandCollapsedNode;
	
	TSharedPtr<FUICommandInfo> ConvertToFunction;
	TSharedPtr<FUICommandInfo> ConvertToSubGraph;

	TSharedPtr<FUICommandInfo> FindInDeformer;
};
