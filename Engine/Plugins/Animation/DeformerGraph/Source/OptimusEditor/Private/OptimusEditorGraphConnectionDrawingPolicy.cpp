// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphConnectionDrawingPolicy.h"
#include "EdGraph/EdGraph.h"

FOptimusEditorGraphConnectionDrawingPolicy::FOptimusEditorGraphConnectionDrawingPolicy(
	int32 InBackLayerID,
	int32 InFrontLayerID,
	float ZoomFactor,
	const FSlateRect& InClippingRect,
	FSlateWindowElementList& InDrawElements,
	UEdGraph* InGraphObj
	) :
	FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements),
	Graph(InGraphObj)
{
}


void FOptimusEditorGraphConnectionDrawingPolicy::DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	auto DrawPin = [this, &ArrangedNodes](UEdGraphPin* ThePin, TSharedRef<SWidget>& InSomePinWidget)
	{
		if (ThePin->Direction == EGPD_Output)
		{
			for (int32 LinkIndex=0; LinkIndex < ThePin->LinkedTo.Num(); ++LinkIndex)
			{
				FArrangedWidget* LinkStartWidgetGeometry = nullptr;
				FArrangedWidget* LinkEndWidgetGeometry = nullptr;

				UEdGraphPin* TargetPin = ThePin->LinkedTo[LinkIndex];

				DetermineLinkGeometry(ArrangedNodes, InSomePinWidget, ThePin, TargetPin, /*out*/ LinkStartWidgetGeometry, /*out*/ LinkEndWidgetGeometry);

				if (( LinkEndWidgetGeometry && LinkStartWidgetGeometry ) && !IsConnectionCulled( *LinkStartWidgetGeometry, *LinkEndWidgetGeometry ))
				{
					FConnectionParams Params;
					DetermineWiringStyle(ThePin, TargetPin, /*inout*/ Params);

#if 0
					if(UseLowDetailConnections())
					{
						const float StartFudgeX = 4.0f;
						const float EndFudgeX = 4.0f;

						const FVector2D StartPoint = FGeometryHelper::VerticalMiddleRightOf(LinkStartWidgetGeometry->Geometry) - FVector2D(StartFudgeX, 0.0f);
						const FVector2D EndPoint = FGeometryHelper::VerticalMiddleLeftOf(LinkEndWidgetGeometry->Geometry) - FVector2D(EndFudgeX, 0);
						const FVector2D FakeTangent = (EndPoint - StartPoint).GetSafeNormal();

						static TArray<FVector2D> LinePoints;
						LinePoints.SetNum(2);
						LinePoints[0] = StartPoint;
						LinePoints[1] = EndPoint;

						FSlateDrawElement::MakeLines(DrawElementsList, WireLayerID, FPaintGeometry(), LinePoints, ESlateDrawEffect::None, Params.WireColor);
					}
					else
#endif
					{
						DrawSplineWithArrow(LinkStartWidgetGeometry->Geometry, LinkEndWidgetGeometry->Geometry, Params);
					}
				}
			}
		}
	};

	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		TSharedRef<SWidget> SomePinWidget = ConnectorIt.Key();
		SGraphPin& PinWidget = static_cast<SGraphPin&>(SomePinWidget.Get());
		
		struct Local
		{
			static void DrawSubPins_Recursive(UEdGraphPin* PinObj, TSharedRef<SWidget>& InSomePinWidget, const TFunctionRef<void(UEdGraphPin* Pin, TSharedRef<SWidget>& PinWidget)>& DrawPinFunction)
			{
				DrawPinFunction(PinObj, InSomePinWidget);

				for(UEdGraphPin* SubPin : PinObj->SubPins)
				{
					DrawSubPins_Recursive(SubPin, InSomePinWidget, DrawPinFunction);
				}
			}
		};

		Local::DrawSubPins_Recursive(PinWidget.GetPinObj(), SomePinWidget, DrawPin);
	}
}

void FOptimusEditorGraphConnectionDrawingPolicy::DetermineLinkGeometry(
	FArrangedChildren& ArrangedNodes, 
	TSharedRef<SWidget>& OutputPinWidget,
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	/*out*/ FArrangedWidget*& StartWidgetGeometry,
	/*out*/ FArrangedWidget*& EndWidgetGeometry
	)
{
	if (const TSharedPtr<SGraphPin>* OutputWidgetPtr = PinToPinWidgetMap.Find(OutputPin))
	{
		StartWidgetGeometry = PinGeometries->Find((*OutputWidgetPtr).ToSharedRef());
	}
	
	if (const TSharedPtr<SGraphPin>* InputWidgetPtr = PinToPinWidgetMap.Find(InputPin))
	{
		EndWidgetGeometry = PinGeometries->Find((*InputWidgetPtr).ToSharedRef());
	}
}

void FOptimusEditorGraphConnectionDrawingPolicy::AddSubPinsToWidgetMap(UEdGraphPin* InPinObj, TSharedPtr<SGraphPin>& InGraphPinWidget)
{
	for(UEdGraphPin* SubPin : InPinObj->SubPins)
	{
		// Only add to the pin-to-pin widget map if the sub-pin widget is not there already
		const TSharedPtr<SGraphPin>* SubPinWidgetPtr = PinToPinWidgetMap.Find(SubPin);
		if(SubPinWidgetPtr == nullptr)
		{
			SubPinWidgetPtr = &InGraphPinWidget;
		}

		TSharedPtr<SGraphPin> PinWidgetPtr = *SubPinWidgetPtr;
		PinToPinWidgetMap.Add(SubPin, PinWidgetPtr);
		AddSubPinsToWidgetMap(SubPin, PinWidgetPtr);
	}
}


void FOptimusEditorGraphConnectionDrawingPolicy::BuildPinToPinWidgetMap(
	TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries)
{
	FConnectionDrawingPolicy::BuildPinToPinWidgetMap(InPinGeometries);

	// Add any sub-pins to the widget map if they arent there already, but with their parents' geometry so that we can
	// properly draw links to sub-pins that are under collapsed parent pins, by making those links point to the parent
	// pin's geometry instead (rather than disappearing).
	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		TSharedPtr<SGraphPin> GraphPinWidget = StaticCastSharedRef<SGraphPin>(ConnectorIt.Key());
		AddSubPinsToWidgetMap(GraphPinWidget->GetPinObj(), GraphPinWidget);
	}
}


void FOptimusEditorGraphConnectionDrawingPolicy::DetermineWiringStyle(
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	FConnectionParams& Params
	)
{
	FConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);
	
	if (ensure(Graph))
	{
		const UEdGraphSchema* Schema = Graph->GetSchema();

		Params.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);
	}
}
