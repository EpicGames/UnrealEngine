// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"

class FOptimusEditorGraphConnectionDrawingPolicy final :
	public FConnectionDrawingPolicy 
{
public:
	FOptimusEditorGraphConnectionDrawingPolicy(
		int32 InBackLayerID,
		int32 InFrontLayerID,
		float ZoomFactor,
		const FSlateRect& InClippingRect,
		FSlateWindowElementList& InDrawElements,
		UEdGraph* InGraphObj
		);
	
	~FOptimusEditorGraphConnectionDrawingPolicy() override = default;

	
	void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	void DetermineLinkGeometry(FArrangedChildren& ArrangedNodes, TSharedRef<SWidget>& OutputPinWidget, UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FArrangedWidget*& StartWidgetGeometry, FArrangedWidget*& EndWidgetGeometry) override;

protected:
	void BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries) override;
	void DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes) override;
	
private:
	void AddSubPinsToWidgetMap(UEdGraphPin* InPinObj, TSharedPtr<SGraphPin>& InGraphPinWidget);	
	
	UEdGraph* Graph;
};
