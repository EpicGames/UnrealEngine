// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphNode.h"

#include "IOptimusComputeKernelProvider.h"
#include "IOptimusDataInterfaceProvider.h"
#include "OptimusEditorHelpers.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphSchema.h"
#include "OptimusEditorCommands.h"

#include "OptimusDataType.h"
#include "OptimusEditorGraphCommands.h"
#include "OptimusNode.h"
#include "OptimusNodePin.h"
#include "IOptimusNodeAdderPinProvider.h"
#include "IOptimusNonCopyableNode.h"
#include "IOptimusPinMutabilityDefiner.h"
#include "OptimusComponentSource.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusNodeGraph.h"

#include "Framework/Commands/GenericCommands.h"
#include "Logging/TokenizedMessage.h"
#include "ToolMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusEditorGraphNode)


#define LOCTEXT_NAMESPACE "OptimusEditorGraphNode"

/** Used when identifying a pin as a grouping pin */
FName UOptimusEditorGraphNode::GroupTypeName("group");


static void CreateAdderPins(UEdGraphNode* InGraphNode)
{
	FEdGraphPinType PinType;
	PinType.PinCategory = OptimusEditor::GetAdderPinCategoryName();
			
	UEdGraphPin* InputAdderPin = InGraphNode->CreatePin(EGPD_Input, PinType, OptimusEditor::GetAdderPinName(EGPD_Input));	
	UEdGraphPin* OutputAdderPin = InGraphNode->CreatePin(EGPD_Output, PinType, OptimusEditor::GetAdderPinName(EGPD_Output));


#if WITH_EDITORONLY_DATA
	InputAdderPin->PinFriendlyName = OptimusEditor::GetAdderPinFriendlyName(EGPD_Input);
	OutputAdderPin->PinFriendlyName = OptimusEditor::GetAdderPinFriendlyName(EGPD_Output);
#endif
};

void UOptimusEditorGraphNode::Construct(UOptimusNode* InModelNode)
{
	// Our graph nodes are not transactional. We handle the transacting ourselves.
	ClearFlags(RF_Transactional);
	
	if (ensure(InModelNode))
	{
		ModelNode = InModelNode;

		NodePosX = int(InModelNode->GetGraphPosition().X);
		NodePosY = int(InModelNode->GetGraphPosition().Y);

		UpdateTopLevelPins();

		// Start with all input pins
		for (const UOptimusNodePin* ModelPin : ModelNode->GetPins())
		{
			if (ModelPin->GetDirection() == EOptimusNodePinDirection::Input)
			{
				CreateGraphPinFromModelPin(ModelPin, EGPD_Input);
			}
		}

		// Then all output pins
		for (const UOptimusNodePin* ModelPin : ModelNode->GetPins())
		{
			if (ModelPin->GetDirection() == EOptimusNodePinDirection::Output)
			{
				CreateGraphPinFromModelPin(ModelPin, EGPD_Output);
			}
		}

		if (const IOptimusNodeAdderPinProvider* AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(ModelNode))
		{
			CreateAdderPins(this);
		}

		SyncDiagnosticStateWithModelNode();
	}
}


UOptimusNodePin* UOptimusEditorGraphNode::FindModelPinFromGraphPin(
	const UEdGraphPin* InGraphPin
	) const
{
	if (InGraphPin == nullptr)
	{
		return nullptr;
	}
	return PathToModelPinMap.FindRef(InGraphPin->GetFName());
}

UEdGraphPin* UOptimusEditorGraphNode::FindGraphPinFromModelPin(
	const UOptimusNodePin* InModelPin
	) const
{
	if (InModelPin == nullptr)
	{
		return nullptr;
	}
	return PathToGraphPinMap.FindRef(InModelPin->GetUniqueName());
}


void UOptimusEditorGraphNode::SynchronizeGraphPinNameWithModelPin(
	const UOptimusNodePin* InModelPin
	)
{
	// FindGraphPinFromModelPin will not work here since the model pin now carries the new pin
	// path but our maps do not. We have to search linearly on the pointer to get the old name.
	FName OldPinPath;
	for (const TPair<FName, UOptimusNodePin*>& Item : PathToModelPinMap)
	{
		if (Item.Value == InModelPin)
		{
			OldPinPath = Item.Key;
			break;
		}
	}

	UEdGraphPin* GraphPin = nullptr;

	if (!OldPinPath.IsNone())
	{
		GraphPin = PathToGraphPinMap.FindRef(OldPinPath);
	}

	if (GraphPin)
	{
		FName NewPinPath = InModelPin->GetUniqueName();

		// Update the resolver maps first.
		PathToModelPinMap.Remove(OldPinPath);
		PathToModelPinMap.Add(NewPinPath, const_cast<UOptimusNodePin *>(InModelPin));

		PathToGraphPinMap.Remove(OldPinPath);
		PathToGraphPinMap.Add(NewPinPath, GraphPin);

		GraphPin->PinName = NewPinPath;
		GraphPin->PinFriendlyName = InModelPin->GetDisplayName();

		for (UOptimusNodePin* ModelSubPin : InModelPin->GetSubPins())
		{
			SynchronizeGraphPinNameWithModelPin(ModelSubPin);
		}
	}

	// The slate node will automatically pick up the new name on the next tick.
}


void UOptimusEditorGraphNode::SynchronizeGraphPinValueWithModelPin(
	const UOptimusNodePin* InModelPin
	)
{
	if (ensure(InModelPin))
	{
		if (InModelPin->GetSubPins().IsEmpty())
		{
			UEdGraphPin *GraphPin = FindGraphPinFromModelPin(InModelPin);

			// Only update the value if the pin cares about it.
			if (ensure(GraphPin) && !GraphPin->bDefaultValueIsIgnored)
			{
				FString ValueString = InModelPin->GetValueAsString();

				if (GraphPin->DefaultValue != ValueString)
				{
					GraphPin->Modify();
					GraphPin->DefaultValue = ValueString;
				}
			}
		}
		else
		{
			for (const UOptimusNodePin* ModelSubPin : InModelPin->GetSubPins())
			{
				SynchronizeGraphPinValueWithModelPin(ModelSubPin);
			}
		}
	}
}


void UOptimusEditorGraphNode::SynchronizeGraphPinTypeWithModelPin(
	const UOptimusNodePin* InModelPin
	)
{
	// If the pin has sub-pins, we may need to remove or rebuild the sub-pins.
	UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin);
	if (ensure(GraphPin))
	{
		// If the graph node had sub-pins, we need to remove those.
		RemoveGraphSubPins(GraphPin);

		// Create new sub-pins, if required, to reflect the new type.
		if (!InModelPin->GetSubPins().IsEmpty())
		{
			for (const UOptimusNodePin* ModelSubPin : InModelPin->GetSubPins())
			{
				CreateGraphPinFromModelPin(ModelSubPin, GraphPin->Direction, GraphPin);
			}
		}

		if (InModelPin->IsGroupingPin())
		{
			GraphPin->PinType.PinCategory = GroupTypeName;
		}
		else
		{
			const FOptimusDataTypeHandle DataType = InModelPin->GetDataType();
			if (!ensure(DataType.IsValid()))
			{
				return;
			}

			GraphPin->PinType = UOptimusEditorGraphSchema::GetPinTypeFromDataType(DataType);
		}
		
		// Notify the node widget that the pins have changed.
		(void)NodePinsChanged.ExecuteIfBound();
	}
}

void UOptimusEditorGraphNode::SynchronizeGraphPinExpansionWithModelPin(const UOptimusNodePin* InModelPin)
{
	// For now, we just use a big sledgehammer.
	(void)NodePinExpansionChanged.ExecuteIfBound();
}


void UOptimusEditorGraphNode::SyncGraphNodeNameWithModelNodeName()
{
	(void)NodeTitleDirtied.ExecuteIfBound();
}


void UOptimusEditorGraphNode::SyncDiagnosticStateWithModelNode()
{
	const EOptimusDiagnosticLevel DiagnosticLevel = ModelNode->GetDiagnosticLevel();
	switch(DiagnosticLevel)
	{
	case EOptimusDiagnosticLevel::None:
	case EOptimusDiagnosticLevel::Info:
		bHasCompilerMessage = false;
		ErrorType = EMessageSeverity::Info;
		break;
	case EOptimusDiagnosticLevel::Warning:
		bHasCompilerMessage = true;
		ErrorType = EMessageSeverity::Warning;
		break;
	case EOptimusDiagnosticLevel::Error:
		bHasCompilerMessage = true;
		ErrorType = EMessageSeverity::Error;
		break;
	}
}


bool UOptimusEditorGraphNode::CanDuplicateNode() const
{
	return Super::CanDuplicateNode() && (!ModelNode || Cast<IOptimusNonCopyableNode>(ModelNode) == nullptr);
}

bool UOptimusEditorGraphNode::CanUserDeleteNode() const
{
	return Super::CanUserDeleteNode() && (!ModelNode || ModelNode->CanUserDeleteNode());
}


FText UOptimusEditorGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (ModelNode)
	{
		return ModelNode->GetDisplayName();
	}

	return {};
}


void UOptimusEditorGraphNode::GetNodeContextMenuActions(
	UToolMenu* InMenu,
	UGraphNodeContextMenuContext* InContext
	) const
{
	if (InContext->Pin)
	{
		
	}
	else if (InContext->Node)
	{
		FToolMenuSection& Clipboard = InMenu->AddSection("OptimusNodeClipboard", LOCTEXT("NodeMenuClipboardHeader", "Clipboard"));
		Clipboard.AddMenuEntry(FGenericCommands::Get().Copy);
		Clipboard.AddMenuEntry(FGenericCommands::Get().Cut);
		Clipboard.AddMenuEntry(FGenericCommands::Get().Paste);
		Clipboard.AddMenuEntry(FGenericCommands::Get().Duplicate);

		FToolMenuSection& PackagingSection = InMenu->AddSection("OptimusNodePackaging", LOCTEXT("NodeMenuPackagingHeader", "Packaging"));
#if 0
		// NOTE: Unsupported for 5.4 
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().ConvertToKernelFunction);
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().ConvertFromKernelFunction);
#endif 
#if 0
		// NOTE: Disabled for 5.0
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().CollapseNodesToFunction);
#endif
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().CollapseNodesToSubGraph);
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().ExpandCollapsedNode);

		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().ConvertToFunction);
		PackagingSection.AddMenuEntry(FOptimusEditorGraphCommands::Get().ConvertToSubGraph);
		// FIXME: Add alignment.
	}
}

FLinearColor UOptimusEditorGraphNode::GetNodeTitleColor() const
{
	if (ModelNode)
	{
		const FLinearColor ImmutableColor = FLinearColor::Green;

		if (ModelNode->GetOwningGraph()->GetGraphType() == EOptimusNodeGraphType::Update)
		{
			if (Cast<IOptimusDataInterfaceProvider>(ModelNode))
			{
				const FOptimusDataTypeRegistry& TypeRegistry = FOptimusDataTypeRegistry::Get();
				FOptimusDataTypeHandle ComponentSourceType = TypeRegistry.FindType(*UOptimusComponentSourceBinding::StaticClass());
				
				// Any writes to data interface provider need to take place every tick, thus they are considered mutable
				for (const UOptimusNodePin* Pin : ModelNode->GetPinsByDirection(EOptimusNodePinDirection::Input, true))
				{
					if (Pin->GetDataType() != ComponentSourceType)
					{
						return Super::GetNodeTitleColor();
					}
				}
			}
			
			if (ModelNode->GetOwningGraph()->DoesNodeHaveMutableInput(ModelNode, {}))
			{
				return Super::GetNodeTitleColor();
			}

			if (IOptimusPinMutabilityDefiner* PinMutabilityDefiner = Cast<IOptimusPinMutabilityDefiner>(ModelNode))
			{
				for (UOptimusNodePin* Pin : ModelNode->GetPins())
				{
					if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
					{
						const EOptimusPinMutability Mutability = PinMutabilityDefiner->GetOutputPinMutability(Pin);
						if (Mutability == EOptimusPinMutability::Mutable)
						{
							return Super::GetNodeTitleColor();
						}
					}
				}
			}
			
			// No mutable output, no mutable input
			return ImmutableColor;
		}
	}
	
	return Super::GetNodeTitleColor();
}

FText UOptimusEditorGraphNode::GetTooltipText() const
{
	if (ensure(ModelNode))
	{
		return ModelNode->GetTooltipText();
	}

	return {};
}


bool UOptimusEditorGraphNode::CreateGraphPinFromModelPin(
	const UOptimusNodePin* InModelPin,
	EEdGraphPinDirection InDirection,
	UEdGraphPin* InParentPin
)
{
	FEdGraphPinType PinType;

	if (InModelPin->IsGroupingPin())
	{
		PinType.PinCategory = GroupTypeName;
	}
	else
	{
		const FOptimusDataTypeHandle DataType = InModelPin->GetDataType();
		if (!ensure(DataType.IsValid()))
		{
			return false;
		}

		PinType = UOptimusEditorGraphSchema::GetPinTypeFromDataType(DataType);
	}


	const FName PinPath = InModelPin->GetUniqueName();
	UEdGraphPin *GraphPin = CreatePin(InDirection, PinType, PinPath);

	GraphPin->PinFriendlyName = InModelPin->GetDisplayName();

	if (InParentPin)
	{
		InParentPin->SubPins.Add(GraphPin);
		GraphPin->ParentPin = InParentPin;
	}

	// Maintain a mapping from the pin path, which is also the graph pin's internal name, 
	// to the original model pin.
	PathToModelPinMap.Add(PinPath, const_cast<UOptimusNodePin *>(InModelPin));
	PathToGraphPinMap.Add(PinPath, GraphPin);

	if (InModelPin->GetSubPins().IsEmpty())
	{
		GraphPin->DefaultValue = InModelPin->GetValueAsString();
	}
	else
	{
		for (const UOptimusNodePin* ModelSubPin : InModelPin->GetSubPins())
		{
			CreateGraphPinFromModelPin(ModelSubPin, InDirection, GraphPin);
		}
	}
	return true;
}

void UOptimusEditorGraphNode::RemoveGraphSubPins(
	UEdGraphPin* InParentPin
	)
{
	TArray<UEdGraphPin*> SubPins = InParentPin->SubPins;
	
	for (UEdGraphPin* SubPin: SubPins)
	{
		PathToModelPinMap.Remove(SubPin->PinName);
		PathToGraphPinMap.Remove(SubPin->PinName);

		// Remove this pin from our owned pins
		Pins.Remove(SubPin);

		if (!SubPin->SubPins.IsEmpty())
		{
			RemoveGraphSubPins(SubPin);
		}
	}
	InParentPin->SubPins.Reset();
}


bool UOptimusEditorGraphNode::ModelPinAdded(const UOptimusNodePin* InModelPin)
{
	EEdGraphPinDirection GraphPinDirection;

	if (InModelPin->GetDirection() == EOptimusNodePinDirection::Input)
	{
		GraphPinDirection = EGPD_Input;
	}
	else if (InModelPin->GetDirection() == EOptimusNodePinDirection::Output)
	{
		GraphPinDirection = EGPD_Output;
	}
	else
	{
		return false;
	}
	
	if (!CreateGraphPinFromModelPin(InModelPin, GraphPinDirection))
	{
		return false;
	}

	UpdateTopLevelPins();
	
	(void)NodePinsChanged.ExecuteIfBound();

	return true;
}


bool UOptimusEditorGraphNode::ModelPinRemoved(const UOptimusNodePin* InModelPin)
{
	UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin);

	if (ensure(GraphPin))
	{
		PathToModelPinMap.Remove(GraphPin->PinName);
		PathToGraphPinMap.Remove(GraphPin->PinName);

		// Remove this pin, and all sub-pins, from our list of owned pins.
		RemoveGraphSubPins(GraphPin);
		Pins.Remove(GraphPin);
		
		UpdateTopLevelPins();

		// This takes care of updating the node, and also deleting the removed graph pins from
		// the graph node widget, which is surprisingly difficult to do from here. 
		(void)NodePinsChanged.ExecuteIfBound();
		
		return true;
	}
	else
	{
		return false;
	}
}

bool UOptimusEditorGraphNode::ModelPinMoved(
	const UOptimusNodePin* InModelPin
	)
{
	const UOptimusNodePin* NextModelPin = InModelPin->GetNextPin();
	UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin);
	UEdGraphPin* NextGraphPin = NextModelPin ? FindGraphPinFromModelPin(NextModelPin) : nullptr;
	
	if (ensure(GraphPin))
	{
		Pins.RemoveSingle(GraphPin);
		if (NextGraphPin)
		{
			const int32 BeforeIndex = Pins.IndexOfByKey(NextGraphPin);
			Pins.Insert(GraphPin, BeforeIndex);
		}
		else
		{
			Pins.Add(GraphPin);
		}

		UpdateTopLevelPins();

		// Update the Slate node so that the pin layout on the widget match the graph node's.
		(void)NodePinsChanged.ExecuteIfBound();
		
		return true;
	}
	else
	{
		return false;
	}
	
}


void UOptimusEditorGraphNode::UpdateTopLevelPins()
{
	TopLevelInputPins.Empty();
	TopLevelOutputPins.Empty();

	if (ensure(ModelNode))
	{
		for (UOptimusNodePin* Pin : ModelNode->GetPins())
		{
			if (Pin->GetDirection() == EOptimusNodePinDirection::Input)
			{
				TopLevelInputPins.Add(Pin);
			}
			else if (Pin->GetDirection() == EOptimusNodePinDirection::Output)
			{
				TopLevelOutputPins.Add(Pin);			
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
