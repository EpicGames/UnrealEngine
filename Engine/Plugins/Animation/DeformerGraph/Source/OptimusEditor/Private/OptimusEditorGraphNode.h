// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"

#include "Containers/Map.h"

#include "OptimusEditorGraphNode.generated.h"

class UOptimusNode;
class UOptimusNodePin;

DECLARE_DELEGATE(FOptimusNodeTitleDirtied);
DECLARE_DELEGATE(FOptimusNodePinsChanged);
DECLARE_DELEGATE(FOptimusNodePinExpansionChanged);

UCLASS()
class UOptimusEditorGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	static FName GroupTypeName;
	
	void Construct(UOptimusNode* InNode);

	UOptimusNodePin* FindModelPinFromGraphPin(const UEdGraphPin* InGraphPin) const;
	UEdGraphPin* FindGraphPinFromModelPin(const UOptimusNodePin* InModelPin) const;

	/// Synchronize the stored name/value/type on the graph pin with the value stored on the node. 
	/// If the pin has sub-pins, the value update is done recursively.
	void SynchronizeGraphPinNameWithModelPin(const UOptimusNodePin* InModelPin);
	void SynchronizeGraphPinValueWithModelPin(const UOptimusNodePin* InModelPin);
	void SynchronizeGraphPinTypeWithModelPin(const UOptimusNodePin* InModelPin);
	void SynchronizeGraphPinExpansionWithModelPin(const UOptimusNodePin* InModelPin);
	
	void SyncGraphNodeNameWithModelNodeName();
	void SyncDiagnosticStateWithModelNode();

	FOptimusNodeTitleDirtied& OnNodeTitleDirtied() { return NodeTitleDirtied; }

	FOptimusNodePinsChanged& OnNodePinsChanged() { return NodePinsChanged; }
	FOptimusNodePinExpansionChanged& OnNodePinExpansionChanged() { return NodePinExpansionChanged; }
	
	// UEdGraphNode overrides
	bool CanDuplicateNode() const override;
	bool CanUserDeleteNode() const override;
	FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	void GetNodeContextMenuActions(UToolMenu* InMenu, UGraphNodeContextMenuContext* InContext) const override;
	FLinearColor GetNodeTitleColor() const override;
	FText GetTooltipText() const override;
	
	// FIXME: Move to private and add accessor function.
	UPROPERTY()
	TObjectPtr<UOptimusNode> ModelNode = nullptr;

protected:
	friend class UOptimusEditorGraph;
	friend class SOptimusEditorGraphNode;

	const TArray<UOptimusNodePin*>& GetTopLevelInputPins() const { return TopLevelInputPins; }
	const TArray<UOptimusNodePin*>& GetTopLevelOutputPins() const { return TopLevelOutputPins; }

	/** Called when a model pin is added after the node creation */
	bool ModelPinAdded(
	    const UOptimusNodePin* InModelPin
		);

	/** Called when a model pin is being removed */
	bool ModelPinRemoved(
	    const UOptimusNodePin* InModelPin
		);

	/** Called after a model pin has been moved */
	bool ModelPinMoved(
		const UOptimusNodePin* InModelPin
		);

private:
	void UpdateTopLevelPins();

	bool CreateGraphPinFromModelPin(
		const UOptimusNodePin* InModelPin,
		EEdGraphPinDirection InDirection, 
	    UEdGraphPin* InParentPin = nullptr);

	void RemoveGraphSubPins(
		UEdGraphPin *InParentPin
		);
	
	TMap<FName, UOptimusNodePin*> PathToModelPinMap;
	TMap<FName, UEdGraphPin*> PathToGraphPinMap;

	// These need to be always-living arrays because of the way STreeView works. See
	// SOptimusEditorGraphNode for usage.
	TArray<UOptimusNodePin*> TopLevelInputPins;
	TArray<UOptimusNodePin*> TopLevelOutputPins;

	FOptimusNodeTitleDirtied NodeTitleDirtied;
	FOptimusNodePinsChanged NodePinsChanged;
	FOptimusNodePinExpansionChanged NodePinExpansionChanged;
};
