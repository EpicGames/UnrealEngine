// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphNodeFactory.h"

#include "OptimusEditorGraphNode.h"
#include "OptimusEditorGraphNode_Comment.h"
#include "Widgets/SOptimusEditorGraphNode.h"
#include "Widgets/SOptimusEditorGraphNode_Comment.h"

TSharedPtr<SGraphNode> FOptimusEditorGraphNodeFactory::CreateNode(UEdGraphNode* InNode) const
{
	if (UOptimusEditorGraphNode* GraphNode = Cast<UOptimusEditorGraphNode>(InNode))
	{
		TSharedRef<SGraphNode> VisualNode = 
			SNew(SOptimusEditorGraphNode).GraphNode(GraphNode);

		VisualNode->SlatePrepass();
		
		// FIXME: Copy visual node's dimensions back to graph node.
		return VisualNode;
	}
	
	if (UOptimusEditorGraphNode_Comment* CommentNode = Cast<UOptimusEditorGraphNode_Comment>(InNode))
	{
		TSharedRef<SGraphNode> VisualNode = 
			SNew(SOptimusEditorGraphNode_Comment, CommentNode);

		VisualNode->SlatePrepass();
		
		// FIXME: Copy visual node's dimensions back to graph node.
		return VisualNode;
	}

	return nullptr;
}

