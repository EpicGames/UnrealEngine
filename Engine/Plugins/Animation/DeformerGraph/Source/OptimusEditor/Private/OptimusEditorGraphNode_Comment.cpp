// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphNode_Comment.h"
#include "OptimusNode_Comment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusEditorGraphNode_Comment)

#define LOCTEXT_NAMESPACE "OptimusEditorGraphNode_Comment"


void UOptimusEditorGraphNode_Comment::Construct(UOptimusNode_Comment* InCommentNode)
{
	// Our graph nodes are not transactional. We handle the transacting ourselves.
	ClearFlags(RF_Transactional);
	NodePosX = UE::LWC::FloatToIntCastChecked<int32>(InCommentNode->GetGraphPosition().X);
	NodePosY = UE::LWC::FloatToIntCastChecked<int32>(InCommentNode->GetGraphPosition().Y);

	// Comment node specific properties
	NodeWidth = UE::LWC::FloatToIntCastChecked<int32>(InCommentNode->GetSize().X);
	NodeHeight = UE::LWC::FloatToIntCastChecked<int32>(InCommentNode->GetSize().Y);
	CommentColor = InCommentNode->CommentColor;
	NodeComment = InCommentNode->Comment;
	FontSize = InCommentNode->FontSize;
	bCommentBubbleVisible_InDetailsPanel = InCommentNode->bBubbleVisible;
	bCommentBubbleVisible = bCommentBubbleVisible_InDetailsPanel;
	bCommentBubblePinned = bCommentBubbleVisible_InDetailsPanel;
	bColorCommentBubble = InCommentNode->bColorBubble;

	InCommentNode->GetOnPropertyChanged().BindUObject(this, &UOptimusEditorGraphNode_Comment::OnModelNodePropertyChanged);
}

void UOptimusEditorGraphNode_Comment::OnModelNodePropertyChanged(UOptimusNode_Comment* InCommentNode)
{
	ResizeNode(InCommentNode->GetSize());
	
	CommentColor = InCommentNode->CommentColor;
	if (NodeComment != InCommentNode->Comment)
	{
		OnRenameNode(InCommentNode->Comment);
	}
	FontSize = InCommentNode->FontSize;
	bCommentBubbleVisible_InDetailsPanel = InCommentNode->bBubbleVisible;
	bCommentBubbleVisible = bCommentBubbleVisible_InDetailsPanel;
	bCommentBubblePinned = bCommentBubbleVisible_InDetailsPanel;
	bColorCommentBubble = InCommentNode->bColorBubble;
}

void UOptimusEditorGraphNode_Comment::OnPositionChanged()
{
	OnPositionChangedDelegate.ExecuteIfBound();
}

void UOptimusEditorGraphNode_Comment::ResizeNode(const FVector2f& NewSize)
{
	Super::ResizeNode(NewSize);

	// Notify graph node widget about the new size, since it is not done automatically on undo 
	OnSizeChangedDelegate.ExecuteIfBound();
}

void UOptimusEditorGraphNode_Comment::PostPlacedNewNode()
{
	// Do nothing here to avoid overriding work done during construct()
}

#undef LOCTEXT_NAMESPACE
