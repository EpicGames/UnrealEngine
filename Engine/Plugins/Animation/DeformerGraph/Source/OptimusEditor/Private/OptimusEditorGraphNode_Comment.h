// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphNode_Comment.h"

#include "OptimusEditorGraphNode_Comment.generated.h"

class UOptimusNode_Comment;

DECLARE_DELEGATE(FOptimusCommentNodeSizeChanged);
DECLARE_DELEGATE(FOptimusCommentNodePositionChanged);

UCLASS()
class UOptimusEditorGraphNode_Comment : public UEdGraphNode_Comment
{
	GENERATED_BODY()

public:
	void Construct(UOptimusNode_Comment* InCommentNode);
	void OnModelNodePropertyChanged(UOptimusNode_Comment* InCommentNode);

	void OnPositionChanged();

	void ResizeNode(const FVector2f& NewSize) override;
	void PostPlacedNewNode() override;
	
	FOptimusCommentNodeSizeChanged& GetOnSizeChanged() {return OnSizeChangedDelegate; }
	FOptimusCommentNodePositionChanged& GetOnPositionChanged() {return OnPositionChangedDelegate; }
private:
	FOptimusCommentNodeSizeChanged OnSizeChangedDelegate;
	FOptimusCommentNodePositionChanged OnPositionChangedDelegate;
};
