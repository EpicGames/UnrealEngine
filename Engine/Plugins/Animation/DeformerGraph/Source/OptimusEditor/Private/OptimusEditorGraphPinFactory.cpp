// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphPinFactory.h"
#include "OptimusEditorGraph.h"

#include "EdGraphSchema_K2.h"
#include "NodeFactory.h"
#include "SGraphPin.h"


TSharedPtr<SGraphPin> FOptimusEditorGraphPinFactory::CreatePin(UEdGraphPin* InPin) const
{
	if (InPin)
	{
		if (const UEdGraphNode* OwningNode = InPin->GetOwningNode())
		{
			// only create pins within optimus graphs
			if (Cast<UOptimusEditorGraph>(OwningNode->GetGraph()) == nullptr)
			{
				return nullptr;
			}
		}

		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
		{
			// Override the "object" type because otherwise we get a pin with an object selector
			// which is not what we want.
			return SNew(SGraphPin, InPin);
		}
	}

	TSharedPtr<SGraphPin> K2PinWidget = FNodeFactory::CreateK2PinWidget(InPin);
	if (K2PinWidget.IsValid())
	{
		return K2PinWidget;
	}

	return nullptr;
}
