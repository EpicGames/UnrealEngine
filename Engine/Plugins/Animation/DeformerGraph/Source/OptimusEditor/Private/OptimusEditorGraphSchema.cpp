// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphSchema.h"

#include "OptimusEditorHelpers.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphSchemaActions.h"
#include "OptimusEditorGraphNode.h"

#include "OptimusDataTypeRegistry.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"
#include "IOptimusNodeAdderPinProvider.h"
#include "IOptimusUnnamedNodePinProvider.h"

#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "OptimusActionStack.h"
#include "OptimusComponentSource.h"
#include "OptimusComputeDataInterface.h"
#include "OptimusDeformer.h"
#include "OptimusEditorGraphConnectionDrawingPolicy.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusFunctionNodeGraphHeaderWithGuid.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/SlateIconFinder.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusEditorGraphSchema)

#define LOCTEXT_NAMESPACE "OptimusEditor"

static bool IsValidAdderPinConnection(const UEdGraphPin* InPinA, const UEdGraphPin* InPinB, TArray<IOptimusNodeAdderPinProvider::FAdderPinAction>* OutActions = nullptr,
	FString* OutReason = nullptr)
{
	const bool bIsPinAAdderPin = OptimusEditor::IsAdderPin(InPinA);
	const bool bIsPinBAdderPin = OptimusEditor::IsAdderPin(InPinB);

	if (bIsPinAAdderPin && bIsPinBAdderPin)
	{
		if (OutReason)
		{
			*OutReason = TEXT("Can't connect adder pin to adder pin");
		}
		return false;
	}

	// The pins should be in the correct order now.
	UOptimusNodePin* ModelPinA = OptimusEditor::GetModelPinFromGraphPin(InPinA);
	UOptimusNodePin* ModelPinB = OptimusEditor::GetModelPinFromGraphPin(InPinB);
	UOptimusNode* ModelNodeA = OptimusEditor::GetModelNodeFromGraphPin(InPinA);
	UOptimusNode* ModelNodeB = OptimusEditor::GetModelNodeFromGraphPin(InPinB);

	// Default to Adder Pin on the PinB side
	EOptimusNodePinDirection NewPinDirection =
		InPinB->Direction == EGPD_Input?
			EOptimusNodePinDirection::Input : EOptimusNodePinDirection::Output;
	UOptimusNode* TargetNode = ModelNodeB;
	UOptimusNodePin* SourcePin = ModelPinA;

	// Flip Source/Target if the Adder Pin is on the PinA side
	if (bIsPinAAdderPin)
	{
		NewPinDirection =
			InPinA->Direction == EGPD_Input?
				EOptimusNodePinDirection::Input : EOptimusNodePinDirection::Output;
		TargetNode = ModelNodeA;
		SourcePin = ModelPinB;
	}

	const IOptimusNodeAdderPinProvider *AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(TargetNode);
	check(AdderPinProvider);

	if (OutActions)
	{
		*OutActions = AdderPinProvider->GetAvailableAdderPinActions(SourcePin, NewPinDirection, OutReason);
		return OutActions->Num() > 0;
	}
	
	const TArray<IOptimusNodeAdderPinProvider::FAdderPinAction> Actions = AdderPinProvider->GetAvailableAdderPinActions(SourcePin, NewPinDirection, OutReason);
	return Actions.Num() > 0;
}

UOptimusEditorGraphSchema::UOptimusEditorGraphSchema()
{

}


void UOptimusEditorGraphSchema::GetGraphActions(
	FGraphActionListBuilderBase& IoActionBuilder, 
	const UEdGraphPin* InFromPin, 
	const UEdGraph* InGraph
	) const
{
	// Basic Nodes
	for (UClass* Class : UOptimusNode::GetAllNodeClasses())
	{
		UOptimusNode* Node = Cast<UOptimusNode>(Class->GetDefaultObject());
		if (Node == nullptr)
		{
			continue;
		}

		const FText NodeName = Node->GetDisplayName();
		const FText NodeCategory = FText::FromName(Node->GetNodeCategory());

		TSharedPtr< FOptimusGraphSchemaAction_NewNode> Action(
			new FOptimusGraphSchemaAction_NewNode(
				NodeCategory,
				NodeName,
				/* Tooltip */{}, 0, /* Keywords */{}
		));

		Action->NodeClass = Class;

		IoActionBuilder.AddAction(Action);
	}

	// Constant Value Nodes
	for (FOptimusDataTypeHandle DataTypeHandle: FOptimusDataTypeRegistry::Get().GetAllTypes())
	{
		// For now only allow variable/resource compatible types to spawn constant nodes
		if (DataTypeHandle->CanCreateProperty() &&
			EnumHasAnyFlags(DataTypeHandle->UsageFlags, EOptimusDataTypeUsageFlags::Resource | EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::Property))
		{
			const FText NodeName = FText::Format(LOCTEXT("ConstantValueNode", "{0} Constant"), DataTypeHandle->DisplayName);
			const FText NodeCategory = FText::FromName(UOptimusNode::CategoryName::Values);
			
			TSharedPtr< FOptimusGraphSchemaAction_NewConstantValueNode> Action(
				new FOptimusGraphSchemaAction_NewConstantValueNode(
					NodeCategory,
					NodeName,
					/* Tooltip */{}, 0, /* Keywords */{}
			));

			Action->DataType = DataTypeHandle;

			IoActionBuilder.AddAction(Action);
		}
	}

	// Data Interface Nodes
	for (UClass* Class : UOptimusComputeDataInterface::GetAllComputeDataInterfaceClasses())
	{
		UOptimusComputeDataInterface* DataInterface = Cast<UOptimusComputeDataInterface>(Class->GetDefaultObject());
		if (!ensure(DataInterface != nullptr))
		{
			continue;
		}

		const FText NodeName = FText::FromString(DataInterface->GetDisplayName());

		const FText NodeCategory = FText::FromName(DataInterface->GetCategory());

		TSharedPtr< FOptimusGraphSchemaAction_NewDataInterfaceNode> Action(
			new FOptimusGraphSchemaAction_NewDataInterfaceNode(
				NodeCategory,
				NodeName,
				/* Tooltip */{}, 0, /* Keywords */{}
		));

		Action->DataInterfaceClass = Class;

		IoActionBuilder.AddAction(Action);
	}

	// Private functions
	{
		if (const UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InGraph))
		{
			if (UOptimusNodeGraph* ModelGraph = Graph->GetModelGraph())
			{
				if (UOptimusDeformer* Deformer = Cast<UOptimusDeformer>(ModelGraph->GetCollectionRoot()))
				{
					for (UOptimusFunctionNodeGraph* FunctionNodeGraph : Deformer->GetFunctionGraphs(UOptimusFunctionNodeGraph::AccessSpecifierPrivateName))
					{
						FOptimusFunctionNodeGraphHeaderWithGuid Header = FunctionNodeGraph->GetHeaderWithGuid();
				
						TSharedPtr< FOptimusGraphSchemaAction_NewFunctionReferenceNode> Action(
									new FOptimusGraphSchemaAction_NewFunctionReferenceNode(
										FText::FromName(Header.Category),
										FText::FromName(Header.FunctionName),
										/* Tooltip */{}, 0, /* Keywords */{}
								));

						Action->AssetPath = Deformer;
						Action->FunctionGraphGuid = Header.FunctionGraphGuid;

						IoActionBuilder.AddAction(Action);
					}
				}	
			}
		}
	}
	
	// Public functions
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		TArray<FAssetData> AssetDatas;

		// This triggers the gathering of asset tags across assets
		AssetRegistryModule.Get().GetAssetsByClass(UOptimusDeformer::StaticClass()->GetClassPathName(), AssetDatas);

		auto AddPublicFunctionAction = [&IoActionBuilder](const FAssetData& AssetData, const FOptimusFunctionNodeGraphHeaderWithGuid& Header)
		{
			const FText FunctionName = FText::FromName(Header.FunctionName);

			const FText Category = FText::FromName(Header.Category);

			TSharedPtr< FOptimusGraphSchemaAction_NewFunctionReferenceNode> Action(
				new FOptimusGraphSchemaAction_NewFunctionReferenceNode(
					Category,
					FunctionName,
					/* Tooltip */{}, 0, /* Keywords */{}
			));

			Action->AssetPath = AssetData.GetSoftObjectPath();
			Action->FunctionGraphGuid = Header.FunctionGraphGuid;

			IoActionBuilder.AddAction(Action);	
		};
		
		for (const FAssetData& AssetData : AssetDatas)
		{
			bool bPublicFunctionImported = false;
			// New public function exports
			{
				FString PublicFunctionString = AssetData.GetTagValueRef<FString>(UOptimusDeformer::PublicFunctionsWithGuidAssetTagName);

				if (!PublicFunctionString.IsEmpty())
				{
					bPublicFunctionImported = true;
					FOptimusFunctionNodeGraphHeaderWithGuidArray PublicFunctionHeaderArray;

					FOptimusFunctionNodeGraphHeaderWithGuidArray::StaticStruct()->ImportText(*PublicFunctionString, &PublicFunctionHeaderArray, nullptr, PPF_None, nullptr, {});

					for (const FOptimusFunctionNodeGraphHeaderWithGuid& Header : PublicFunctionHeaderArray.Headers)
					{
						AddPublicFunctionAction(AssetData, Header);	
					}
				}	
			}
			
			// Old public function exports
			if (!bPublicFunctionImported)
			{
				FString PublicFunctionString = AssetData.GetTagValueRef<FString>(UOptimusDeformer::PublicFunctionsAssetTagName);

				if (!PublicFunctionString.IsEmpty())
				{
					bPublicFunctionImported = true;
					FOptimusFunctionNodeGraphHeaderArray PublicFunctionHeaderArray;

					FOptimusFunctionNodeGraphHeaderArray::StaticStruct()->ImportText(*PublicFunctionString, &PublicFunctionHeaderArray, nullptr, PPF_None, nullptr, {});

					for (const FOptimusFunctionNodeGraphHeader& Header : PublicFunctionHeaderArray.Headers)
					{

						// Unfortunately we need to do a load here because we need to grab the latest Guid from the function graph
						if (UOptimusFunctionNodeGraph* FunctionNodeGraph = Header.GraphPath.LoadSynchronous())
						{
							FOptimusFunctionNodeGraphHeaderWithGuid HeaderWithGuid = FunctionNodeGraph->GetHeaderWithGuid();
							AddPublicFunctionAction(AssetData, HeaderWithGuid);
						}
					}
				}	
			}

			
		}	
	}
	
	// Loop nodes
	{
		TSharedPtr< FOptimusGraphSchemaAction_NewLoopTerminalNodes> Action(
			new FOptimusGraphSchemaAction_NewLoopTerminalNodes(
				{},
				FText::FromName(TEXT("Loop")),
				/* Tooltip */{}, 0, /* Keywords */{}
		));
		
		IoActionBuilder.AddAction(Action);
	}
	
	// Comment nodes
	{
		TSharedPtr< FOptimusGraphSchemaAction_NewCommentNode> Action(
			new FOptimusGraphSchemaAction_NewCommentNode(
				{},
				FText::FromName(TEXT("Comment")),
				/* Tooltip */{}, 0, /* Keywords */{}
		));
		
		IoActionBuilder.AddAction(Action);
	}	
}


bool UOptimusEditorGraphSchema::TryCreateConnection(
	UEdGraphPin* InPinA, 
	UEdGraphPin* InPinB) const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif
	
	if (InPinA->Direction == EGPD_Input)
	{
		Swap(InPinA, InPinB);
	}

	// The pins should be in the correct order now.
	UOptimusNodePin *OutputModelPin = OptimusEditor::GetModelPinFromGraphPin(InPinA);
	UOptimusNodePin* InputModelPin = OptimusEditor::GetModelPinFromGraphPin(InPinB);

	if (OutputModelPin && InputModelPin)
	{
		if (!OutputModelPin->CanCannect(InputModelPin))
		{
			return false;
		}

		UOptimusNodeGraph *Graph = OutputModelPin->GetOwningNode()->GetOwningGraph();
		return Graph->AddLink(OutputModelPin, InputModelPin);
	}

	// Pins might be connectable if one of the two pins is an adder pin;
	if (OptimusEditor::IsAdderPin(InPinA) || OptimusEditor::IsAdderPin(InPinB))
	{
		TArray<IOptimusNodeAdderPinProvider::FAdderPinAction> AvailableActions;
		const bool bCanConnect = IsValidAdderPinConnection(InPinA, InPinB, &AvailableActions);

		if (!bCanConnect)
		{
			return false;
		}

		UOptimusNode* TargetNode = OptimusEditor::GetModelNodeFromGraphPin(InPinB);
		UOptimusNodePin* SourcePin = OutputModelPin;

		// Flip Source/Target if the Adder Pin is on the Output side 
		if (OptimusEditor::IsAdderPin(InPinA))
		{
			// Add new pin on the output pin's node
			TargetNode = OptimusEditor::GetModelNodeFromGraphPin(InPinA);
			SourcePin = InputModelPin;
		}

		IOptimusNodeAdderPinProvider *AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(TargetNode);
		UOptimusNodeGraph *Graph = TargetNode->GetOwningGraph();
		
		if (AvailableActions.Num() == 1)
		{
			return Graph->ConnectAdderPin(AdderPinProvider, AvailableActions[0], SourcePin);
		}

		FMenuBuilder MenuBuilder(true, NULL);

		MenuBuilder.BeginSection(NAME_None, FText::FromString(TEXT("Add Pin To Group")));

		for (const IOptimusNodeAdderPinProvider::FAdderPinAction& Action : AvailableActions)
		{
			FText EntryName;

			if (Action.bCanAutoLink)
			{
				EntryName = FText::FromString(Action.DisplayName.ToString() + TEXT(" * "));
			}
			else
			{
				EntryName = FText::FromName(Action.DisplayName);
			}
			
			MenuBuilder.AddMenuEntry(
				EntryName,
				Action.ToolTip,
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateLambda([AdderPinProvider, Graph, Action, SourcePin]()
					{
						Graph->ConnectAdderPin(AdderPinProvider, Action, SourcePin);
					}),
					FCanExecuteAction()
				));
		}

		MenuBuilder.EndSection();
		
		FSlateApplication::Get().PushMenu(
			FSlateApplication::Get().GetUserFocusedWidget(0).ToSharedRef(),
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu)
			);

		return false;
	}

	return false;
}


const FPinConnectionResponse UOptimusEditorGraphSchema::CanCreateConnection(
	const UEdGraphPin* InPinA, 
	const UEdGraphPin* InPinB
	) const
{
	if (InPinA->Direction == EGPD_Input)
	{
		Swap(InPinA, InPinB);
	}

	// The pins should be in the correct order now.
	UOptimusNodePin* OutputModelPin = OptimusEditor::GetModelPinFromGraphPin(InPinA);
	UOptimusNodePin* InputModelPin = OptimusEditor::GetModelPinFromGraphPin(InPinB);

	if (InputModelPin && OutputModelPin)
	{
		FString FailureReason;
		bool bCanConnect = OutputModelPin->CanCannect(InputModelPin, &FailureReason);

		return FPinConnectionResponse(
			bCanConnect ? CONNECT_RESPONSE_MAKE :  CONNECT_RESPONSE_DISALLOW,
			FText::FromString(FailureReason));
	}
	
	// Pins might be connectable if one of the two pins is an adder pin;
	if (OptimusEditor::IsAdderPin(InPinA) || OptimusEditor::IsAdderPin(InPinB))
	{
		FString FailureReason;
		const bool bCanConnect = IsValidAdderPinConnection(InPinA, InPinB, nullptr, &FailureReason);

		return FPinConnectionResponse(
			bCanConnect ? CONNECT_RESPONSE_MAKE :  CONNECT_RESPONSE_DISALLOW,
			FText::FromString(FailureReason));
	}
	
	return FPinConnectionResponse( CONNECT_RESPONSE_DISALLOW,FText::GetEmpty());
}


void UOptimusEditorGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	UOptimusEditorGraphNode* GraphNode = Cast<UOptimusEditorGraphNode>(TargetPin.GetOwningNode());
	UOptimusEditorGraph* EditorGraph = Cast<UOptimusEditorGraph>(GraphNode->GetGraph());

	if (ensure(EditorGraph))
	{
		UOptimusNodeGraph* ModelGraph = EditorGraph->GetModelGraph();

		UOptimusNodePin *TargetModelPin = GraphNode->FindModelPinFromGraphPin(&TargetPin);

		if (ensure(TargetModelPin))
		{
			ModelGraph->RemoveAllLinks(TargetModelPin);
		}
	}
}


void UOptimusEditorGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{

}


FConnectionDrawingPolicy* UOptimusEditorGraphSchema::CreateConnectionDrawingPolicy(
	int32 InBackLayerID,
	int32 InFrontLayerID,
	float InZoomFactor,
	const FSlateRect& InClippingRect,
	FSlateWindowElementList& InDrawElements,
	UEdGraph* InGraphObj
	) const
{
	return new FOptimusEditorGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}


void UOptimusEditorGraphSchema::GetGraphContextActions(
	FGraphContextMenuBuilder& IoContextMenuBuilder
	) const
{
	GetGraphActions(IoContextMenuBuilder, IoContextMenuBuilder.FromPin, IoContextMenuBuilder.CurrentGraph);
}


bool UOptimusEditorGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* InGraph, UEdGraphNode* InNode) const
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InNode->GetGraph());

	return Graph->GetModelGraph()->RemoveNode(Graph->FindModelNodeFromGraphNode(InNode));
}

void UOptimusEditorGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, FGraphDisplayInfo& DisplayInfo) const
{
	const UOptimusEditorGraph& EditorGraph = static_cast<const UOptimusEditorGraph&>(Graph);
	DisplayInfo.PlainName = FText::FromString(EditorGraph.GetModelGraph()->GetName());
	DisplayInfo.DisplayName = DisplayInfo.PlainName;
}


FEdGraphPinType UOptimusEditorGraphSchema::GetPinTypeFromDataType(
	FOptimusDataTypeHandle InDataType
	)
{
	FEdGraphPinType PinType;

	if (InDataType.IsValid())
	{
		// Set the categories as defined by the registered data type. We hijack the PinSubCategory
		// so that we can query back to the registry for whether the pin color should come out of the
		// K2 schema or the registered custom color.
		PinType.PinCategory = InDataType->TypeCategory;
		PinType.PinSubCategory = InDataType->TypeName;
		PinType.PinSubCategoryObject = InDataType->TypeObject;
	}

	return PinType;
}



FLinearColor UOptimusEditorGraphSchema::GetPinTypeColor(
	const FEdGraphPinType& InPinType
	) const
{
	return GetColorFromPinType(InPinType);
}


const FSlateBrush* UOptimusEditorGraphSchema::GetIconFromPinType(
	const FEdGraphPinType& InPinType
	)
{
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
	const UClass *PinObjectClass = Cast<UClass>(InPinType.PinSubCategoryObject.Get());

	if (PinObjectClass)
	{
		IconBrush = FSlateIconFinder::FindIconBrushForClass(PinObjectClass);
	}

	return IconBrush;
}


FLinearColor UOptimusEditorGraphSchema::GetColorFromPinType(const FEdGraphPinType& InPinType)
{
	if (OptimusEditor::IsAdderPinType(InPinType))
	{
		const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
		return Settings->WildcardPinTypeColor;
	}
	
	if (InPinType.PinCategory == UOptimusEditorGraphNode::GroupTypeName)
	{
		// We allow direct connection from component source to group pin, thus use the same color for them
		FOptimusDataTypeHandle ComponentSourceType = FOptimusDataTypeRegistry::Get().FindType(*UOptimusComponentSourceBinding::StaticClass());

		if (ensure(ComponentSourceType->bHasCustomPinColor))
		{
			return ComponentSourceType->CustomPinColor;
		}
	}
	
	// Use the PinSubCategory value to resolve the type. It's set in
	// UOptimusEditorGraphSchema::GetPinTypeFromDataType.
	FOptimusDataTypeHandle DataType = FOptimusDataTypeRegistry::Get().FindType(InPinType.PinSubCategory);

	// Unset data types get black pins.
	if (!DataType.IsValid())
	{
		return FLinearColor::Black;
	}

	// If the data type has custom color, use that. Otherwise fall back on the K2 schema
	// since we want to be compatible with known types (which also have preferences for them).
	if (DataType->bHasCustomPinColor)
	{
		return DataType->CustomPinColor;
	}

	return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(InPinType);
}

FText UOptimusEditorGraphSchema::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if (UOptimusNodePin* ModelPin = OptimusEditor::GetModelPinFromGraphPin(Pin))
	{
		if (const IOptimusUnnamedNodePinProvider* UnnamedNodePinProvider = Cast<const IOptimusUnnamedNodePinProvider>(ModelPin->GetOwningNode()))
		{
			if (UnnamedNodePinProvider->IsPinNameHidden(ModelPin))
			{
				return FText::GetEmpty();
			}
		}
	}
	
	return Super::GetPinDisplayName(Pin);
}


void UOptimusEditorGraphSchema::TrySetDefaultValue(
	UEdGraphPin& Pin, 
	const FString& NewDefaultValue, 
	bool bMarkAsModified
	) const
{
	UOptimusNodePin* ModelPin = OptimusEditor::GetModelPinFromGraphPin(&Pin);
	if (ensure(ModelPin))
	{
		// Kill the existing transaction, since it copies the wrong node.
		GEditor->CancelTransaction(0);

		ModelPin->SetValueFromString(NewDefaultValue);
	}
}

#undef LOCTEXT_NAMESPACE
