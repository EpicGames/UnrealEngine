// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

#include "OptimusDataType.h"

#include "OptimusEditorGraphSchema.generated.h"

struct FOptimusDataType;
struct FSlateBrush;
class UOptimusNodeGraph;


UCLASS()
class UOptimusEditorGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()
public:

	static const FName GraphName_OptimusDeformer;

	UOptimusEditorGraphSchema();

	void GetGraphActions(
		FGraphActionListBuilderBase& IoActionBuilder,
		const UEdGraphPin *InFromPin,
		const UEdGraph* InGraph) const;


	static FEdGraphPinType GetPinTypeFromDataType(FOptimusDataTypeHandle InDataType);
	static const FSlateBrush *GetIconFromPinType(const FEdGraphPinType& InPinType);
	static FLinearColor GetColorFromPinType(const FEdGraphPinType& InPinType);

	// UEdGraphSchema overrides
	FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	void GetGraphContextActions(FGraphContextMenuBuilder& IoContextMenuBuilder) const override;
	bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const override;

	void GetGraphDisplayInformation(const UEdGraph& Graph, FGraphDisplayInfo& DisplayInfo) const override;

	void TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified = true) const override;

	FLinearColor GetPinTypeColor(const FEdGraphPinType& InPinType) const override;

	// Don't re-create the entire graph on node add/remove.
	bool ShouldAlwaysPurgeOnModification() const override { return false; }	
};
