// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorGraphSchemaActions.h"

#include "OptimusComponentSource.h"
#include "OptimusDeformer.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphNode.h"
#include "OptimusEditorHelpers.h"
#include "OptimusFunctionNodeGraph.h"

#include "OptimusNode.h"
#include "OptimusResourceDescription.h"
#include "OptimusVariableDescription.h"
#include "OptimusNodeGraph.h"

#include "EdGraph/EdGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusEditorGraphSchemaActions)


UEdGraphNode* FOptimusGraphSchemaAction_NewNode::PerformAction(
	UEdGraph* InParentGraph, 
	UEdGraphPin* InFromPin, 
	const FVector2f& InLocation, 
	bool bInSelectNewNode /*= true*/
	)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr) && ensure(NodeClass != nullptr))
	{
		UOptimusNode* ModelNode = Graph->GetModelGraph()->AddNode(NodeClass, FDeprecateSlateVector2D(InLocation));

 		// FIXME: Automatic connection from the given pin.

		UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(ModelNode);
		if (GraphNode && bInSelectNewNode)
		{
			Graph->SelectNodeSet({GraphNode});
		}
		return GraphNode;
	}

	return nullptr;
}


UEdGraphNode* FOptimusGraphSchemaAction_NewConstantValueNode::PerformAction(
	UEdGraph* InParentGraph,
	UEdGraphPin* InFromPin,
	const FVector2f& InLocation,
	bool bInSelectNewNode
	)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr) && ensure(DataType.IsValid()))
	{
		UOptimusNode* ModelNode = Graph->GetModelGraph()->AddValueNode(DataType, FDeprecateSlateVector2D(InLocation));

		// FIXME: Automatic connection from the given pin.

		UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(ModelNode);
		if (GraphNode && bInSelectNewNode)
		{
			Graph->SelectNodeSet({GraphNode});
		}
		return GraphNode;
	}

	return nullptr;
}


UEdGraphNode* FOptimusGraphSchemaAction_NewDataInterfaceNode::PerformAction(
	UEdGraph* InParentGraph,
	UEdGraphPin* InFromPin,
	const FVector2f& InLocation,
	bool bInSelectNewNode
	)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr) && ensure(DataInterfaceClass != nullptr))
	{
		UOptimusNode* ModelNode = Graph->GetModelGraph()->AddDataInterfaceNode(DataInterfaceClass, FDeprecateSlateVector2D(InLocation));

		// FIXME: Automatic connection from the given pin.

		UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(ModelNode);
		if (GraphNode && bInSelectNewNode)
		{
			Graph->SelectNodeSet({GraphNode});
		}
		return GraphNode;
	}

	return nullptr;
}

UEdGraphNode* FOptimusGraphSchemaAction_NewLoopTerminalNodes::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* FromPin,
	const FVector2f& Location, bool bInSelectNewNode)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr))
	{
		TArray<UOptimusNode*> Nodes = Graph->GetModelGraph()->AddLoopTerminalNodes(FDeprecateSlateVector2D(Location));

		if (ensure(Nodes.Num() == 2))
		{
			UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(Nodes[0]);
			if (GraphNode && bInSelectNewNode)
			{
				Graph->SelectNodeSet({GraphNode});
			}
			return GraphNode;
		}
	}

	return nullptr;
}

UEdGraphNode* FOptimusGraphSchemaAction_NewCommentNode::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* FromPin, const FVector2f& Location,
	bool bInSelectNewNode)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr))
	{
		UOptimusNode* ModelNode = OptimusEditor::CreateCommentNode(Graph, FDeprecateSlateVector2D(Location));

		UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(ModelNode);
		if (GraphNode && bInSelectNewNode)
		{
			Graph->SelectNodeSet({GraphNode});
		}
		return GraphNode;	
	}

	return nullptr;
}

UEdGraphNode* FOptimusGraphSchemaAction_NewFunctionReferenceNode::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InFromPin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(InParentGraph);
	
	if (ensure(Graph != nullptr))
	{
		if (UOptimusDeformer* Deformer = AssetPath.LoadSynchronous())
		{
			if (UOptimusFunctionNodeGraph* FunctionGraph = Deformer->FindFunctionByGuid(FunctionGraphGuid))
			{
				UOptimusNode* ModelNode = Graph->GetModelGraph()->AddFunctionReferenceNode(FunctionGraph, FDeprecateSlateVector2D(InLocation));

				// FIXME: Automatic connection from the given pin.

				UEdGraphNode* GraphNode = Graph->FindGraphNodeFromModelNode(ModelNode);
				if (GraphNode && bInSelectNewNode)
				{
					Graph->SelectNodeSet({GraphNode});
				}
				return GraphNode;	
			}
		}
	
	}

	return nullptr;
}


static FText GetGraphTooltip(UOptimusNodeGraph* InGraph)
{
	return FText::GetEmpty();
}


FOptimusSchemaAction_Graph::FOptimusSchemaAction_Graph(
	UOptimusNodeGraph* InGraph,
	const FText& InCategory) : 
		FEdGraphSchemaAction(
			InCategory, 
			FText::FromString(InGraph->GetName()), 
			GetGraphTooltip(InGraph), 
			0, 
			FText(), 
			int32(EOptimusSchemaItemGroup::Graphs) 
		), 
		GraphType(InGraph->GetGraphType())
{
	GraphPath = InGraph->GetCollectionPath();
}


FOptimusSchemaAction_Binding::FOptimusSchemaAction_Binding(
	UOptimusComponentSourceBinding* InBinding
	) :
	FEdGraphSchemaAction(
			FText::GetEmpty(),
			FText::FromString(InBinding->GetName()),
			FText::GetEmpty(),
			0,
			FText(),
			int32(EOptimusSchemaItemGroup::Bindings)
		)

{
	BindingName = InBinding->GetFName();
}


FOptimusSchemaAction_Resource::FOptimusSchemaAction_Resource(
	UOptimusResourceDescription* InResource
	) :
	FEdGraphSchemaAction(
			FText::GetEmpty(),
			FText::FromString(InResource->GetName()),
			FText::GetEmpty(),
			0,
			FText(),
			int32(EOptimusSchemaItemGroup::Resources)
		)
{
	ResourceName = InResource->GetFName();
}


FOptimusSchemaAction_Variable::FOptimusSchemaAction_Variable(
	UOptimusVariableDescription* InVariable 
	) : 
	FEdGraphSchemaAction(
          FText::GetEmpty(),
          FText::FromString(InVariable->GetName()),
          FText::GetEmpty(),
          0,
          FText(),
          int32(EOptimusSchemaItemGroup::Variables))
{
	VariableName = InVariable->GetFName();
}
