// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "Templates/SubclassOf.h"

#include "OptimusComputeDataInterface.h"
#include "OptimusDataType.h"
#include "OptimusFunctionNodeGraph.h"
#include "OptimusFunctionNodeGraphHeader.h"

#include "OptimusEditorGraphSchemaActions.generated.h"


class UOptimusComponentSourceBinding;
class UOptimusNode;
class UOptimusNodeGraph;
class UOptimusResourceDescription;
class UOptimusVariableDescription;
enum class EOptimusNodeGraphType;


enum class EOptimusSchemaItemGroup
{
	InvalidGroup = 0,
	Graphs,
	Bindings,
	Variables,
	Resources,
};


/// Action to add a new Optimus node to the graph
USTRUCT()
struct FOptimusGraphSchemaAction_NewNode : 
	public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	UPROPERTY()
	TSubclassOf<UOptimusNode> NodeClass;

	static FName StaticGetTypeId() { static FName Type("FOptimusDeformerGraphSchemaAction_NewNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
};

USTRUCT()
struct FOptimusGraphSchemaAction_NewConstantValueNode : 
	public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	UPROPERTY()
	FOptimusDataTypeRef DataType;

	static FName StaticGetTypeId() { static FName Type("FOptimusGraphSchemaAction_NewConstantValueNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
};

USTRUCT()
struct FOptimusGraphSchemaAction_NewDataInterfaceNode : 
	public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	UPROPERTY()
	TSubclassOf<UOptimusComputeDataInterface> DataInterfaceClass;

	static FName StaticGetTypeId() { static FName Type("FOptimusGraphSchemaAction_NewDataInterfaceNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bInSelectNewNode = true) override;
};


USTRUCT()
struct FOptimusGraphSchemaAction_NewLoopTerminalNodes : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;
	
	static FName StaticGetTypeId() { static FName Type("FOptimusGraphSchemaAction_LoopTerminal"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	UEdGraphNode* PerformAction(class UEdGraph* InParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bInSelectNewNode = true) override;
};

USTRUCT()
struct FOptimusGraphSchemaAction_NewCommentNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;
	
	static FName StaticGetTypeId() { static FName Type("FOptimusGraphSchemaAction_CommentNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	using FEdGraphSchemaAction::PerformAction;
	UEdGraphNode* PerformAction(class UEdGraph* InParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bInSelectNewNode = true) override;
};

USTRUCT()
struct FOptimusGraphSchemaAction_NewFunctionReferenceNode: 
	public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	TSoftObjectPtr<UOptimusDeformer> AssetPath;

	FGuid FunctionGraphGuid;

	static FName StaticGetTypeId() { static FName Type("FOptimusGraphSchemaAction_NewFunctionReferenceNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	UEdGraphNode* PerformAction(class UEdGraph* InParentGraph, UEdGraphPin* InFromPin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
};

/// Reference to a UOptimusNodeGraph.
USTRUCT()
struct FOptimusSchemaAction_Graph : 
	public FEdGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId()
	{
		static FName Type("FOptimusSchemaAction_Graph");
		return Type;
	}
	FName GetTypeId() const override { return StaticGetTypeId(); }

	FString GraphPath;

	EOptimusNodeGraphType GraphType;

	FOptimusSchemaAction_Graph() = default;

	FOptimusSchemaAction_Graph(
	    UOptimusNodeGraph* InGraph,
	    const FText& InCategory = FText::GetEmpty());

	// FEdGraphSchemaAction overrides
	bool IsParentable() const override { return true; }
};


/// Reference to a UOptimusComponentSourceBinding.
USTRUCT()
struct FOptimusSchemaAction_Binding : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId()
	{
		static FName Type("FOptimusSchemaAction_Binding");
		return Type;
	}
	FName GetTypeId() const override { return StaticGetTypeId(); }

	FName BindingName;

	FOptimusSchemaAction_Binding() = default;

	FOptimusSchemaAction_Binding( UOptimusComponentSourceBinding* InBinding );
	// FEdGraphSchemaAction overrides
	bool IsParentable() const override { return false; }
};


/// Reference to a UOptimusResourceDescription.
USTRUCT()
struct FOptimusSchemaAction_Resource : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId()
	{
		static FName Type("FOptimusSchemaAction_Resource");
		return Type;
	}
	FName GetTypeId() const override { return StaticGetTypeId(); }

	FName ResourceName;

	FOptimusSchemaAction_Resource() = default;

	FOptimusSchemaAction_Resource( UOptimusResourceDescription* InResource );

	// FEdGraphSchemaAction overrides
	bool IsParentable() const override { return false; }
};


/// Reference to a UOptimusVariableDescription.
USTRUCT()
struct FOptimusSchemaAction_Variable : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	static FName StaticGetTypeId()
	{
		static FName Type("FOptimusSchemaAction_Variable");
		return Type;
	}
	FName GetTypeId() const override { return StaticGetTypeId(); }

	FName VariableName;

	FOptimusSchemaAction_Variable() = default;

	FOptimusSchemaAction_Variable( UOptimusVariableDescription* InVariable );

	// FEdGraphSchemaAction overrides
	bool IsParentable() const override { return false; }
};
