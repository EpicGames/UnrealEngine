// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorHelpers.h"

#include "GraphEditorSettings.h"
#include "OptimusEditorGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphNode.h"

#include "OptimusNode.h"
#include "OptimusNodePin.h"
#include "OptimusNodeGraph.h"
#include "Layout/SlateRect.h"

UOptimusNode* OptimusEditor::GetModelNodeFromGraphPin(const UEdGraphPin* InGraphPin)
{
	return FindModelNodeFromGraphNode(InGraphPin->GetOwningNode());
}

UOptimusNodePin* OptimusEditor::GetModelPinFromGraphPin(const UEdGraphPin* InGraphPin)
{
	const UOptimusNode* ModelNode = GetModelNodeFromGraphPin(InGraphPin);
	
	if (ensure(ModelNode))
	{
		return ModelNode->FindPin(InGraphPin->GetName());
	}

	return nullptr;
}

UOptimusNode* OptimusEditor::FindModelNodeFromGraphNode(const UEdGraphNode* InGraphNode)
{
	if (!InGraphNode)
	{
		return nullptr;
	}
	
	UOptimusEditorGraph* EditorGraph = Cast<UOptimusEditorGraph>(InGraphNode->GetGraph());
	if (!ensure(EditorGraph))
	{
		return nullptr;
	}

	return EditorGraph->FindModelNodeFromGraphNode(InGraphNode);
}

FName OptimusEditor::GetAdderPinName(EEdGraphPinDirection InDirection)
{
	return InDirection == EGPD_Input ? TEXT("_AdderPinInput") : TEXT("_AdderPinOutput");
};
	
FText OptimusEditor::GetAdderPinFriendlyName(EEdGraphPinDirection InDirection)
{
	return InDirection == EGPD_Input ? FText::FromString(TEXT("New Input")) : FText::FromString(TEXT("New Output"));
};

FName OptimusEditor::GetAdderPinCategoryName()
{
	return TEXT("OptimusAdderPin");
}

bool OptimusEditor::IsAdderPin(const UEdGraphPin* InGraphPin)
{
	if (InGraphPin->PinType.PinCategory == GetAdderPinCategoryName())
	{
		return true;
	}
		
	return false;
};

bool OptimusEditor::IsAdderPinType(const FEdGraphPinType& InPinType)
{
	if (InPinType.PinCategory == GetAdderPinCategoryName())
	{
		return true;
	}
	return false;
}

UOptimusNode* OptimusEditor::CreateCommentNode(UOptimusEditorGraph* InEditorGraph, const FVector2D& InPosition)
{
	const FVector2D DefaultSize(400, 100);
	FSlateRect Rect;
	if (!InEditorGraph->GetBoundsForSelectedNodes(Rect))
	{
		Rect = FSlateRect(InPosition.X, InPosition.Y, InPosition.X+DefaultSize.X, InPosition.Y+DefaultSize.Y);
	}

	const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();
	constexpr bool bCreatedFromUI = true;
	return InEditorGraph->GetModelGraph()->AddCommentNode(Rect.GetTopLeft(), Rect.GetSize(), GraphEditorSettings->DefaultCommentNodeTitleColor, bCreatedFromUI);	
}


