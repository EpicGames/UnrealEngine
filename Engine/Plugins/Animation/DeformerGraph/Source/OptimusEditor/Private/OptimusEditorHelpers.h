// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class UEdGraphPin;
class UEdGraphNode;
class UOptimusNode;
class UOptimusNodePin;
struct FEdGraphPinType;
class UOptimusEditorGraph;
enum EEdGraphPinDirection : int;

namespace OptimusEditor
{
	UOptimusNode* GetModelNodeFromGraphPin(const UEdGraphPin* InGraphPin);

	UOptimusNodePin* GetModelPinFromGraphPin(const UEdGraphPin* InGraphPin);

	UOptimusNode* FindModelNodeFromGraphNode(const UEdGraphNode* InGraphNode);

	FName GetAdderPinName(EEdGraphPinDirection InDirection);
		
	FText GetAdderPinFriendlyName(EEdGraphPinDirection InDirection);

	FName GetAdderPinCategoryName();

	bool IsAdderPin(const UEdGraphPin* InGraphPin);

	bool IsAdderPinType(const FEdGraphPinType& InPinType);

	UOptimusNode* CreateCommentNode(UOptimusEditorGraph* InGraph, const FVector2D& InPosition);
}
