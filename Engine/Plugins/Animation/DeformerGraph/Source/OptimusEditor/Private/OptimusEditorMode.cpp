// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorMode.h"

#include "OptimusEditor.h"
#include "OptimusEditorTabSummoners.h"

#include "IPersonaToolkit.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "PersonaTabs.h"


FName FOptimusEditorMode::ModeId("OptimusEditorMode");

FOptimusEditorMode::FOptimusEditorMode(TSharedRef<FOptimusEditor> InEditorApp) :
	FApplicationMode(ModeId),
	EditorPtr(InEditorApp)
{
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorNodePaletteTabSummoner>(InEditorApp));
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorExplorerTabSummoner>(InEditorApp));
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorGraphTabSummoner>(InEditorApp));
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorFindTabSummoner>(InEditorApp));
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorCompilerOutputTabSummoner>(InEditorApp));
	TabFactories.RegisterFactory(MakeShared<FOptimusEditorShaderTextEditorTabSummoner>(InEditorApp));
	
	TSharedRef<FWorkflowCentricApplication> WorkflowCentricApp = StaticCastSharedRef<FWorkflowCentricApplication>(InEditorApp);
	
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InEditorApp, FOnDetailsCreated::CreateSP(&InEditorApp.Get(), &FOptimusEditor::HandleDetailsCreated)));

	FPersonaViewportArgs ViewportArgs(InEditorApp->GetPersonaToolkit()->GetPreviewScene());
	ViewportArgs.ContextName = TEXT("OptimusEditor.Viewport");
	ViewportArgs.bShowStats = true;
	ViewportArgs.bShowPlaySpeedMenu = false;
	ViewportArgs.bShowTimeline = true;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(InEditorApp, &FOptimusEditor::HandleViewportCreated);

	PersonaModule.RegisterPersonaViewportTabFactories(TabFactories, InEditorApp, ViewportArgs);
	
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InEditorApp, InEditorApp->GetPersonaToolkit()->GetPreviewScene()));
	
	TabLayout = CreatePaneLayout();
}

TSharedRef<FTabManager::FLayout> FOptimusEditorMode::CreatePaneLayout()
{
	// The default layout looks like so:
	// 
	// +-----------------------------------------+
	// |                Toolbar                  |
	// +-----+---------------------------+-------+
	// |     |                           |       |
	// | Pre |                           | Deets |
	// |     |                           |       |
	// +-----+          Graph            +-------+
	// |     |                           |       |
	// | Pex +---------------------------+ Text  |
	// |     |          Output           |       |
	// +-----+---------------------------+-------+
	//
	// Pre = 3D Preview 
	// Pex = Node Palette/explorer
	// Deets = Details panel
	// Text = Shader Text Editor

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("Standalone_OptimusEditor_Layout_v07")
		->AddArea(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split(							// - Main work area
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split(						// -- Preview + palette
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.3f)
					->Split(					// --- Preview widget
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->AddTab(TEXT("Viewport"), ETabState::OpenedTab)
					)
					->Split(					// --- Node palette
						FTabManager::NewStack()
						->AddTab(FOptimusEditorNodePaletteTabSummoner::TabId, ETabState::OpenedTab)
						->AddTab(FOptimusEditorExplorerTabSummoner::TabId, ETabState::OpenedTab)
						->SetForegroundTab(FOptimusEditorNodePaletteTabSummoner::TabId)
					)
				)
				->Split(						// -- Graph + output
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.8f)
					->Split(					// --- Graph
						FTabManager::NewStack()->SetSizeCoefficient(0.8f)
						->SetHideTabWell(true)
						->AddTab(FOptimusEditorGraphTabSummoner::TabId, ETabState::OpenedTab)
					)
					->Split(					// --- Output
						FTabManager::NewStack()->SetSizeCoefficient(0.2f)
						->AddTab(FOptimusEditorCompilerOutputTabSummoner::TabId, ETabState::OpenedTab)
						->AddTab(FOptimusEditorFindTabSummoner::TabId, ETabState::OpenedTab)
						->SetForegroundTab(FOptimusEditorCompilerOutputTabSummoner::TabId)
					)
				)
				->Split(						// -- Details + Shader Text Editor
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.2f)
					->Split(                   // --- Details 
						FTabManager::NewStack()
						->AddTab(FPersonaTabs::DetailsID, ETabState::OpenedTab)
						->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
						->SetForegroundTab(FPersonaTabs::DetailsID)
					)
					->Split(					// ---  Shader Text Editor
						FTabManager::NewStack()
						->AddTab(FOptimusEditorShaderTextEditorTabSummoner::TabId, ETabState::ClosedTab)
					)
				)
			)
		);

	return Layout;
}


void FOptimusEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FWorkflowCentricApplication> HostingApp = EditorPtr.Pin();

	if (HostingApp)
	{
		// FIXME: For when we want to add toolbox support. 
		// HostingApp->RegisterTabSpawners(InTabManager.ToSharedRef());
		HostingApp->PushTabFactories(TabFactories);
	}
}
