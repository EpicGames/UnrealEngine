// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FOptimusEditor;

class FOptimusEditorMode : public FApplicationMode
{
public:
	static FName ModeId;

	FOptimusEditorMode(TSharedRef<FOptimusEditor> InEditorApp);

	// FApplicationMode overrides
	void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	static TSharedRef<FTabManager::FLayout> CreatePaneLayout();
	
	FWorkflowAllowedTabSet TabFactories;

private:
	TWeakPtr<FOptimusEditor> EditorPtr;
};
