// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "EdGraphUtilities.h"
#include "IAssetTools.h"
#include "OptimusExecutionDomain.h"
#include "OptimusBindingTypes.h"
#include "OptimusComponentSource.h"
#include "OptimusDataType.h"
#include "OptimusDataTypeRegistry.h"
#include "OptimusDeformerAssetActions.h"
#include "OptimusDeformerInstance.h"
#include "OptimusDetailsCustomization.h"
#include "OptimusEditor.h"
#include "OptimusEditorClipboard.h"
#include "OptimusEditorCommands.h"
#include "OptimusEditorGraphCommands.h"
#include "OptimusEditorGraphNodeFactory.h"
#include "OptimusEditorGraphPinFactory.h"
#include "OptimusEditorStyle.h"
#include "OptimusHelpers.h"
#include "OptimusResourceDescription.h"
#include "OptimusShaderText.h"
#include "OptimusSource.h"
#include "OptimusSourceAssetActions.h"
#include "OptimusValidatedName.h"
#include "OptimusValueContainer.h"
#include "OptimusValueContainerStruct.h"
#include "PropertyEditorModule.h"
#include "UObject/FieldIterator.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "UserDefinedStructureCompilerUtils.h"
#include "Widgets/SOptimusEditorGraphExplorer.h"
#include "Widgets/SOptimusShaderTextDocumentTextBox.h"

#define LOCTEXT_NAMESPACE "OptimusEditorModule"

DEFINE_LOG_CATEGORY(LogOptimusEditor);

FOptimusEditorModule::FOptimusEditorModule() :
	Clipboard(MakeShared<FOptimusEditorClipboard>())
{
}

void FOptimusEditorModule::StartupModule()
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	
	TSharedRef<IAssetTypeActions> OptimusDeformerAssetAction = MakeShared<FOptimusDeformerAssetActions>();
	AssetTools.RegisterAssetTypeActions(OptimusDeformerAssetAction);
	RegisteredAssetTypeActions.Add(OptimusDeformerAssetAction);

	TSharedRef<IAssetTypeActions> OptimusSourceAssetAction = MakeShared<FOptimusSourceAssetActions>();
	AssetTools.RegisterAssetTypeActions(OptimusSourceAssetAction);
	RegisteredAssetTypeActions.Add(OptimusSourceAssetAction);

	FOptimusEditorCommands::Register();
	FOptimusEditorGraphCommands::Register();
	FOptimusEditorGraphExplorerCommands::Register();
	FOptimusShaderTextEditorDocumentTextBoxCommands::Register();
	FOptimusEditorStyle::Register();

	GraphNodeFactory = MakeShared<FOptimusEditorGraphNodeFactory>();
	FEdGraphUtilities::RegisterVisualNodeFactory(GraphNodeFactory);

	GraphPinFactory = MakeShared<FOptimusEditorGraphPinFactory>();
	FEdGraphUtilities::RegisterVisualPinFactory(GraphPinFactory);

	RegisterPropertyCustomizations();
}

void FOptimusEditorModule::ShutdownModule()
{
	UnregisterPropertyCustomizations();

	FEdGraphUtilities::UnregisterVisualPinFactory(GraphPinFactory);
	FEdGraphUtilities::UnregisterVisualNodeFactory(GraphNodeFactory);

	FOptimusEditorStyle::Unregister();
	FOptimusEditorGraphExplorerCommands::Unregister();
	FOptimusEditorGraphCommands::Unregister();
	FOptimusEditorCommands::Unregister();
	
	if (FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		IAssetTools& AssetTools = AssetToolsModule->Get();

		for (const TSharedRef<IAssetTypeActions>& Action : RegisteredAssetTypeActions)
		{
			AssetTools.UnregisterAssetTypeActions(Action);
		}
	}
}

TSharedRef<IOptimusEditor> FOptimusEditorModule::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UOptimusDeformer* DeformerObject)
{
	TSharedRef<FOptimusEditor> OptimusEditor = MakeShared<FOptimusEditor>();
	OptimusEditor->Construct(Mode, InitToolkitHost, DeformerObject);
	return OptimusEditor;
}

FOptimusEditorClipboard& FOptimusEditorModule::GetClipboard() const
{
	return Clipboard.Get();
}

void FOptimusEditorModule::PreChange(const UUserDefinedStruct* Changed,
	FStructureEditorUtils::EStructureEditorChangeInfo ChangedType)
{
	// the following is similar to
	// FUserDefinedStructureCompilerInner::ReplaceStructWithTempDuplicate()
	// it is necessary since things like optimus value container class/objects can reference the old struct
	// and we need to keep the old struct alive in those classes so that they can get GCed properly.
	
	UUserDefinedStruct* StructureToReinstance = (UUserDefinedStruct*)Changed;

	FUserDefinedStructureCompilerUtils::ReplaceStructWithTempDuplicateByPredicate(
		StructureToReinstance,
		[](FStructProperty* InStructProperty){ return Cast<UOptimusValueContainerGeneratorClass>(InStructProperty->GetOwnerClass()) != nullptr; },
		[](UStruct*){});
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Search for referencing packages to the currently changing user defined struct 
	TArray<FAssetIdentifier> Referencers;
	AssetRegistry.GetReferencers(Changed->GetPackage()->GetFName(), Referencers);
	for (const FAssetIdentifier& Identifier : Referencers)
	{
		TArray<FAssetData> Assets;
		AssetRegistry.GetAssetsByPackageName(Identifier.PackageName, Assets);

		for (const FAssetData& Asset : Assets)
		{
			
			if (Asset.IsInstanceOf(UOptimusDeformer::StaticClass()))
			{
				// Only care about loaded assets, which may have active instances
				if (UOptimusDeformer* DeformerAsset = Cast<UOptimusDeformer>(Asset.FastGetAsset(false)))
				{
					DeformerAsset->SetAllInstancesCanbeActive(false);
				}
			}
		}
	}

	UserDefinedStructsPendingPostChange++;
}

void FOptimusEditorModule::PostChange(const UUserDefinedStruct* Changed,
	FStructureEditorUtils::EStructureEditorChangeInfo ChangedType)
{
	FOptimusDataTypeRegistry::Get().RefreshStructType(const_cast<UUserDefinedStruct*>(Changed));

	UserDefinedStructsPendingPostChange--;

	// Only recompile/reactivate all instances once all struct changes have been processed
	if (UserDefinedStructsPendingPostChange == 0)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		// Search for referencing packages to the currently changing user defined struct 
		TArray<FAssetIdentifier> Referencers;
		AssetRegistry.GetReferencers(Changed->GetPackage()->GetFName(), Referencers);
		for (const FAssetIdentifier& Identifier : Referencers)
		{
			TArray<FAssetData> Assets;
			AssetRegistry.GetAssetsByPackageName(Identifier.PackageName, Assets);

			for (const FAssetData& Asset : Assets)
			{
				if (Asset.IsInstanceOf(UOptimusDeformer::StaticClass()))
				{
					// Only care about loaded assets, which may have active instances
					if (UOptimusDeformer* DeformerAsset = Cast<UOptimusDeformer>(Asset.FastGetAsset(false)))
					{
						DeformerAsset->Compile();
						DeformerAsset->SetAllInstancesCanbeActive(true);
					}
				}
			}
		}	
	}
}

void FOptimusEditorModule::RegisterPropertyCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	auto RegisterPropertyCustomization = [&](FName InStructName, auto InCustomizationFactory)
	{
		PropertyModule.RegisterCustomPropertyTypeLayout(
			InStructName, 
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(InCustomizationFactory)
			);
		CustomizedProperties.Add(InStructName);
	};

	RegisterPropertyCustomization(FOptimusDataTypeRef::StaticStruct()->GetFName(), &FOptimusDataTypeRefCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusExecutionDomain::StaticStruct()->GetFName(), &FOptimusExecutionDomainCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusDataDomain::StaticStruct()->GetFName(), &FOptimusDataDomainCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusShaderText::StaticStruct()->GetFName(), &FOptimusShaderTextCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusParameterBinding::StaticStruct()->GetFName(), &FOptimusParameterBindingCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusParameterBindingArray::StaticStruct()->GetFName(), &FOptimusParameterBindingArrayCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusValueContainerStruct::StaticStruct()->GetFName(), &FOptimusValueContainerStructCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusValidatedName::StaticStruct()->GetFName(), &FOptimusValidatedNameCustomization::MakeInstance);
	RegisterPropertyCustomization(FOptimusDeformerInstanceComponentBinding::StaticStruct()->GetFName(), &FOptimusDeformerInstanceComponentBindingCustomization::MakeInstance);
	
	auto RegisterDetailCustomization = [&](FName InStructName, auto InCustomizationFactory)
	{
		PropertyModule.RegisterCustomClassLayout(
			InStructName,
			FOnGetDetailCustomizationInstance::CreateStatic(InCustomizationFactory)
		);
		CustomizedClasses.Add(InStructName);
	};

	RegisterDetailCustomization(UOptimusSource::StaticClass()->GetFName(), &FOptimusSourceDetailsCustomization::MakeInstance);
	RegisterDetailCustomization(UOptimusComponentSourceBinding::StaticClass()->GetFName(), &FOptimusComponentSourceBindingDetailsCustomization::MakeInstance);
	RegisterDetailCustomization(UOptimusResourceDescription::StaticClass()->GetFName(), &FOptimusResourceDescriptionDetailsCustomization::MakeInstance);
	PropertyModule.NotifyCustomizationModuleChanged();
}

void FOptimusEditorModule::UnregisterPropertyCustomizations()
{
	if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (const FName& PropertyName: CustomizedProperties)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
		for (const FName& ClassName : CustomizedClasses)
		{
			PropertyModule->UnregisterCustomClassLayout(ClassName);
		}
		PropertyModule->NotifyCustomizationModuleChanged();
	}
}

IMPLEMENT_MODULE(FOptimusEditorModule, OptimusEditor)

#undef LOCTEXT_NAMESPACE
