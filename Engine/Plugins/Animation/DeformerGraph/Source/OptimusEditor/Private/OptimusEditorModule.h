// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOptimusEditorModule.h"


class FOptimusEditorGraphNodeFactory;
class FOptimusEditorGraphPinFactory;
class IAssetTypeActions;

class FOptimusEditorModule : public IOptimusEditorModule
{
public:
	FOptimusEditorModule();
	
	// IModuleInterface implementations
	void StartupModule() override;
	void ShutdownModule() override;

	// IOptimusEditorModule implementations
	TSharedRef<IOptimusEditor> CreateEditor(
		const EToolkitMode::Type Mode, 
		const TSharedPtr<IToolkitHost>& InitToolkitHost, 
		UOptimusDeformer* DeformerObject
	) override;

	FOptimusEditorClipboard& GetClipboard() const override;

	// FStructureEditorUtils::INotifyOnStructChanged overrides to react to user defined struct member changes
	virtual void PreChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType) override;
	virtual void PostChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType) override;
	
private:
	void RegisterPropertyCustomizations();
	void UnregisterPropertyCustomizations();

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	TSharedPtr<FOptimusEditorGraphNodeFactory> GraphNodeFactory;
	TSharedPtr<FOptimusEditorGraphPinFactory> GraphPinFactory;

	TArray<FName> CustomizedProperties;
	TArray<FName> CustomizedClasses;

	TSharedRef<FOptimusEditorClipboard> Clipboard;

	int32 UserDefinedStructsPendingPostChange = 0;
};
