// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorStyle.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"


#define CORE_IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)


FOptimusEditorStyle::FOptimusEditorStyle() :
    FSlateStyleSet("OptimusEditorStyle")
{
	static const FVector2D IconSize10x10(10.0f, 10.0f);
	static const FVector2D IconSize16x12(16.0f, 12.0f);
	static const FVector2D IconSize16x16(16.0f, 16.0f);
	static const FVector2D IconSize20x20(20.0f, 20.0f);
	static const FVector2D IconSize64x64(64.0f, 64.0f);

	static const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	FSlateStyleSet::SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Animation/DeformerGraph/Resources"));

	// Asset icons
	{
		Set("ClassIcon.OptimusDeformer", new IMAGE_BRUSH_SVG("Icons/DeformerGraph_16", IconSize16x16, DefaultForeground));
		Set("ClassThumbnail.OptimusDeformer", new IMAGE_BRUSH_SVG("Icons/DeformerGraph_64", IconSize64x64, DefaultForeground));
	}

	// Text editor styles
	{
		const FSlateFontInfo Consolas10  = FCoreStyle::GetDefaultFontStyle("Mono", 9);

		const FTextBlockStyle NormalText = FTextBlockStyle()
			.SetFont(Consolas10)
			.SetColorAndOpacity(FLinearColor::White)
			.SetShadowOffset(FVector2D::ZeroVector)
			.SetShadowColorAndOpacity(FLinearColor::Black)
			.SetSelectedBackgroundColor(FLinearColor::Blue)
			.SetHighlightColor(FLinearColor::Yellow)
			.SetHighlightShape(BOX_BRUSH("Images/TextBlockHighlightShape", FMargin(3.f / 8.f)));

		const FVector2D Icon8x8(8.0f, 8.0f);
		const FTextBlockStyle ErrorText = FTextBlockStyle(NormalText)
			.SetUnderlineBrush(IMAGE_BRUSH("Images/White", Icon8x8, FLinearColor::Red, ESlateBrushTileType::Both))
			.SetColorAndOpacity(FLinearColor::Red);
		
		Set("TextEditor.NormalText", NormalText);

		Set("SyntaxHighlight.HLSL.Normal", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor::White));// yellow
		Set("SyntaxHighlight.HLSL.Operator", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffcfcfcf)))); // light grey
		Set("SyntaxHighlight.HLSL.Keyword", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff006ab4)))); // blue
		Set("SyntaxHighlight.HLSL.String", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffdfd706)))); // pinkish
		Set("SyntaxHighlight.HLSL.Number", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff6db3a8)))); // cyan
		Set("SyntaxHighlight.HLSL.Comment", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xff57a64a)))); // green
		Set("SyntaxHighlight.HLSL.PreProcessorKeyword", FTextBlockStyle(NormalText).SetColorAndOpacity(FLinearColor(FColor(0xffcfcfcf)))); // light grey

		Set("SyntaxHighlight.HLSL.Error", ErrorText); 
		
		Set("TextEditor.Border", new BOX_BRUSH("Images/TextEditorBorder", FMargin(4.0f/16.0f), FLinearColor(0.02f,0.02f,0.02f,1)));

		const FEditableTextBoxStyle EditableTextBoxStyle = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetBackgroundImageNormal( FSlateNoResource() )
			.SetBackgroundImageHovered( FSlateNoResource() )
			.SetBackgroundImageFocused( FSlateNoResource() )
			.SetBackgroundImageReadOnly( FSlateNoResource() );
		
		Set("TextEditor.EditableTextBox", EditableTextBoxStyle);

		FSearchBoxStyle SearchBoxStyle = FCoreStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox");
		SearchBoxStyle.SetLeftAlignGlassImageAndClearButton(true);
		SearchBoxStyle.SetLeftAlignSearchResultButtons(false);
		Set("TextEditor.SearchBoxStyle", SearchBoxStyle);

	}

	// Toolbar icons
	{
		Set("Toolbar.CompileStatus.Background", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", IconSize20x20));
	}
	
	// Compile status icons.
	{
		Set("Toolbar.CompileStatus.Overlay.Modified", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Unknown_Badge", IconSize20x20, FStyleColors::AccentYellow));
		Set("Toolbar.CompileStatus.Overlay.Warning", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Warning_Badge", IconSize20x20, FStyleColors::Warning));
		Set("Toolbar.CompileStatus.Overlay.Good", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Good_Badge", IconSize20x20, FStyleColors::AccentGreen));
		Set("Toolbar.CompileStatus.Overlay.Error", new CORE_IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Fail_Badge", IconSize20x20, FStyleColors::Error));
	}

	// Graph type icons
	{
		Set("GraphType.Setup", new IMAGE_BRUSH_SVG("Icons/Graph_Setup", IconSize16x16, DefaultForeground));
		Set("GraphType.Trigger", new IMAGE_BRUSH_SVG("Icons/Graph_Trigger", IconSize16x16, DefaultForeground));
		Set("GraphType.Update", new IMAGE_BRUSH_SVG("Icons/Graph_Update", IconSize16x16, DefaultForeground));
		Set("GraphType.SubGraph", new IMAGE_BRUSH_SVG("Icons/Graph_SubGraph", IconSize16x16, DefaultForeground));
	}

	// Graph styles
	{
		FTextBlockStyle NormalText = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		Set( "Node.PinLabel", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,218.0f/255.0f) )
			.SetShadowOffset( FVector2D::ZeroVector )
			.SetShadowColorAndOpacity( FLinearColor(0.8f,0.8f,0.8f, 0.5) )
			);
		Set( "Node.GroupLabel", FTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("Node.PinLabel"))
			.SetFont( DEFAULT_FONT( "Bold", 9 ) )
			);

		Set( "Node.ToolTipLabel", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Bold", 9 ))
			.SetColorAndOpacity(FLinearColor::Black)
			);
		Set( "Node.ToolTipContent", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9))
			.SetColorAndOpacity(FLinearColor::Black)
			);
		
		
		const FTableViewStyle NodePinTreeViewStyle = FTableViewStyle();
		Set("Node.PinTreeView", NodePinTreeViewStyle);		
		
		Set("Node.Pin.Resource_Connected", new IMAGE_BRUSH_SVG("Icons/Resource_Pin_Connected", IconSize16x12, DefaultForeground));
		Set("Node.Pin.Resource_Disconnected", new IMAGE_BRUSH_SVG("Icons/Resource_Pin_Disconnected", IconSize16x12, DefaultForeground));		

		Set("Node.Pin.Value_Connected", new IMAGE_BRUSH_SVG("Icons/Value_Pin_Connected", IconSize16x12, DefaultForeground));
		Set("Node.Pin.Value_Disconnected", new IMAGE_BRUSH_SVG("Icons/Value_Pin_Disconnected", IconSize16x12, DefaultForeground));		

		Set("Node.Pin.Grouping", new IMAGE_BRUSH_SVG("Icons/Grouping_Pin", IconSize16x12, DefaultForeground));

		Set("Node.PinTree.Arrow_Collapsed_Left", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Collapsed_Left", IconSize10x10, DefaultForeground));
		Set("Node.PinTree.Arrow_Collapsed_Hovered_Left", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Collapsed_Hovered_Left", IconSize10x10, DefaultForeground));

		Set("Node.PinTree.Arrow_Expanded_Left", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Expanded_Left", IconSize10x10, DefaultForeground));
		Set("Node.PinTree.Arrow_Expanded_Hovered_Left", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Expanded_Hovered_Left", IconSize10x10, DefaultForeground));

		Set("Node.PinTree.Arrow_Collapsed_Right", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Collapsed_Right", IconSize10x10, DefaultForeground));
		Set("Node.PinTree.Arrow_Collapsed_Hovered_Right", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Collapsed_Hovered_Right", IconSize10x10, DefaultForeground));

		Set("Node.PinTree.Arrow_Expanded_Right", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Expanded_Right", IconSize10x10, DefaultForeground));
		Set("Node.PinTree.Arrow_Expanded_Hovered_Right", new IMAGE_BRUSH_SVG("Icons/TreeArrow_Expanded_Hovered_Right", IconSize10x10, DefaultForeground));
	}
}

void FOptimusEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}


void FOptimusEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}


FOptimusEditorStyle& FOptimusEditorStyle::Get()
{
	static FOptimusEditorStyle Instance;
	return Instance;
}

#undef CORE_IMAGE_BRUSH_SVG
