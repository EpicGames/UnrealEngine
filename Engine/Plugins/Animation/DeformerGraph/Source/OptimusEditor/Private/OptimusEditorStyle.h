// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


class FOptimusEditorStyle
    : public FSlateStyleSet
{
public:
	static FOptimusEditorStyle& Get();

protected:
	friend class FOptimusEditorModule;

	static void Register();
	static void Unregister();

private:
	FOptimusEditorStyle();
};
