// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusEditorTabSummoners.h"

#include "Widgets/Docking/SDockTab.h"

#include "FindInDeformer.h"
#include "GraphEditor.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "OptimusEditor.h"
#include "Widgets/SOptimusEditorGraphExplorer.h"
#include "Widgets/SOptimusNodePalette.h"
#include "Widgets/SOptimusShaderTextEditor.h"

#define LOCTEXT_NAMESPACE "OptimusEditorTabSummoners"

const FName FOptimusEditorNodePaletteTabSummoner::TabId("OptimusEditor_Palette");
const FName FOptimusEditorExplorerTabSummoner::TabId("OptimusEditor_Explorer");
const FName FOptimusEditorGraphTabSummoner::TabId("OptimusEditor_Graph");
const FName FOptimusEditorFindTabSummoner::TabId("OptimusEditor_Find");
const FName FOptimusEditorCompilerOutputTabSummoner::TabId("OptimusEditor_Output"); 
const FName FOptimusEditorShaderTextEditorTabSummoner::TabId("OptimusEditor_ShaderTextEditor");


FOptimusEditorNodePaletteTabSummoner::FOptimusEditorNodePaletteTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
	) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("NodePaletteTab_TabLabel", "Palette");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette");
	
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("NodePaletteTab_MenuLabel", "Node Palette");
	ViewMenuTooltip = LOCTEXT("NodePaletteTab_MenuLabel_Tooltip", "Show the Node Palette tab");
}


TSharedRef<SWidget> FOptimusEditorNodePaletteTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SOptimusNodePalette, EditorPtr.Pin());	
}



FOptimusEditorExplorerTabSummoner::FOptimusEditorExplorerTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
	) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("NodeExplorerTab_TabLabel", "Explorer");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.BlueprintCore");
	
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("NodeExplorerTab_MenuLabel", "Graph Explorer");
	ViewMenuTooltip = LOCTEXT("NodeExplorerTab_MenuLabel_Tooltip", "Show the Graph Explorer tab");
}


TSharedRef<SWidget> FOptimusEditorExplorerTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info
	) const
{
	return SNew(SOptimusEditorGraphExplorer, EditorPtr.Pin());
}


FOptimusEditorGraphTabSummoner::FOptimusEditorGraphTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
	) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("NodeGraphTab_TabLabel", "Graph");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
	
	bIsSingleton = false;

	ViewMenuDescription = LOCTEXT("NodeGraphTab_MenuLabel", "Node Graph");
	ViewMenuTooltip = LOCTEXT("NodeGraphTab_MenuLabel_Tooltip", "Show the Node Graph tab");
}


TSharedRef<SWidget> FOptimusEditorGraphTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info
	) const
{
	TSharedRef<SWidget> Test = EditorPtr.Pin()->GetGraphEditorWidget().ToSharedRef();
	// FIXME: Move to own widget.
	return EditorPtr.Pin()->GetGraphEditorWidget().ToSharedRef();
}


FOptimusEditorFindTabSummoner::FOptimusEditorFindTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("FindTab_TabLabel", "Find");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");

	bIsSingleton = false;

	ViewMenuDescription = LOCTEXT("FindTab_MenuLabel", "Find");
	ViewMenuTooltip = LOCTEXT("FindTab_MenuLabel_Tooltip", "Search contents of the Graph");
}


TSharedRef<SWidget> FOptimusEditorFindTabSummoner::CreateTabBody(
	const FWorkflowTabSpawnInfo& Info
) const
{
	return EditorPtr.Pin()->GetFindResultsWidget().ToSharedRef();
}


FOptimusEditorCompilerOutputTabSummoner::FOptimusEditorCompilerOutputTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
	) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("NodeCompilerOutputTab_TabLabel", "Compiler Output");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.StatsViewer");
	
	bIsSingleton = false;

	ViewMenuDescription = LOCTEXT("NodeCompilerOutputTab_MenuLabel", "Compiler Output");
	ViewMenuTooltip = LOCTEXT("NodeCompilerOutputTab_MenuLabel_Tooltip", "Show the Compiler Output tab");
}


TSharedRef<SWidget> FOptimusEditorCompilerOutputTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	// -- Compiler results
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	return MessageLogModule.CreateLogListingWidget(EditorPtr.Pin()->GetMessageLog());
}


FOptimusEditorShaderTextEditorTabSummoner::FOptimusEditorShaderTextEditorTabSummoner(
	TSharedRef<FOptimusEditor> InEditorApp
	) :
	FWorkflowTabFactory(TabId, InEditorApp),
	EditorPtr(InEditorApp)
{
	TabLabel = LOCTEXT("ShaderTextEditorTab_TabLabel", "Shader Text Editor");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.TextEditor");
	
	ViewMenuDescription = LOCTEXT("ShaderTextEditorTab_MenuLabel", "Shader Text Editor");
	ViewMenuTooltip = LOCTEXT("ShaderTextEditorTab_MenuLabel_Tooltip", "Show the Shader Text Editor Tab");	
}


TSharedRef<SDockTab> FOptimusEditorShaderTextEditorTabSummoner::OnSpawnTab(const FSpawnTabArgs& SpawnArgs,
	TWeakPtr<FTabManager> WeakTabManager) const
{
	TSharedRef<SDockTab> ShaderTextEditorTab = FWorkflowTabFactory::OnSpawnTab(SpawnArgs, WeakTabManager);

	const TSharedRef<SOptimusShaderTextEditor> ShaderTextEditorWidget = SNew(SOptimusShaderTextEditor, EditorPtr.Pin(), ShaderTextEditorTab);
	ShaderTextEditorTab->SetContent(ShaderTextEditorWidget);

	return ShaderTextEditorTab;
}

#undef LOCTEXT_NAMESPACE
