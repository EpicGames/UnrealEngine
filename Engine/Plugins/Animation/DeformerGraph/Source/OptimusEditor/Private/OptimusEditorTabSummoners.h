// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FOptimusEditor;

struct FOptimusEditorNodePaletteTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;
	
	explicit FOptimusEditorNodePaletteTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);
	
	TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};


struct FOptimusEditorExplorerTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;
	
	explicit FOptimusEditorExplorerTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);
	
	TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};


struct FOptimusEditorGraphTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;
	
	explicit FOptimusEditorGraphTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);
	
	TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};


struct FOptimusEditorFindTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;

	explicit FOptimusEditorFindTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);

	TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};



struct FOptimusEditorCompilerOutputTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;
	
	explicit FOptimusEditorCompilerOutputTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);
	
	TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};



struct FOptimusEditorShaderTextEditorTabSummoner :
	public FWorkflowTabFactory
{
	static const FName TabId;
	
	explicit FOptimusEditorShaderTextEditorTabSummoner(TSharedRef<FOptimusEditor> InEditorApp);

	/** Callback function for spawning the tab */
	virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const override;
	
protected:
	TWeakPtr<FOptimusEditor> EditorPtr;
};



