// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusHLSLSyntaxHighlighter.h"
#include "OptimusEditorStyle.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/SlateTextUnderlineLineHighlighter.h"

TSharedRef<FOptimusHLSLSyntaxHighlighter> FOptimusHLSLSyntaxHighlighter::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	return MakeShareable(new FOptimusHLSLSyntaxHighlighter(CreateTokenizer(), InSyntaxTextStyle));
}

void FOptimusHLSLSyntaxHighlighter::SetCompilerMessages(const TArray<FOptimusCompilerDiagnostic> &InCompilerMessages)
{
	CompilerMessages.Reset();

	for (const FOptimusCompilerDiagnostic& Message: InCompilerMessages)
	{
		CompilerMessages.Add(Message.Line - 1, Message);
	}
	MakeDirty();
}


FOptimusHLSLSyntaxHighlighter::FOptimusHLSLSyntaxHighlighter(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle) :
	FHLSLSyntaxHighlighterMarshaller(MoveTemp(InTokenizer), InSyntaxTextStyle)
{
}

void FOptimusHLSLSyntaxHighlighter::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout,	TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	LineHighlightsToAdd.Empty();
	FHLSLSyntaxHighlighterMarshaller::ParseTokens(SourceString, TargetTextLayout, TokenizedLines);

	TargetTextLayout.SetLineHighlights(LineHighlightsToAdd);
}

FTextLayout::FNewLineData FOptimusHLSLSyntaxHighlighter::ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& CurrentParseState)
{
	FTextLayout::FNewLineData LineData = FHLSLSyntaxHighlighterMarshaller::ProcessTokenizedLine(TokenizedLine, LineNumber, SourceString, CurrentParseState);

	FTextBlockStyle ErrorTextStyle = SyntaxTextStyle.ErrorTextStyle;
	TSharedPtr<FSlateTextUnderlineLineHighlighter> UnderlineLineHighlighter = FSlateTextUnderlineLineHighlighter::Create(ErrorTextStyle.UnderlineBrush, ErrorTextStyle.Font, ErrorTextStyle.ColorAndOpacity, ErrorTextStyle.ShadowOffset, ErrorTextStyle.ShadowColorAndOpacity);
	
	TArray<const FOptimusCompilerDiagnostic *> Diagnostics;
	CompilerMessages.MultiFindPointer(LineNumber, Diagnostics);
		
	for (const FOptimusCompilerDiagnostic *Diagnostic: Diagnostics)
	{
		// ColumnStart/ColumnEnd are 1-based, closed interval. FTextRange is 0 based, half-closed interval. 
		FTextRange UnderlineRange(Diagnostic->ColumnStart - 1, Diagnostic->ColumnEnd);

		// The highlighting lines have to match the runs and not exceed their bounds, so we chop them up. 
		for (TSharedRef< IRun > Run: LineData.Runs)
		{
			FTextRange ChoppedRange(Run->GetTextRange().Intersect(UnderlineRange));

			if (!ChoppedRange.IsEmpty())
			{
				LineHighlightsToAdd.Add(FTextLineHighlight(LineNumber, UnderlineRange, FSlateTextUnderlineLineHighlighter::DefaultZIndex, UnderlineLineHighlighter.ToSharedRef()));
			}
		}
	}

	return LineData;
}

FOptimusHLSLSyntaxHighlighter::FSyntaxTextStyle FOptimusHLSLSyntaxHighlighter::GetSyntaxTextStyle()
{
	return FSyntaxTextStyle( FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Normal"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Operator"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Keyword"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.String"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Number"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Comment"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.PreProcessorKeyword"),
			FOptimusEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("SyntaxHighlight.HLSL.Error"));
}
