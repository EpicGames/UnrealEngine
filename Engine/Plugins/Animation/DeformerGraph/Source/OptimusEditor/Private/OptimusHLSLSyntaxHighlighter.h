// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/SyntaxTokenizer.h"

// TEMP
#include "Text/HLSLSyntaxHighlighterMarshaller.h"
#include "OptimusDiagnostic.h"

class FTextLayout;

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FOptimusHLSLSyntaxHighlighter : public FHLSLSyntaxHighlighterMarshaller
{
public:

	static TSharedRef<FOptimusHLSLSyntaxHighlighter> Create(const FSyntaxTextStyle& InSyntaxTextStyle = GetSyntaxTextStyle());
	static FSyntaxTextStyle GetSyntaxTextStyle();

	void SetCompilerMessages(const TArray<FOptimusCompilerDiagnostic> &InCompilerMessages);

protected:

	FOptimusHLSLSyntaxHighlighter(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;
	virtual FTextLayout::FNewLineData ProcessTokenizedLine(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine, const int32& LineNumber, const FString& SourceString, EParseState& CurrentParseState) override;

	TMultiMap<int32 /*Line*/, FOptimusCompilerDiagnostic> CompilerMessages;

private:

	TArray<FTextLineHighlight> LineHighlightsToAdd;
};
