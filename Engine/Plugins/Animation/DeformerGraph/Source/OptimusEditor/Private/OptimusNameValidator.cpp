// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusNameValidator.h"

#include "OptimusNodeGraph.h"


FOptimusNameValidator::FOptimusNameValidator(
	const UObject *InOuter,
	const UClass* InObjectClass,
	FName InExistingName
	) :
	ExistingName(InExistingName)
{
	if (InOuter)
	{
		ForEachObjectWithOuter(InOuter, [this, InObjectClass](const UObject* InObject)
		{
			if (InObject->IsA(InObjectClass))
			{
				Names.Add(InObject->GetFName());
			}
		}, EGetObjectsFlags::None);
	}
}


EValidatorResult FOptimusNameValidator::IsValid(const FString& InName, bool)
{
	if (InName.Len() >= NAME_SIZE)
	{
		return EValidatorResult::TooLong;
	}
	else if (!FName::IsValidXName(InName, InvalidCharacters()))
	{
		return EValidatorResult::ContainsInvalidCharacters;
	}

	return IsValid(FName(*InName));
}


EValidatorResult FOptimusNameValidator::IsValid(const FName& InName, bool)
{
	if (InName == NAME_None)
	{
		return EValidatorResult::EmptyName;
	}
	else if (InName == ExistingName)
	{
		return EValidatorResult::Ok;
	}
	else if (Names.Contains(InName))
	{
		return EValidatorResult::AlreadyInUse;
	}
	else
	{
		return EValidatorResult::Ok;
	}
}
