// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet2/Kismet2NameValidators.h"


class IOptimusNodeGraphCollectionOwner;


class FOptimusNameValidator :
	public INameValidatorInterface
{
public:
	FOptimusNameValidator(const UObject *InOuter, const UClass* InObjectClass, FName InExistingName);
	
	/// Disallowed characters in a graph/variable/buffer name. Period is disallowed since it's
	/// a path separator (like with BP), and slash is also disallowed since we use it for 
	/// constructing name paths for undo, space is also not allowed since it complicates unique name checking
	static const TCHAR *InvalidCharacters() { return TEXT("./$ "); }

	// INameValidatorInterface overrides
	EValidatorResult IsValid(const FName& InName, bool bInOriginal = false) override;

	EValidatorResult IsValid(const FString& InName, bool bInOriginal = false) override;

private:
	// Existing names to validate against.
	TSet<FName> Names;

	// The current name.
	FName ExistingName;
};
