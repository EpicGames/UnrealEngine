// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSourceAssetActions.h"
#include "OptimusSource.h"

FOptimusSourceAssetActions::FOptimusSourceAssetActions(EAssetTypeCategories::Type InAssetCategoryBit)
	: AssetCategoryBit(InAssetCategoryBit)
{
}

FText FOptimusSourceAssetActions::GetName() const
{
	return NSLOCTEXT("AssetTypeActions", "OptimusSourceAssetName", "Deformer Source Library");
}

FColor FOptimusSourceAssetActions::GetTypeColor() const
{
	return FColor::Turquoise;
}

UClass* FOptimusSourceAssetActions::GetSupportedClass() const
{
	return UOptimusSource::StaticClass();
}

uint32 FOptimusSourceAssetActions::GetCategories()
{
	return AssetCategoryBit;
}

const TArray<FText>& FOptimusSourceAssetActions::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("AssetTypeActions", "AnimDeformersSubMenu", "Deformers")
	};
	return SubMenus;
}
