// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "AssetTypeActions_Base.h"

class FOptimusSourceAssetActions
	: public FAssetTypeActions_Base
{
public:
	FOptimusSourceAssetActions(EAssetTypeCategories::Type InAssetCategoryBit = EAssetTypeCategories::Animation);

protected:
	//~ Begin FAssetTypeActions_Base Interface.
	FText GetName() const override;
	FColor GetTypeColor() const override;
	UClass* GetSupportedClass() const override;
	uint32 GetCategories() override;
	const TArray<FText>& GetSubMenus() const override;
	//~ End FAssetTypeActions_Base Interface.

private:
	EAssetTypeCategories::Type AssetCategoryBit;
};
