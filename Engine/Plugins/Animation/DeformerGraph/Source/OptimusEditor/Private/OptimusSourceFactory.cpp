// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSourceFactory.h"
#include "OptimusSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusSourceFactory)

UOptimusSourceFactory::UOptimusSourceFactory()
{
	SupportedClass = UOptimusSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UOptimusSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UOptimusSource* Kernel = NewObject<UOptimusSource>(InParent, InClass, InName, Flags);

	return Kernel;
}
