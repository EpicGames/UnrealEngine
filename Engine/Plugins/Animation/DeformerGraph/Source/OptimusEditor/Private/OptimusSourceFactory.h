// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "OptimusSourceFactory.generated.h"

UCLASS(hidecategories = Object)
class UOptimusSourceFactory : public UFactory
{
	GENERATED_BODY()

public:
	UOptimusSourceFactory();

	virtual UObject* FactoryCreateNew(
		UClass* InClass, 
		UObject* InParent, 
		FName InName, 
		EObjectFlags Flags, 
		UObject* Context, 
		FFeedbackContext* Warn
		) override;
};
