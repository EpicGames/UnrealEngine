// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusDataTypeSelector.h"

#include "OptimusEditorGraphSchema.h"

#include "OptimusDataTypeRegistry.h"

#include "EdGraphSchema_K2.h"
#include "OptimusHelpers.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Animation/AttributeTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "UObject/UObjectIterator.h"


#define LOCTEXT_NAMESPACE "OptimusDataTypeSelector"



void SOptimusDataTypeSelector::Construct(const FArguments& InArgs)
{
	CurrentDataType = InArgs._CurrentDataType;
	ViewType = InArgs._ViewType;
	bViewOnly = InArgs._bViewOnly;
	UsageMask = InArgs._UsageMask;
	OnDataTypeChanged = InArgs._OnDataTypeChanged;

	
	TSharedPtr<SWidget> IconWidget;
	TSharedPtr<SWidget> ViewWidget;

	IconWidget = SNew(SImage)
	                .Image(this, &SOptimusDataTypeSelector::GetTypeIconImage)
	                .ColorAndOpacity(this, &SOptimusDataTypeSelector::GetTypeIconColor);

	if (ViewType == EViewType::IconOnly)
	{
		ViewWidget = IconWidget;
	}
	else
	{
		ViewWidget = SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
		    .HAlign(HAlign_Left)
			.AutoWidth()
			[
				IconWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
		    .VAlign(VAlign_Center)
		    .HAlign(HAlign_Left)
		    .AutoWidth()
		    [	
				SNew(STextBlock)
		        .Text(this, &SOptimusDataTypeSelector::GetTypeDescription)
				.Font(InArgs._Font)
			];
	}

	TSharedPtr<SWidget> Widget;

	if (bViewOnly)
	{
		Widget = ViewWidget;
	}
	else if (ViewType == EViewType::IconOnly)
	{
		Widget = SAssignNew(TypeComboButton, SComboButton)
			.OnGetMenuContent(this, &SOptimusDataTypeSelector::GetMenuContent)
			.ContentPadding(0)
			.HasDownArrow(false)
			.ButtonStyle(FAppStyle::Get(), "BlueprintEditor.CompactPinTypeSelector")
			.ButtonContent()
			[
				ViewWidget.ToSharedRef()
			];
	}
	else
	{
		Widget = SNew(SBox)
			.MinDesiredWidth(100.0f)
			[
				SAssignNew(TypeComboButton, SComboButton)
		        .MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
				.OnGetMenuContent(this, &SOptimusDataTypeSelector::GetMenuContent)
				.ContentPadding(0)
				.ButtonContent()
				[
					ViewWidget.ToSharedRef()
				]
			];
	}
	
	Widget->SetToolTipText(TAttribute<FText>(this, &SOptimusDataTypeSelector::GetTypeTooltip));

	ChildSlot
	[
		Widget.ToSharedRef()
	];
}


const FSlateBrush* SOptimusDataTypeSelector::GetTypeIconImage(FOptimusDataTypeHandle InDataType) const
{
	if (InDataType.IsValid())
	{
		FEdGraphPinType PinType = UOptimusEditorGraphSchema::GetPinTypeFromDataType(InDataType);

		return UOptimusEditorGraphSchema::GetIconFromPinType(PinType);
	}
	else
	{
		return nullptr;
	}
}


FSlateColor SOptimusDataTypeSelector::GetTypeIconColor(FOptimusDataTypeHandle InDataType) const
{
	if (InDataType.IsValid())
	{
		FEdGraphPinType PinType = UOptimusEditorGraphSchema::GetPinTypeFromDataType(InDataType);
		return GetDefault<UOptimusEditorGraphSchema>()->GetPinTypeColor(PinType);
	}
	else
	{
		return FLinearColor::Transparent;
	}
}


FText SOptimusDataTypeSelector::GetTypeDescription(FOptimusDataTypeHandle InDataType) const
{
	if (InDataType.IsValid())
	{
		if (UsageMask.IsSet())
		{
			if (EnumHasAnyFlags(InDataType->UsageFlags, UsageMask.Get()))
			{
				if (UsageMask.Get() == EOptimusDataTypeUsageFlags::AnimAttributes)
				{
					if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InDataType->TypeObject))
					{
						if (!UE::Anim::AttributeTypes::IsTypeRegistered(UserDefinedStruct))
						{
							return FText::FromString(TEXT("<Unregistered> ") + InDataType->DisplayName.ToString());
						}
					}
				}

				return InDataType->DisplayName;
			}
		}
		
		return FText::FromString(TEXT("<Unsupported> ") + InDataType->DisplayName.ToString());
	}
	
	return FText::FromString(TEXT("<None>"));
}


FText SOptimusDataTypeSelector::GetTypeTooltip(FOptimusDataTypeHandle InDataType) const
{
	if (!InDataType.IsValid())
	{
		return FText::GetEmpty();
	}

	if (!UsageMask.IsSet())
	{
		return FText::GetEmpty();
	}

	if (UsageMask.Get() == EOptimusDataTypeUsageFlags::AnimAttributes)
	{
		if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InDataType->TypeObject))
		{
			if (!EnumHasAnyFlags(InDataType->UsageFlags, EOptimusDataTypeUsageFlags::AnimAttributes))
			{
				return FText::FromString(TEXT("Type contains unsupported members or nested arrays"));
			}
			else if (!UE::Anim::AttributeTypes::IsTypeRegistered(UserDefinedStruct))
			{
				return FText::FromString(TEXT("Please register the type in Project Settings - Animation - CustomAttributes - User Defined Struct Animation Attributes."));
			}
		}
	}


	return FText::GetEmpty();

}

FText SOptimusDataTypeSelector::GetTypeTooltip() const
{
	FText EditText;
	if (IsEnabled() && !bViewOnly)
	{
		EditText = LOCTEXT("DataTypeSelector", "Click to see the data type menu and change the current type.");
	}

	return FText::Format(LOCTEXT("TypeTooltip", "{0} Current Type: {1}"), EditText, GetTypeDescription());
}


TSharedRef<SWidget> SOptimusDataTypeSelector::GetMenuContent()
{
	AllDataTypeItems.Reset();

	TArray<FOptimusDataTypeHandle> UnsupportedUserDefinedStructs;
	FOptimusDataTypeHandle SelectedItem;
	for (FOptimusDataTypeHandle DataType : FOptimusDataTypeRegistry::Get().GetAllTypes())
	{
		if (!UsageMask.IsSet() ||
			UsageMask.Get() == EOptimusDataTypeUsageFlags::None ||
			EnumHasAnyFlags(DataType->UsageFlags, UsageMask.Get()))
		{
			AllDataTypeItems.Add(DataType);
		}
		else if (UsageMask.Get() == EOptimusDataTypeUsageFlags::AnimAttributes)
		{
			if (UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(DataType->TypeObject.Get()))
			{
				UnsupportedUserDefinedStructs.Add(DataType);	
			}
		}
	}

	// Show unsupported types for animation attribute at the end
	// so that we can provide tooltips on why they are not supported
	AllDataTypeItems.Append(UnsupportedUserDefinedStructs);

	for (FOptimusDataTypeHandle DataType : AllDataTypeItems)
	{
		if (DataType == CurrentDataType.Get())
		{
			SelectedItem = DataType;
			break;
		}
	}

	ViewDataTypeItems = AllDataTypeItems;

	if (!MenuContent.IsValid())
	{
		TypeListView = SNew(SDataTypeListView)
			.ListItemsSource(&ViewDataTypeItems)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(this, &SOptimusDataTypeSelector::GenerateTypeListRow)
			.OnSelectionChanged(this, &SOptimusDataTypeSelector::OnTypeSelectionChanged)
			.ScrollbarVisibility(EVisibility::Visible);

		FilterBox = SNew(SSearchBox)
			.OnTextChanged(this, &SOptimusDataTypeSelector::OnFilterTextChanged)
			.OnTextCommitted(this, &SOptimusDataTypeSelector::OnFilterTextCommitted);

		MenuContent = SNew(SMenuOwner)
			[
				SNew(SListViewSelectorDropdownMenu<FOptimusDataTypeHandle>, FilterBox, TypeListView)
				[
					SNew(SVerticalBox) 
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.f, 4.f, 4.f, 4.f)
					[
						FilterBox.ToSharedRef()
					] 
					+ SVerticalBox::Slot()
					.MaxHeight(400.0f)
					.Padding(4.f, 4.f, 4.f, 4.f)
					[
						SNew(SBox)
						.MinDesiredWidth(150.0f)
						.MinDesiredHeight(200.0f)
						[
							TypeListView.ToSharedRef()
						]
					]
				]
			];
	}

	// Refresh in case ViewDataTypeItems has changed
	TypeListView->RebuildList();
	
	// Update the current selection
	if (SelectedItem.IsValid())
	{
		TypeListView->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
	}

	FilterBox->SetText(FText::GetEmpty());

	return MenuContent.ToSharedRef();
}


TSharedRef<ITableRow> SOptimusDataTypeSelector::GenerateTypeListRow(
	FOptimusDataTypeHandle InItem, 
	const TSharedRef<STableViewBase>& InOwnerList
	)
{
	using FDataTypeListRow = STableRow<FOptimusDataTypeHandle>;

	return SNew(FDataTypeListRow, InOwnerList)
		.Content()
		[
			SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
	        + SHorizontalBox::Slot()
	        .VAlign(VAlign_Center)
	        .HAlign(HAlign_Left)
	        .AutoWidth()
	        [
				SNew(SImage)
	            .Image_Lambda([InItem, this]() { return SOptimusDataTypeSelector::GetTypeIconImage(InItem); } )
	            .ColorAndOpacity_Lambda([InItem, this]() { return SOptimusDataTypeSelector::GetTypeIconColor(InItem); } )
			] 
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([InItem, this]() { return SOptimusDataTypeSelector::GetTypeDescription(InItem); })
				.HighlightText(FilterText)
				.Font(FAppStyle::GetFontStyle(TEXT("Kismet.TypePicker.NormalFont")))
				.ToolTipText_Lambda([InItem, this](){ return SOptimusDataTypeSelector::GetTypeTooltip(InItem); })
			]
		];
}


void SOptimusDataTypeSelector::OnTypeSelectionChanged(
	FOptimusDataTypeHandle InSelection, 
	ESelectInfo::Type InSelectInfo
	)
{
	if (InSelectInfo != ESelectInfo::OnNavigation)
	{
		// Close the menu
		if (TypeComboButton.IsValid())
		{
			TypeComboButton->SetIsOpen(false);
		}
		OnDataTypeChanged.ExecuteIfBound(InSelection);
	}
}


void SOptimusDataTypeSelector::OnFilterTextChanged(const FText& InNewText)
{
	FilterText = InNewText;

	ViewDataTypeItems = GetFilteredItems(AllDataTypeItems, InNewText);
	TypeListView->RequestListRefresh();

	// Select the first item that matches.
	if (!ViewDataTypeItems.IsEmpty())
	{
		// Navigate to the selection so that it won't trigger the data type selection callback.
		TypeListView->SetSelection(ViewDataTypeItems[0], ESelectInfo::OnNavigation);
	}
}


void SOptimusDataTypeSelector::OnFilterTextCommitted(const FText& InNewText, ETextCommit::Type InCommitInfo)
{
	if (InCommitInfo == ETextCommit::OnEnter)
	{
		TArray<FOptimusDataTypeHandle> SelectedItems  = TypeListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			// Do a direct selection to trigger the data type selection callback.
			TypeListView->SetSelection(SelectedItems[0]);
		}
	}
}


TArray<FOptimusDataTypeHandle> SOptimusDataTypeSelector::GetFilteredItems(
	const TArray<FOptimusDataTypeHandle>& InItems, 
	const FText& InSearchText
	) const
{
	// Trim and sanitized the filter text (so that it more likely matches the action descriptions)
	FString TrimmedFilterString = FText::TrimPrecedingAndTrailing(InSearchText).ToString();

	if (TrimmedFilterString.IsEmpty())
	{
		return InItems;
	}

	// Tokenize the search box text into a set of terms; all of them must be present to pass the filter
	TArray<FString> FilterTerms;
	TrimmedFilterString.ParseIntoArray(FilterTerms, TEXT(" "), true);

	// Generate a list of sanitized versions of the strings
	TArray<FString> SanitizedFilterTerms;
	for (const FString& FilterTerm: FilterTerms)
	{
		FString EachString = FName::NameToDisplayString(FilterTerm, false);
		EachString = EachString.Replace(TEXT(" "), TEXT(""));
		SanitizedFilterTerms.Add(EachString);
	}

	TArray<FOptimusDataTypeHandle> Result;
	for (FOptimusDataTypeHandle Item : InItems)
	{
		const FText LocalizedDescription = GetTypeDescription(Item);
		const FString LocalizedDescriptionString = LocalizedDescription.ToString();
		const FString* SourceDescriptionStringPtr = FTextInspector::GetSourceString(LocalizedDescription);

		// Test both the localized and source strings for a match
		const FString MangledLocalizedDescriptionString = LocalizedDescriptionString.Replace(TEXT(" "), TEXT(""));
		const FString MangledSourceDescriptionString = (SourceDescriptionStringPtr && *SourceDescriptionStringPtr != LocalizedDescriptionString) ? SourceDescriptionStringPtr->Replace(TEXT(" "), TEXT("")) : FString();

		bool bFilterTextMatches = true;
		for (int32 Index = 0; Index < FilterTerms.Num() && bFilterTextMatches; ++Index)
		{
			const bool bMatchesLocalizedTerm = MangledLocalizedDescriptionString.Contains(FilterTerms[Index]) || MangledLocalizedDescriptionString.Contains(SanitizedFilterTerms[Index]);
			const bool bMatchesSourceTerm = !MangledSourceDescriptionString.IsEmpty() && (MangledSourceDescriptionString.Contains(FilterTerms[Index]) || MangledSourceDescriptionString.Contains(SanitizedFilterTerms[Index]));
			bFilterTextMatches = bFilterTextMatches && (bMatchesLocalizedTerm || bMatchesSourceTerm);
		}

		if (bFilterTextMatches)
		{
			Result.Add(Item);
		}
	}

	return Result;
}


#undef LOCTEXT_NAMESPACE
