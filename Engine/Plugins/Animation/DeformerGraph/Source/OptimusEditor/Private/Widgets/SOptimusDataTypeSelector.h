// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusDataType.h"

#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


template<typename T> class SListView;
class ITableRow;
class SComboButton;
class SMenuOwner;
class SSearchBox;
class STableViewBase;
struct FSlateBrush;


DECLARE_DELEGATE_OneParam(FOnDataTypeChanged, FOptimusDataTypeHandle)


using SDataTypeListView = SListView<FOptimusDataTypeHandle>;


class SOptimusDataTypeSelector : 
	public SCompoundWidget
{
public:
	enum class EViewType
	{
		IconOnly,
		IconAndText
	};

	SLATE_BEGIN_ARGS( SOptimusDataTypeSelector ) : 
		_ViewType(EViewType::IconAndText),
		_bViewOnly(false),
		_UsageMask(EOptimusDataTypeUsageFlags::None), 
		_Font(FAppStyle::GetFontStyle(TEXT("NormalFont")))
		{}
		SLATE_ATTRIBUTE( FOptimusDataTypeHandle, CurrentDataType )
		SLATE_ARGUMENT( EViewType, ViewType )
		SLATE_ARGUMENT( bool, bViewOnly )
		SLATE_ATTRIBUTE( EOptimusDataTypeUsageFlags, UsageMask)
	    SLATE_ATTRIBUTE(FSlateFontInfo, Font)
	    SLATE_EVENT(FOnDataTypeChanged, OnDataTypeChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments &InArgs);


private:
	const FSlateBrush* GetTypeIconImage(FOptimusDataTypeHandle InDataType) const;
	const FSlateBrush* GetTypeIconImage() const { return GetTypeIconImage(CurrentDataType.Get()); }

	FSlateColor GetTypeIconColor(FOptimusDataTypeHandle InDataType) const;
	FSlateColor GetTypeIconColor() const { return GetTypeIconColor(CurrentDataType.Get()); }
	FText GetTypeDescription(FOptimusDataTypeHandle InDataType) const;
	FText GetTypeDescription() const { return GetTypeDescription(CurrentDataType.Get()); }
	FText GetTypeTooltip(FOptimusDataTypeHandle InDataType) const;
	FText GetTypeTooltip() const;
	
	TSharedRef<SWidget> GetMenuContent();

	TSharedRef<ITableRow> GenerateTypeListRow(FOptimusDataTypeHandle InItem, const TSharedRef<STableViewBase>& InOwnerList);
	void OnTypeSelectionChanged(FOptimusDataTypeHandle InSelection, ESelectInfo::Type InSelectInfo);

	void OnFilterTextChanged(const FText& InNewText);
	void OnFilterTextCommitted(const FText& InNewText, ETextCommit::Type InCommitInfo);

	TArray<FOptimusDataTypeHandle> GetFilteredItems(const TArray<FOptimusDataTypeHandle> &InItems, const FText &InSearchText) const;

	TAttribute<FOptimusDataTypeHandle> CurrentDataType;
	EViewType ViewType;
	bool bViewOnly;
	TAttribute<EOptimusDataTypeUsageFlags> UsageMask;
	FOnDataTypeChanged OnDataTypeChanged;

	TSharedPtr<SComboButton> TypeComboButton;
	TSharedPtr<SMenuOwner> MenuContent;
	TSharedPtr<SDataTypeListView> TypeListView;
	TSharedPtr<SSearchBox> FilterBox;
	FText FilterText;

	TArray<FOptimusDataTypeHandle> AllDataTypeItems;
	TArray<FOptimusDataTypeHandle> ViewDataTypeItems;
};
