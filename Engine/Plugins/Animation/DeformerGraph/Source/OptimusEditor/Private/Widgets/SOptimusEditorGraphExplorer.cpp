// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusEditorGraphExplorer.h"

#include "OptimusEditor.h"
#include "OptimusEditorGraphSchema.h"
#include "OptimusEditorGraphSchemaActions.h"
#include "SOptimusEditorGraphExplorerItem.h"
#include "SOptimusEditorGraphExplorerActions.h"

#include "IOptimusNodeGraphCollectionOwner.h"
#include "OptimusDeformer.h"
#include "OptimusNodeGraph.h"
#include "OptimusResourceDescription.h"
#include "OptimusVariableDescription.h"

#include "Styling/AppStyle.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SGraphActionMenu.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "SPositiveActionButton.h"

// Ick.
#include "GraphActionNode.h"
#include "OptimusFunctionNodeGraph.h"


#define LOCTEXT_NAMESPACE "OptimusGraphExplorer"


FOptimusEditorGraphExplorerCommands::FOptimusEditorGraphExplorerCommands()
	: TCommands<FOptimusEditorGraphExplorerCommands>(
          TEXT("OptimusEditorGraphExplorer"), NSLOCTEXT("Contexts", "Explorer", "Explorer"),
          NAME_None, FAppStyle::GetAppStyleSetName())
{
}


void FOptimusEditorGraphExplorerCommands::RegisterCommands()
{
	UI_COMMAND(OpenGraph, "Open Graph", "Opens up this graph in the editor.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CreateSetupGraph, "Add New Setup Graph", "Create a new setup graph and show it in the editor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateTriggerGraph, "Add New Trigger Graph", "Create a new external trigger graph and show it in the editor.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CreateBinding, "Add New Component Binding", "Create a component binding.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateResource, "Add New Resource", "Create a shader resource.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateVariable, "Add New Variable", "Create a variable on the asset.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(DeleteEntry, "Delete", "Deletes this graph, resource or variable from this deformer.", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
}


void SOptimusEditorGraphExplorer::Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InOptimusEditor)
{
	OptimusEditor = InOptimusEditor;

	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().AddRaw(this, &SOptimusEditorGraphExplorer::Refresh);
	}	

	RegisterCommands();

	CreateWidgets();
}


SOptimusEditorGraphExplorer::~SOptimusEditorGraphExplorer()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().RemoveAll(this);
	}
}


void SOptimusEditorGraphExplorer::Refresh()
{
	bNeedsRefresh = false;

	GraphActionMenu->RefreshAllActions(/*bPreserveExpansion=*/ true);
}


void SOptimusEditorGraphExplorer::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bNeedsRefresh)
	{
		Refresh();
		bNeedsRefresh = false;
	}

	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
}


void SOptimusEditorGraphExplorer::RegisterCommands()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	if (Editor)
	{
		TSharedPtr<FUICommandList> ToolKitCommandList = Editor->GetToolkitCommands();

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().OpenGraph,
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnOpenGraph),
			FCanExecuteAction(), FGetActionCheckState(), 
			FIsActionButtonVisible::CreateSP(this, &SOptimusEditorGraphExplorer::CanOpenGraph));

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().CreateSetupGraph,
			FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnCreateSetupGraph),
			FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanCreateSetupGraph)
			);

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().CreateTriggerGraph,
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnCreateTriggerGraph),
		    FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanCreateTriggerGraph));

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().CreateBinding,
			FExecuteAction::CreateLambda([this]() { OnCreateBinding(nullptr); }),
			FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanCreateBinding));

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().CreateResource,	
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnCreateResource),
		    FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanCreateResource));

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().CreateVariable,
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnCreateVariable),
		    FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanCreateVariable));

		ToolKitCommandList->MapAction(FOptimusEditorGraphExplorerCommands::Get().DeleteEntry,
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnDeleteEntry),
		    FCanExecuteAction(), FGetActionCheckState(),
		    FIsActionButtonVisible::CreateSP(this, &SOptimusEditorGraphExplorer::CanDeleteEntry));

		ToolKitCommandList->MapAction(FGenericCommands::Get().Rename,
		    FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnRenameEntry),
		    FCanExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::CanRenameEntry));

	}
}


void SOptimusEditorGraphExplorer::CreateWidgets()
{
	TSharedPtr<SWidget> AddNewMenu = SNew(SPositiveActionButton)
		.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new item."))
		.OnGetMenuContent(this, &SOptimusEditorGraphExplorer::CreateAddNewMenuWidget)
		.IsEnabled(this, &SOptimusEditorGraphExplorer::IsEditingMode)
		.Icon(FAppStyle::GetBrush("Plus"))
		.Text(LOCTEXT("AddNew", "Add"));

	FMenuBuilder ViewOptions(true, nullptr);

	ViewOptions.AddMenuEntry(
	    LOCTEXT("ShowEmptySections", "Show Empty Sections"),
	    LOCTEXT("ShowEmptySectionsTooltip", "Should we show empty sections? eg. Graphs, Functions...etc."),
	    FSlateIcon(),
	    FUIAction(
	        FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnToggleShowEmptySections),
	        FCanExecuteAction(),
	        FIsActionChecked::CreateSP(this, &SOptimusEditorGraphExplorer::IsShowingEmptySections)),
	    NAME_None,
	    EUserInterfaceActionType::ToggleButton,
	    TEXT("OptimusGraphExplorer_ShowEmptySections"));

	SAssignNew(FilterBox, SSearchBox)
	    .OnTextChanged_Lambda([this](const FText&) { GraphActionMenu->GenerateFilteredItems(true); } );

	// create the main action list piece of this widget
	SAssignNew(GraphActionMenu, SGraphActionMenu, false)
	    .OnGetFilterText_Lambda([this]() { return FilterBox->GetText(); })
	    .OnCreateWidgetForAction(this, &SOptimusEditorGraphExplorer::OnCreateWidgetForAction)
	    .OnCollectAllActions(this, &SOptimusEditorGraphExplorer::CollectAllActions)
	    .OnCollectStaticSections(this, &SOptimusEditorGraphExplorer::CollectStaticSections)
	    .OnActionDragged(this, &SOptimusEditorGraphExplorer::OnActionDragged)
	    .OnCategoryDragged(this, &SOptimusEditorGraphExplorer::OnCategoryDragged)
	    .OnActionSelected(this, &SOptimusEditorGraphExplorer::OnActionSelected)
	    .OnActionDoubleClicked(this, &SOptimusEditorGraphExplorer::OnActionDoubleClicked)
	    .OnContextMenuOpening(this, &SOptimusEditorGraphExplorer::OnContextMenuOpening)
	    .OnCategoryTextCommitted(this, &SOptimusEditorGraphExplorer::OnCategoryNameCommitted)
	    .OnCanRenameSelectedAction(this, &SOptimusEditorGraphExplorer::CanRequestRenameOnActionNode)
	    .OnGetSectionTitle(this, &SOptimusEditorGraphExplorer::OnGetSectionTitle)
	    .OnGetSectionWidget(this, &SOptimusEditorGraphExplorer::OnGetSectionWidget)
		.DefaultRowExpanderBaseIndentLevel(1)
	    .AlphaSortItems(false)
	    .UseSectionStyling(true);

	// now piece together all the content for this widget
	ChildSlot
	[
		SNew(SVerticalBox)
	    + SVerticalBox::Slot()
	    .AutoHeight()
	    [
			SNew(SBorder)
	        .Padding(4.0f)
	        .BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	        [
				SNew(SVerticalBox)
	            + SVerticalBox::Slot()
	            .AutoHeight()
	            [
					SNew(SHorizontalBox)
	                + SHorizontalBox::Slot()
	                .AutoWidth()
	                .Padding(0, 0, 2, 0)
	                [
						AddNewMenu.ToSharedRef()
					]
	                + SHorizontalBox::Slot()
	                .FillWidth(1.0f)
	                .VAlign(VAlign_Center)
	                [
						FilterBox.ToSharedRef()
					]
	                + SHorizontalBox::Slot()
	                .AutoWidth()
	                .Padding(2, 0, 0, 0)
	                [
						SNew(SComboButton)
		                .ButtonStyle(FAppStyle::Get(), "SimpleButton")
	                    .ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
	                    .ForegroundColor(FSlateColor::UseForeground())
	                    .HasDownArrow(false)
	                    .ContentPadding(0)
	                    .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
	                    .MenuContent()
	                    [
							ViewOptions.MakeWidget()
						]
	                    .ButtonContent()
	                    [
	                    	SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Settings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		]
	    + SVerticalBox::Slot()
	    .FillHeight(1.0f)
	    [
			GraphActionMenu.ToSharedRef()
		]
	];

	Refresh();
}


TSharedRef<SWidget> SOptimusEditorGraphExplorer::CreateAddNewMenuWidget()
{
	FMenuBuilder MenuBuilder(/* bShouldCloseWindowAfterMenuSelection= */true, OptimusEditor.Pin()->GetToolkitCommands());

	BuildAddNewMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}


void SOptimusEditorGraphExplorer::BuildAddNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AddNewItem", LOCTEXT("AddOperations", "Add New"));

	MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateSetupGraph);
	MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateTriggerGraph);

	MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateBinding);
	MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateResource);
	MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateVariable);

	MenuBuilder.EndSection();
}


TSharedRef<SWidget> SOptimusEditorGraphExplorer::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return SNew(SOptimusEditorGraphExplorerItem, InCreateData, OptimusEditor.Pin());
}

void SOptimusEditorGraphExplorer::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (!Editor)
	{
		return;
	}

	// FIXME: This should be purely interface-based.
	UOptimusDeformer *Deformer = Editor->GetDeformer();
	if (!Deformer)
	{
		return;
	}
	
	for (UOptimusNodeGraph* Graph : Deformer->GetGraphs())
	{
		TSharedPtr<FOptimusSchemaAction_Graph> GraphAction = MakeShared<FOptimusSchemaAction_Graph>(Graph);
		OutAllActions.AddAction(GraphAction);

		CollectChildGraphActions(OutAllActions, Graph, {});
	}
	
	for (UOptimusComponentSourceBinding* Binding : Deformer->GetComponentBindings())
	{
		TSharedPtr<FOptimusSchemaAction_Binding> BindingAction = MakeShared<FOptimusSchemaAction_Binding>(Binding);
		OutAllActions.AddAction(BindingAction);
	}

	for (UOptimusResourceDescription* Resource : Deformer->GetResources())
	{
		TSharedPtr<FOptimusSchemaAction_Resource> ResourceAction = MakeShared<FOptimusSchemaAction_Resource>(Resource);
		OutAllActions.AddAction(ResourceAction);
	}

	for (UOptimusVariableDescription* Variable : Deformer->GetVariables())
	{
		TSharedPtr<FOptimusSchemaAction_Variable> VariableAction = MakeShared<FOptimusSchemaAction_Variable>(Variable);
		OutAllActions.AddAction(VariableAction);
	}
}


void SOptimusEditorGraphExplorer::CollectChildGraphActions(
	FGraphActionListBuilderBase& OutAllActions,
	const UOptimusNodeGraph* InParentGraph,
	const FText& InParentGraphCategory
	)
{
	const FText ParentGraphName = FText::FromName(InParentGraph->GetFName());
	FText Category;
	if (!InParentGraphCategory.IsEmpty())
	{
		Category = FText::Format(FText::FromString(TEXT("{0}|{1}")), InParentGraphCategory, ParentGraphName);
	}
	else
	{
		Category = ParentGraphName;
	}

	for (UOptimusNodeGraph* SubGraph: InParentGraph->GetGraphs())
	{
		TSharedPtr<FOptimusSchemaAction_Graph> GraphAction = MakeShared<FOptimusSchemaAction_Graph>(SubGraph, Category);
		OutAllActions.AddAction(GraphAction);
	}

	for (const UOptimusNodeGraph* SubGraph: InParentGraph->GetGraphs())
	{
		if (!SubGraph->GetGraphs().IsEmpty())
		{
			CollectChildGraphActions(OutAllActions, SubGraph, Category);
		}
	}
}


void SOptimusEditorGraphExplorer::CollectStaticSections(TArray<int32>& StaticSectionIDs)
{
	if (IsShowingEmptySections())
	{
		StaticSectionIDs.Add(int32(EOptimusSchemaItemGroup::Graphs));
		StaticSectionIDs.Add(int32(EOptimusSchemaItemGroup::Bindings));
		StaticSectionIDs.Add(int32(EOptimusSchemaItemGroup::Resources));
		StaticSectionIDs.Add(int32(EOptimusSchemaItemGroup::Variables));
	}
}


FReply SOptimusEditorGraphExplorer::OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (!Editor.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FEdGraphSchemaAction> Action(InActions.Num() > 0 ? InActions[0] : nullptr);
	if (!Action.IsValid())
	{
		return FReply::Unhandled();
	}

	if (Action->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId())
	{
		FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(Action.Get());
		UOptimusComponentSourceBinding* Binding = Editor->GetDeformer()->ResolveComponentBinding(BindingAction->BindingName);

		return FReply::Handled().BeginDragDrop(FOptimusEditorGraphDragAction_Binding::New(Action, Binding));
	}
	else if (Action->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
	{
		FOptimusSchemaAction_Variable* VariableAction = static_cast<FOptimusSchemaAction_Variable*>(Action.Get());
		UOptimusVariableDescription* VariableDesc = Editor->GetDeformer()->ResolveVariable(VariableAction->VariableName);

		return FReply::Handled().BeginDragDrop(FOptimusEditorGraphDragAction_Variable::New(Action, VariableDesc));
	}
	else if (Action->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId())
	{
		FOptimusSchemaAction_Resource* ResourceAction = static_cast<FOptimusSchemaAction_Resource*>(Action.Get());
		UOptimusResourceDescription* ResourceDesc = Editor->GetDeformer()->ResolveResource(ResourceAction->ResourceName);

		return FReply::Handled().BeginDragDrop(FOptimusEditorGraphDragAction_Resource::New(Action, ResourceDesc));
	}

	return FReply::Unhandled();
}


FReply SOptimusEditorGraphExplorer::OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent)
{
	// There's no dragging of categories.
	return FReply::Unhandled();
}


void SOptimusEditorGraphExplorer::OnActionSelected(
	const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, 
	ESelectInfo::Type InSelectionType
	)
{
	if (!(InSelectionType == ESelectInfo::OnMouseClick || 
		  InSelectionType == ESelectInfo::OnKeyPress || 
		  InSelectionType == ESelectInfo::OnNavigation || 
		  InActions.Num() == 0))
	{
		return;
	}

    TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	TSharedPtr<FEdGraphSchemaAction> Action(InActions.Num() > 0 ? InActions[0] : nullptr);

	if (!ensure(Editor) || !Action.IsValid())
	{
		return;
	}

	if (Action->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
	{
		FOptimusSchemaAction_Graph* GraphAction = static_cast<FOptimusSchemaAction_Graph*>(Action.Get());
		UOptimusNodeGraph* NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

		if (NodeGraph)
		{
			Editor->InspectObject(NodeGraph);
		}
	}
	else if (Action->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId())
	{
		FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(Action.Get());
		UOptimusComponentSourceBinding* Binding = Editor->GetDeformer()->ResolveComponentBinding(BindingAction->BindingName);
		if (Binding)
		{
			Editor->InspectObject(Binding);
		}
	}
	else if (Action->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId())
	{
		FOptimusSchemaAction_Resource* ResourceAction = static_cast<FOptimusSchemaAction_Resource*>(Action.Get());
		UOptimusResourceDescription* Resource = Editor->GetDeformer()->ResolveResource(ResourceAction->ResourceName);
		if (Resource)
		{
			Editor->InspectObject(Resource);
		}
	}
	else if (Action->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
	{
		FOptimusSchemaAction_Variable* VariableAction = static_cast<FOptimusSchemaAction_Variable*>(Action.Get());
		UOptimusVariableDescription* Variable = Editor->GetDeformer()->ResolveVariable(VariableAction->VariableName);
		if (Variable)
		{
			Editor->InspectObject(Variable);
		}
	}
}

void SOptimusEditorGraphExplorer::OnActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions)
{
    TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	
	TSharedPtr<FEdGraphSchemaAction> Action(InActions.Num() > 0 ? InActions[0] : nullptr);

	if (!Editor || !Action.IsValid())
	{
		return;
	}

	if (Action->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
	{
		FOptimusSchemaAction_Graph* GraphAction = static_cast<FOptimusSchemaAction_Graph *>(Action.Get());
		UOptimusNodeGraph *NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

		if (NodeGraph)
		{
			Editor->SetEditGraph(NodeGraph);
		}
	}
}


TSharedPtr<SWidget> SOptimusEditorGraphExplorer::OnContextMenuOpening()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (!Editor.IsValid())
	{
		return TSharedPtr<SWidget>();
	}

	FMenuBuilder MenuBuilder(/*Close after selection*/true, Editor->GetToolkitCommands());

	if (SelectionHasContextMenu())
	{
		MenuBuilder.BeginSection("BasicOperations");
		MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().OpenGraph);
		MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().DeleteEntry);
		MenuBuilder.EndSection();
	}
	else
	{
		BuildAddNewMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}


void SOptimusEditorGraphExplorer::OnCategoryNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, TWeakPtr<struct FGraphActionNode> InAction)
{
}


bool SOptimusEditorGraphExplorer::CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const
{
	TSharedPtr<FGraphActionNode> SelectedNode = InSelectedNode.Pin();
	if (SelectedNode)
	{
		if (SelectedNode->IsActionNode())
		{
			return CanRenameAction(SelectedNode->Action);
		}
	}

	return false;
}


bool SOptimusEditorGraphExplorer::CanRenameAction(TSharedPtr<FEdGraphSchemaAction> InAction) const
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	if (Editor.IsValid() && InAction.IsValid())
	{
		if (InAction->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
		{
			FOptimusSchemaAction_Graph* GraphAction = static_cast<FOptimusSchemaAction_Graph *>(InAction.Get());
			UOptimusNodeGraph *NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

			if (NodeGraph)
			{
				// Only trigger graphs can be renamed.
				return NodeGraph->GetGraphType() == EOptimusNodeGraphType::ExternalTrigger;
			}
		}
		else if (InAction->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId() ||
			     InAction->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId() ||
				 InAction->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
		{
			// Resources and variables can always be renamed.
			return true;
		}
	}

	return false;
}


FText SOptimusEditorGraphExplorer::OnGetSectionTitle(int32 InSectionID)
{
	switch (EOptimusSchemaItemGroup(InSectionID))
	{
	case EOptimusSchemaItemGroup::InvalidGroup:
		ensureMsgf(false, TEXT("Invalid group"));
		break;

	case EOptimusSchemaItemGroup::Graphs:
		return NSLOCTEXT("GraphActionNode", "Graphs", "Graphs");

	case EOptimusSchemaItemGroup::Bindings:
		return NSLOCTEXT("GraphActionNode", "ComponentBindings", "Component Bindings");

	case EOptimusSchemaItemGroup::Resources:
		return NSLOCTEXT("GraphActionNode", "Resources", "Resources");

	case EOptimusSchemaItemGroup::Variables:
		return NSLOCTEXT("GraphActionNode", "Variables", "Variables");
	}

	return FText::GetEmpty();
}


TSharedRef<SWidget> SOptimusEditorGraphExplorer::OnGetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID)
{
	FText AddNewTooltipText;

	switch (EOptimusSchemaItemGroup(InSectionID))
	{
	case EOptimusSchemaItemGroup::InvalidGroup:
		ensureMsgf(false, TEXT("Invalid group"));
		break;

	case EOptimusSchemaItemGroup::Graphs:
		AddNewTooltipText = LOCTEXT("AddNewGraphTooltip", "Create a new graph");
		break;

	case EOptimusSchemaItemGroup::Bindings:
		AddNewTooltipText = LOCTEXT("AddNewBindingTooltip", "Create a new component binding");
		break;
		
	case EOptimusSchemaItemGroup::Resources:
		AddNewTooltipText = LOCTEXT("AddNewResourceTooltip", "Create a new shader resource");
		break;

	case EOptimusSchemaItemGroup::Variables:
		AddNewTooltipText = LOCTEXT("AddNewVariableTooltip", "Create a new externally visible variable");
		break;
	}

	TWeakPtr<SWidget> WeakRowWidget(RowWidget);

	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	TSharedPtr<FUICommandList> ToolKitCommandList;
	if (Editor)
	{
		ToolKitCommandList = Editor->GetToolkitCommands();
	}

	TSharedPtr<FUICommandInfo> AddCommand;
	TSharedPtr<SWidget> AddMenuWidget;
	
	switch (EOptimusSchemaItemGroup(InSectionID))
	{
	case EOptimusSchemaItemGroup::InvalidGroup:
		checkNoEntry();
		return SNullWidget::NullWidget;

	case EOptimusSchemaItemGroup::Graphs:
		{
			FMenuBuilder MenuBuilder(true, ToolKitCommandList);
			MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateSetupGraph);
			MenuBuilder.AddMenuEntry(FOptimusEditorGraphExplorerCommands::Get().CreateTriggerGraph);
			AddMenuWidget = MenuBuilder.MakeWidget();
		}
		break;

	case EOptimusSchemaItemGroup::Bindings:
		{
			FMenuBuilder MenuBuilder(true, ToolKitCommandList);
			TArray<const UOptimusComponentSource*> Sources = UOptimusComponentSource::GetAllSources();
			Algo::Sort(Sources, [](const UOptimusComponentSource* ItemA, const UOptimusComponentSource* ItemB)
			{
				return ItemA->GetDisplayName().CompareTo(ItemB->GetDisplayName()) < 0;
			});
			for (const UOptimusComponentSource* Source: Sources)
			{
				FUIAction Action(FExecuteAction::CreateSP(this, &SOptimusEditorGraphExplorer::OnCreateBinding, Source));
				MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("AddComponentBinding", "Create {0} Binding"), Source->GetDisplayName()), FText(), FSlateIcon(), Action);
			}
			AddMenuWidget = MenuBuilder.MakeWidget();
		}
		break;

	case EOptimusSchemaItemGroup::Resources:
		AddCommand = FOptimusEditorGraphExplorerCommands::Get().CreateResource;
		break;

	case EOptimusSchemaItemGroup::Variables:
		AddCommand = FOptimusEditorGraphExplorerCommands::Get().CreateVariable;
		break;
	}

	if (AddMenuWidget.IsValid())
	{
		return SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(FMargin(1, 0))
		    .ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
			.OnGetMenuContent_Lambda([AddMenuWidget]() { return AddMenuWidget.ToSharedRef(); })
			.IsEnabled(this, &SOptimusEditorGraphExplorer::CanAddNewElementToSection, InSectionID)
			.HasDownArrow(false)
		    .HAlign(HAlign_Center)
		    .VAlign(VAlign_Center)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.ToolTipText(AddNewTooltipText)
			];
	}
	else if (AddCommand.IsValid())
	{
		return SNew(SButton)
		    .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(FMargin(1, 0))
		    .ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		    .OnClicked(this, &SOptimusEditorGraphExplorer::OnAddButtonClickedOnSection, InSectionID)
		    .IsEnabled(this, &SOptimusEditorGraphExplorer::CanAddNewElementToSection, InSectionID)
		    .HAlign(HAlign_Center)
		    .VAlign(VAlign_Center)
		    [
				SNew(SImage)
		    	.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
		    	.ColorAndOpacity(FSlateColor::UseForeground())
		        .ToolTipText(AddNewTooltipText)
			];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}


// Called when the + button on a section is clicked.
FReply SOptimusEditorGraphExplorer::OnAddButtonClickedOnSection(int32 InSectionID)
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor.IsValid())
	{
		switch (EOptimusSchemaItemGroup(InSectionID))
		{
		case EOptimusSchemaItemGroup::Graphs:
			// Handled by the submenu.
			break;

		case EOptimusSchemaItemGroup::Bindings:
			Editor->GetToolkitCommands()->ExecuteAction(FOptimusEditorGraphExplorerCommands::Get().CreateBinding.ToSharedRef());
			break;
			
		case EOptimusSchemaItemGroup::Resources:
			Editor->GetToolkitCommands()->ExecuteAction(FOptimusEditorGraphExplorerCommands::Get().CreateResource.ToSharedRef());
			break;

		case EOptimusSchemaItemGroup::Variables:
			Editor->GetToolkitCommands()->ExecuteAction(FOptimusEditorGraphExplorerCommands::Get().CreateVariable.ToSharedRef());
			break;
		}
	}

	return FReply::Handled();
}


bool SOptimusEditorGraphExplorer::CanAddNewElementToSection(int32 InSectionID) const
{
	return true;
}


void SOptimusEditorGraphExplorer::OnToggleShowEmptySections()
{
	// FIXME: Move to preferences
	bShowEmptySections = !bShowEmptySections;

	Refresh();
}


bool SOptimusEditorGraphExplorer::IsShowingEmptySections() const
{
	return bShowEmptySections;
}


TSharedPtr<FEdGraphSchemaAction> SOptimusEditorGraphExplorer::GetFirstSelectedAction(
	FName InTypeName
	) const
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	TSharedPtr<FEdGraphSchemaAction> SelectedAction(SelectedActions.Num() > 0 ? SelectedActions[0] : nullptr);
	if (SelectedAction.IsValid() && 
		(SelectedAction->GetTypeId() == InTypeName || InTypeName == NAME_None))
	{
		return SelectedActions[0];
	}

	return TSharedPtr<FEdGraphSchemaAction>();
}


void SOptimusEditorGraphExplorer::OnOpenGraph()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	if (Editor)
	{
		FOptimusSchemaAction_Graph* GraphAction = SelectionAsType<FOptimusSchemaAction_Graph>();
		UOptimusNodeGraph* NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

		if (NodeGraph)
		{
			Editor->SetEditGraph(NodeGraph);
		}
	}
}


bool SOptimusEditorGraphExplorer::CanOpenGraph()
{
	return SelectionAsType<FOptimusSchemaAction_Graph>() != nullptr;
}


void SOptimusEditorGraphExplorer::OnCreateSetupGraph()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (Deformer)
		{
			Deformer->AddSetupGraph();
		}
	}	
}


bool SOptimusEditorGraphExplorer::CanCreateSetupGraph()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		const TArray<UOptimusNodeGraph *> Graphs = Editor->GetDeformer()->GetGraphs();
		return Graphs[0]->GetGraphType() != EOptimusNodeGraphType::Setup;
	}

	return false;
}


void SOptimusEditorGraphExplorer::OnCreateTriggerGraph()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (Deformer)
		{
			Deformer->AddTriggerGraph(TEXT("TriggerGraph"));
		}
	}
}


bool SOptimusEditorGraphExplorer::CanCreateTriggerGraph()
{
	return OptimusEditor.IsValid();
}


void SOptimusEditorGraphExplorer::OnCreateBinding(const UOptimusComponentSource* InComponentSource)
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (Deformer)
		{
			// Go with the default type.
			Deformer->AddComponentBinding(InComponentSource);
		}
	}
}


bool SOptimusEditorGraphExplorer::CanCreateResource()
{
	return OptimusEditor.IsValid();
}


void SOptimusEditorGraphExplorer::OnCreateResource()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (Deformer)
		{
			// Go with the default type.
			Deformer->AddResource(FOptimusDataTypeRef());
		}
	}
}


bool SOptimusEditorGraphExplorer::CanCreateBinding()
{
	return OptimusEditor.IsValid();
}


void SOptimusEditorGraphExplorer::OnCreateVariable()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (Deformer)
		{
			// Go with the default type.
			Deformer->AddVariable(FOptimusDataTypeRef());
		}
	}
}


bool SOptimusEditorGraphExplorer::CanCreateVariable()
{
	return OptimusEditor.IsValid();
}


void SOptimusEditorGraphExplorer::OnDeleteEntry()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	if (Editor)
	{
		UOptimusDeformer* Deformer = Editor->GetDeformer();

		if (FOptimusSchemaAction_Graph* GraphAction = SelectionAsType<FOptimusSchemaAction_Graph>())
		{
			UOptimusNodeGraph* NodeGraph = Deformer->ResolveGraphPath(GraphAction->GraphPath);

			if (NodeGraph)
			{
				Deformer->RemoveGraph(NodeGraph);
			}
		}
		else if (FOptimusSchemaAction_Binding* BindingAction = SelectionAsType<FOptimusSchemaAction_Binding>())
		{
			UOptimusComponentSourceBinding* Binding = Deformer->ResolveComponentBinding(BindingAction->BindingName);

			if (Binding)
			{
				Deformer->RemoveComponentBinding(Binding);
			}
		}
		else if (FOptimusSchemaAction_Resource* ResourceAction = SelectionAsType<FOptimusSchemaAction_Resource>())
		{
			UOptimusResourceDescription* Resource = Deformer->ResolveResource(ResourceAction->ResourceName);

			if (Resource)
			{
				Deformer->RemoveResource(Resource);
			}
		}
		else if (FOptimusSchemaAction_Variable* VariableAction = SelectionAsType<FOptimusSchemaAction_Variable>())
		{
			UOptimusVariableDescription* Variable = Editor->GetDeformer()->ResolveVariable(VariableAction->VariableName);
			if (Variable)
			{
				Deformer->RemoveVariable(Variable);
			}
		}
	}
}


bool SOptimusEditorGraphExplorer::CanDeleteEntry()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	if (Editor)
	{
		if (FOptimusSchemaAction_Graph* GraphAction = SelectionAsType<FOptimusSchemaAction_Graph>())
		{
			UOptimusNodeGraph* NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

			if (NodeGraph)
			{
				// Can't delete the update graph or bad things happen.
				return NodeGraph->GetGraphType() != EOptimusNodeGraphType::Update;
			}
		}
		else if (SelectionAsType<FOptimusSchemaAction_Binding>() ||
				 SelectionAsType<FOptimusSchemaAction_Resource>() || 
				 SelectionAsType<FOptimusSchemaAction_Variable>())
		{
			return true;
		}
	}
	return false;
}


void SOptimusEditorGraphExplorer::OnRenameEntry()
{
	GraphActionMenu->OnRequestRenameOnActionNode();
}


bool SOptimusEditorGraphExplorer::CanRenameEntry()
{
	return CanRenameAction(GetFirstSelectedAction(NAME_None));
}


bool SOptimusEditorGraphExplorer::IsEditingMode() const
{
	return true;
}


bool SOptimusEditorGraphExplorer::SelectionHasContextMenu() const
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);
	return SelectedActions.Num() > 0;
}


#undef LOCTEXT_NAMESPACE
