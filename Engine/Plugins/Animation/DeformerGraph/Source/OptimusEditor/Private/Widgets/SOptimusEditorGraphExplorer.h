// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"


class UOptimusComponentSource;
class UOptimusNodeGraph;
class FMenuBuilder;
class FOptimusEditor;
class FUICommandList;
class SGraphActionMenu;
class SSearchBox;
class SWidget;
class UOptimusEditorGraph;
struct FCreateWidgetForActionData;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;

class FOptimusEditorGraphExplorerCommands : public TCommands<FOptimusEditorGraphExplorerCommands>
{
public:
	FOptimusEditorGraphExplorerCommands();

	TSharedPtr<FUICommandInfo> OpenGraph;
	TSharedPtr<FUICommandInfo> CreateSetupGraph;
	TSharedPtr<FUICommandInfo> CreateTriggerGraph;

	TSharedPtr<FUICommandInfo> CreateBinding;
	TSharedPtr<FUICommandInfo> CreateResource;
	TSharedPtr<FUICommandInfo> CreateVariable;

	TSharedPtr<FUICommandInfo> DeleteEntry;

	void RegisterCommands() override;
};

class SOptimusEditorGraphExplorer : 
	public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOptimusEditorGraphExplorer) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InOptimusEditor);
	~SOptimusEditorGraphExplorer();

	// Refresh the graph action menu.
	void Refresh();
	void SetFocusedGraph(UOptimusEditorGraph* InEditorGraph) { EditorGraph = InEditorGraph; }

	// SWidget overrides

	void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void RegisterCommands();
	void CreateWidgets();

	TSharedRef<SWidget> CreateAddNewMenuWidget();

	void BuildAddNewMenu(FMenuBuilder& MenuBuilder);

	TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData);
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
	void CollectChildGraphActions(FGraphActionListBuilderBase& OutAllActions, const UOptimusNodeGraph* InParentGraph, const FText& InParentGraphCategory);
	void CollectStaticSections(TArray<int32>& StaticSectionIDs);

	FReply OnActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& MouseEvent);
	FReply OnCategoryDragged(const FText& InCategory, const FPointerEvent& MouseEvent);
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, ESelectInfo::Type InSelectionType);

	void OnActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions);
	TSharedPtr<SWidget> OnContextMenuOpening();
	void OnCategoryNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit, TWeakPtr<struct FGraphActionNode> InAction);
	bool CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNode) const;
	bool CanRenameAction(TSharedPtr<FEdGraphSchemaAction> InAction) const;


	FText OnGetSectionTitle(int32 InSectionID);
	TSharedRef<SWidget> OnGetSectionWidget(TSharedRef<SWidget> RowWidget, int32 InSectionID);

	FReply OnAddButtonClickedOnSection(int32 InSectionID);
	bool CanAddNewElementToSection(int32 InSectionID) const;

	/** Support functions for view options for Show Empty Sections */
	void OnToggleShowEmptySections();
	bool IsShowingEmptySections() const;

	template<typename SchemaActionType> 
	SchemaActionType* SelectionAsType() const
	{
	    return static_cast<SchemaActionType*>(GetFirstSelectedAction(SchemaActionType::StaticGetTypeId()).Get());
	}

	TSharedPtr<FEdGraphSchemaAction> GetFirstSelectedAction(
		FName InTypeName
		) const;

	// Command functions
	void OnOpenGraph();
	bool CanOpenGraph();

	void OnCreateSetupGraph();
	bool CanCreateSetupGraph();

	void OnCreateTriggerGraph();
	bool CanCreateTriggerGraph();

	void OnCreateBinding(const UOptimusComponentSource* InComponentSource);
	bool CanCreateBinding();
	
	void OnCreateResource();
	bool CanCreateResource();

	void OnCreateVariable();
	bool CanCreateVariable();

	void OnDeleteEntry();
	bool CanDeleteEntry();

	void OnRenameEntry();
	bool CanRenameEntry();

	bool IsEditingMode() const;

	bool SelectionHasContextMenu() const;

	TWeakPtr<FOptimusEditor> OptimusEditor;
		
	// FIXME: Get from the editor?
	UOptimusEditorGraph *EditorGraph;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	TSharedPtr<SSearchBox> FilterBox;

	// On demand refresh requests prior to tick.
	bool bNeedsRefresh;

	bool bShowEmptySections = true;
};

