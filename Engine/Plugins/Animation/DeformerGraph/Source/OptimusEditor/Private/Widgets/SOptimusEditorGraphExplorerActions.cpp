// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusEditorGraphExplorerActions.h"

#include "OptimusComponentSource.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphSchema.h"

#include "OptimusNodeGraph.h"
#include "OptimusResourceDescription.h"
#include "OptimusVariableDescription.h"

#include "EdGraph/EdGraph.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"


#define LOCTEXT_NAMESPACE "OptimusExplorerActions"

static bool GetIconAndColorFromDataType(
	FOptimusDataTypeHandle InDataType,
    const FSlateBrush*& OutPrimaryBrush,
    FSlateColor& OutIconColor,
    FSlateBrush const*& OutSecondaryBrush,
    FSlateColor& OutSecondaryColor
	)
{
	if (InDataType)
	{
		FEdGraphPinType PinType = UOptimusEditorGraphSchema::GetPinTypeFromDataType(InDataType);

		OutPrimaryBrush = UOptimusEditorGraphSchema::GetIconFromPinType(PinType);
		OutIconColor = UOptimusEditorGraphSchema::GetColorFromPinType(PinType);
		OutSecondaryBrush = nullptr;
		return true;
	}
	else
	{
		return false;
	}
}


TSharedRef<FOptimusEditorGraphDragAction_Binding> FOptimusEditorGraphDragAction_Binding::New(
	TSharedPtr<FEdGraphSchemaAction> InAction,
	UOptimusComponentSourceBinding* InBinding
	)
{
	TSharedRef<FOptimusEditorGraphDragAction_Binding> Operation = MakeShared<FOptimusEditorGraphDragAction_Binding>();

	Operation->SourceAction = InAction;
	Operation->WeakBinding = InBinding;
	Operation->Construct();
	return Operation;
}


FReply FOptimusEditorGraphDragAction_Binding::DroppedOnPanel(
	const TSharedRef<SWidget>& InPanel,
	const FVector2f& InScreenPosition,
	const FVector2f& InGraphPosition,
	UEdGraph& InGraph
	)
{
	if (!InGraph.GetSchema()->IsA<UOptimusEditorGraphSchema>())
	{
		return FReply::Unhandled();
	}

	UOptimusComponentSourceBinding* Binding = WeakBinding.Get();
	if (!Binding)
	{
		return FReply::Unhandled();
	}

	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(&InGraph);
	if (!ensure(Graph))
	{
		return FReply::Unhandled();
	}

	UOptimusNodeGraph* ModelGraph = Graph->GetModelGraph();

	ModelGraph->AddComponentBindingGetNode(Binding, FDeprecateSlateVector2D(InGraphPosition));

	return FReply::Handled();
}


TSharedRef<FOptimusEditorGraphDragAction_Variable> FOptimusEditorGraphDragAction_Variable::New(
	TSharedPtr<FEdGraphSchemaAction> InAction, 
	UOptimusVariableDescription* InVariableDesc
	)
{
	TSharedRef<FOptimusEditorGraphDragAction_Variable> Operation = MakeShared<FOptimusEditorGraphDragAction_Variable>();

	Operation->SourceAction = InAction;
	Operation->WeakVariableDesc = InVariableDesc;
	Operation->Construct();
	return Operation;
}


void FOptimusEditorGraphDragAction_Variable::HoverTargetChanged()
{
	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}


FReply FOptimusEditorGraphDragAction_Variable::DroppedOnPanel(
	const TSharedRef<SWidget>& InPanel, 
	const FVector2f& InScreenPosition,
	const FVector2f& InGraphPosition,
	UEdGraph& InGraph
	)
{
	if (!InGraph.GetSchema()->IsA<UOptimusEditorGraphSchema>())
	{
		return FReply::Unhandled();
	}

	UOptimusVariableDescription* VariableDesc = WeakVariableDesc.Get();
	if (!VariableDesc)
	{
		return FReply::Unhandled();
	}

	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(&InGraph);
	if (!ensure(Graph))
	{
		return FReply::Unhandled();
	}

	UOptimusNodeGraph* ModelGraph = Graph->GetModelGraph();

	ModelGraph->AddVariableGetNode(VariableDesc, FDeprecateSlateVector2D(InGraphPosition));

	return FReply::Handled();
}


void FOptimusEditorGraphDragAction_Variable::GetDefaultStatusSymbol(const FSlateBrush*& OutPrimaryBrush, FSlateColor& OutIconColor, FSlateBrush const*& OutSecondaryBrush, FSlateColor& OutSecondaryColor) const
{
	UOptimusVariableDescription* VariableDesc = WeakVariableDesc.Get();
	if (!VariableDesc ||
	    !GetIconAndColorFromDataType(VariableDesc->DataType.Resolve(), OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor))
	{
		return FGraphSchemaActionDragDropAction::GetDefaultStatusSymbol(OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor);
	}
}


TSharedRef<FOptimusEditorGraphDragAction_Resource> FOptimusEditorGraphDragAction_Resource::New(
	TSharedPtr<FEdGraphSchemaAction> InAction, 
	UOptimusResourceDescription* InResourceDesc
	)
{
	TSharedRef<FOptimusEditorGraphDragAction_Resource> Operation = MakeShared<FOptimusEditorGraphDragAction_Resource>();

	Operation->SourceAction = InAction;
	Operation->WeakResourceDesc = InResourceDesc;
	Operation->Construct();
	return Operation;
}


void FOptimusEditorGraphDragAction_Resource::HoverTargetChanged()
{
	FGraphSchemaActionDragDropAction::HoverTargetChanged();
}


FReply FOptimusEditorGraphDragAction_Resource::DroppedOnPanel(
	const TSharedRef<SWidget>& InPanel, 
	const FVector2f& InScreenPosition,
	const FVector2f& InGraphPosition,
	UEdGraph& InGraph
	)
{
	if (!InGraph.GetSchema()->IsA<UOptimusEditorGraphSchema>())
	{
		return FReply::Unhandled();
	}

	UOptimusResourceDescription *ResourceDesc = WeakResourceDesc.Get();
	if (!ResourceDesc)
	{
		return FReply::Unhandled();
	}

	UOptimusEditorGraph *Graph = Cast<UOptimusEditorGraph>(&InGraph);
	if (!ensure(Graph))
	{
		return FReply::Unhandled();
	}

	UOptimusNodeGraph *ModelGraph = Graph->GetModelGraph();

	FMenuBuilder MenuBuilder(true, NULL);
	const FText ResourceNameText = FText::FromName(ResourceDesc->ResourceName);

	MenuBuilder.BeginSection("OptimusResourceDroppedOn", ResourceNameText);

	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("CreateResource", "Get/Set {0}"), ResourceNameText),
		FText::Format(LOCTEXT("CreateResourceToolTip", "Create Getter/Setter for resource '{0}'"), ResourceNameText),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ModelGraph, ResourceDesc, InGraphPosition]() {
				ModelGraph->AddResourceNode(ResourceDesc, FDeprecateSlateVector2D(InGraphPosition));
			}),
			FCanExecuteAction()));

	MenuBuilder.AddMenuEntry(
	    FText::Format(LOCTEXT("CreateGetResource", "Get {0}"), ResourceNameText),
	    FText::Format(LOCTEXT("CreateGetResourceToolTip", "Create Getter for resource '{0}'"), ResourceNameText),
	    FSlateIcon(),
	    FUIAction(
	        FExecuteAction::CreateLambda([ModelGraph, ResourceDesc, InGraphPosition]() { 
				ModelGraph->AddResourceGetNode(ResourceDesc, FDeprecateSlateVector2D(InGraphPosition)); 
			}), 
			FCanExecuteAction()));

	MenuBuilder.AddMenuEntry(
	    FText::Format(LOCTEXT("CreateSetResource", "Set {0}"), ResourceNameText),
	    FText::Format(LOCTEXT("CreateSetResourceToolTip", "Create Setter for resource '{0}'"), ResourceNameText),
	    FSlateIcon(),
	    FUIAction(
	        FExecuteAction::CreateLambda([ModelGraph, ResourceDesc, InGraphPosition]() {
		        ModelGraph->AddResourceSetNode(ResourceDesc, FDeprecateSlateVector2D(InGraphPosition));
	        }), 
	        FCanExecuteAction()));

	MenuBuilder.EndSection();

	TSharedRef<SWidget> PanelWidget = InPanel;
	// Show dialog to choose getter vs setter
	FSlateApplication::Get().PushMenu(
	    PanelWidget,
	    FWidgetPath(),
	    MenuBuilder.MakeWidget(),
	    InScreenPosition,
	    FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

	return FReply::Handled();
}


void FOptimusEditorGraphDragAction_Resource::GetDefaultStatusSymbol(
	const FSlateBrush*& OutPrimaryBrush, 
	FSlateColor& OutIconColor, 
	FSlateBrush const*& OutSecondaryBrush, 
	FSlateColor& OutSecondaryColor
	) const
{
	UOptimusResourceDescription *ResourceDesc = WeakResourceDesc.Get();
	if (!ResourceDesc || 
		!GetIconAndColorFromDataType(ResourceDesc->DataType.Resolve(), OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor) )
	{
		return FGraphSchemaActionDragDropAction::GetDefaultStatusSymbol(OutPrimaryBrush, OutIconColor, OutSecondaryBrush, OutSecondaryColor);
	}
}



#undef LOCTEXT_NAMESPACE
