// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GraphEditorDragDropAction.h"

class UOptimusComponentSourceBinding;
class UOptimusNodeGraph;
class UOptimusResourceDescription;
class UOptimusVariableDescription;
struct FEdGraphSchemaAction;


class FOptimusEditorGraphDragAction_Binding :
	public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FOptimusEditorGraphDragAction_Binding, FGraphSchemaActionDragDropAction)

	static TSharedRef<FOptimusEditorGraphDragAction_Binding> New(
		TSharedPtr<FEdGraphSchemaAction> InAction, 
		UOptimusComponentSourceBinding *InBinding
		);

	// FGraphEditorDragDropAction overrides
	FReply DroppedOnPanel(
		const TSharedRef<SWidget>& InPanel, 
		const FVector2f& InScreenPosition,
		const FVector2f& InGraphPosition,
		UEdGraph& InGraph
		) override;
	
private:
	TWeakObjectPtr<UOptimusComponentSourceBinding> WeakBinding;
};


class FOptimusEditorGraphDragAction_Variable :
	public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FOptimusEditorGraphDragAction_Variable, FGraphSchemaActionDragDropAction)

	static TSharedRef<FOptimusEditorGraphDragAction_Variable> New(
		TSharedPtr<FEdGraphSchemaAction> InAction, 
		UOptimusVariableDescription *InVariableDesc
		);

	// FGraphEditorDragDropAction overrides
	void HoverTargetChanged() override;
	FReply DroppedOnPanel(
		const TSharedRef<SWidget>& InPanel, 
		const FVector2f& InScreenPosition,
		const FVector2f& InGraphPosition,
		UEdGraph& InGraph
		) override;
	
protected:
	// 
	void GetDefaultStatusSymbol(
		const FSlateBrush*& OutPrimaryBrush, 
		FSlateColor& OutIconColor, 
		FSlateBrush const*& OutSecondaryBrush, 
		FSlateColor& OutSecondaryColor
		) const override;


private:
	TWeakObjectPtr<UOptimusVariableDescription> WeakVariableDesc;
};



class FOptimusEditorGraphDragAction_Resource : 
	public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FOptimusEditorGraphDragAction_Resource, FGraphSchemaActionDragDropAction)

	static TSharedRef<FOptimusEditorGraphDragAction_Resource> New(
	    TSharedPtr<FEdGraphSchemaAction> InAction,
	    UOptimusResourceDescription* InResourceDesc);

	// FGraphEditorDragDropAction overrides
	void HoverTargetChanged() override;
	FReply DroppedOnPanel(
	    const TSharedRef<SWidget>& InPanel,
		const FVector2f& InScreenPosition,
		const FVector2f& InGraphPosition,
	    UEdGraph& InGraph) override;

protected:
	//
	void GetDefaultStatusSymbol(
	    const FSlateBrush*& OutPrimaryBrush,
	    FSlateColor& OutIconColor,
	    FSlateBrush const*& OutSecondaryBrush,
	    FSlateColor& OutSecondaryColor) const override;


private:
	TWeakObjectPtr<UOptimusResourceDescription> WeakResourceDesc;
};
