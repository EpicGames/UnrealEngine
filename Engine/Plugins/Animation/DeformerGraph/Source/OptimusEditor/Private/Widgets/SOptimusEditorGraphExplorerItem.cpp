// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusEditorGraphExplorerItem.h"

#include "OptimusActionStack.h"
#include "OptimusEditor.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphSchema.h"
#include "OptimusEditorGraphSchemaActions.h"
#include "OptimusNameValidator.h"

#include "IOptimusNodeGraphCollectionOwner.h"
#include "OptimusDeformer.h"
#include "OptimusNodeGraph.h"
#include "OptimusResourceDescription.h"
#include "OptimusVariableDescription.h"
#include "PacketHandler.h"
#include "SOptimusDataTypeSelector.h"
#include "Styling/SlateIconFinder.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"


class SResourceDataTypeSelectorHelper :
	public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SResourceDataTypeSelectorHelper ) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UOptimusResourceDescription* InResource, TAttribute<bool> bInIsReadOnly)	
	{
		WeakResource = InResource;

		ChildSlot
		[
			SNew(SOptimusDataTypeSelector)
			.CurrentDataType(this, &SResourceDataTypeSelectorHelper::OnGetDataType)
			.UsageMask(EOptimusDataTypeUsageFlags::Resource)
			.ViewType(SOptimusDataTypeSelector::EViewType::IconOnly)
			.bViewOnly(bInIsReadOnly.Get())			// FIXME: May be dynamic.
			.OnDataTypeChanged(this, &SResourceDataTypeSelectorHelper::OnDataTypeChanged)
		];
	}

private:
	FOptimusDataTypeHandle OnGetDataType() const
	{
		if (const UOptimusResourceDescription *Resource = WeakResource.Get())
		{
			return Resource->DataType.Resolve();
		}
		else
		{
			return FOptimusDataTypeHandle();
		}
	}

	void OnDataTypeChanged(FOptimusDataTypeHandle InDataType)
	{
		if (UOptimusResourceDescription *Resource = WeakResource.Get())
		{
			Resource->GetOwningDeformer()->SetResourceDataType(Resource, InDataType);
		}
	}

	TWeakObjectPtr<UOptimusResourceDescription> WeakResource;
};


class SVariableDataTypeSelectorHelper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVariableDataTypeSelectorHelper) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UOptimusVariableDescription* InVariable, TAttribute<bool> bInIsReadOnly)
	{
		WeakVariable = InVariable;

		ChildSlot
		    [SNew(SOptimusDataTypeSelector)
		            .CurrentDataType(this, &SVariableDataTypeSelectorHelper::OnGetDataType)
		            .UsageMask(EOptimusDataTypeUsageFlags::Variable | EOptimusDataTypeUsageFlags::Property)
		            .ViewType(SOptimusDataTypeSelector::EViewType::IconOnly)
		            .bViewOnly(bInIsReadOnly.Get()) // FIXME: May be dynamic.
		            .OnDataTypeChanged(this, &SVariableDataTypeSelectorHelper::OnDataTypeChanged)];
	}

private:
	FOptimusDataTypeHandle OnGetDataType() const
	{
		UOptimusVariableDescription* Variable = WeakVariable.Get();
		if (Variable)
		{
			return Variable->DataType.Resolve();
		}
		else
		{
			return FOptimusDataTypeHandle();
		}
	}

	void OnDataTypeChanged(FOptimusDataTypeHandle InDataType)
	{
		if (UOptimusVariableDescription *Variable = WeakVariable.Get())
		{
			Variable->GetOwningDeformer()->SetVariableDataType(Variable, InDataType);
		}
	}

	TWeakObjectPtr<UOptimusVariableDescription> WeakVariable;
};


class SBindingSourceSelectorHelper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBindingSourceSelectorHelper) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UOptimusComponentSourceBinding* InBinding, TAttribute<bool> bInEnabled)
	{
		UOptimusComponentSource* CurrentSource = InBinding->ComponentType->GetDefaultObject<UOptimusComponentSource>();
		TSharedPtr<FString> CurrentSelection;
		for (const UOptimusComponentSource* Source: UOptimusComponentSource::GetAllSources())
		{
			if (!InBinding->IsPrimaryBinding() || Source->IsUsableAsPrimarySource())
			{
				TSharedPtr<FString> SourceName = MakeShared<FString>(Source->GetDisplayName().ToString());
				if (Source == CurrentSource)
				{
					CurrentSelection = SourceName;
				}
				ComponentSources.Add(SourceName);
			}
		}
		Algo::Sort(ComponentSources, [](TSharedPtr<FString> ItemA, TSharedPtr<FString> ItemB)
		{
			return ItemA->Compare(*ItemB) < 0;
		});
		
		WeakBinding = InBinding;

		ChildSlot
		[
			SNew(STextComboBox)
				.OptionsSource(&ComponentSources)
				.InitiallySelectedItem(CurrentSelection)
				.OnSelectionChanged(this, &SBindingSourceSelectorHelper::ComponentSourceChanged)
		];

		SetEnabled(TAttribute<bool>::Create([bInEnabled]() { return !bInEnabled.Get(); }));
	}

private:
	void ComponentSourceChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
	{
		UOptimusComponentSourceBinding* Binding = WeakBinding.Get();
		UOptimusDeformer* Deformer = Binding->GetOwningDeformer();
		
		for (const UOptimusComponentSource* Source: UOptimusComponentSource::GetAllSources())
		{
			if (*Selection == Source->GetDisplayName().ToString())
			{
				Deformer->SetComponentBindingSource(Binding, Source);
				return;
			}
		}
	}
	
	TArray<TSharedPtr<FString>> ComponentSources;
	
	TWeakObjectPtr<UOptimusComponentSourceBinding> WeakBinding;
};


void SOptimusEditorGraphExplorerItem::Construct(
	const FArguments& InArgs, 
	FCreateWidgetForActionData* const InCreateData, 
	TWeakPtr<FOptimusEditor> InOptimusEditor
	)
{
	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;
	OptimusEditor = InOptimusEditor;

	TWeakPtr<FEdGraphSchemaAction> WeakGraphAction = GraphAction;
	const bool bIsReadOnlyCreate = InCreateData->bIsReadOnly;

	FSlateFontInfo NameFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	if (GraphAction->GetTypeId() != FOptimusSchemaAction_Binding::StaticGetTypeId())
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				CreateIconWidget(InCreateData, bIsReadOnlyCreate)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(/* horizontal */ 3.0f, /* vertical */ 3.0f)
			[
				CreateTextSlotWidget(InCreateData, bIsReadOnlyCreate)
			]		
		];	
	}
	else
	{
		TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
		FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(GraphAction.Get());
		IOptimusPathResolver* PathResolver = Editor->GetDeformerInterface<IOptimusPathResolver>();
		UOptimusComponentSourceBinding* Binding = PathResolver->ResolveComponentBinding(BindingAction->BindingName);
		
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 2.0f)
			[
				CreateIconWidget(InCreateData, bIsReadOnlyCreate)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(0.6f)
			.VAlign(VAlign_Center)
			.Padding(/* horizontal */ 3.0f, /* vertical */ 0.0f)
			[
				CreateTextSlotWidget(InCreateData, bIsReadOnlyCreate || Binding->IsPrimaryBinding())
			]		
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBindingSourceSelectorHelper, Binding, bIsReadOnlyCreate)
			]
		];	
	}
}


TSharedRef<SWidget> SOptimusEditorGraphExplorerItem::CreateIconWidget(
	FCreateWidgetForActionData* const InCreateData,
	TAttribute<bool> bInIsReadOnly)
{
	TSharedPtr<FEdGraphSchemaAction> Action = InCreateData->Action;
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	
	TSharedPtr<SWidget> IconWidget;

	if (ensure(Action) && ensure(Editor))
	{
		IOptimusPathResolver* PathResolver = Editor->GetDeformerInterface<IOptimusPathResolver>();
		if (Action->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Graph* GraphAction = static_cast<FOptimusSchemaAction_Graph*>(Action.Get());
			const UOptimusNodeGraph* NodeGraph = PathResolver->ResolveGraphPath(GraphAction->GraphPath);
			if (ensure(NodeGraph))
			{
				IconWidget = SNew(SImage)
					.Image(UOptimusEditorGraph::GetGraphTypeIcon(NodeGraph));
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Resource* ResourceAction = static_cast<FOptimusSchemaAction_Resource*>(Action.Get());
			UOptimusResourceDescription* Resource = PathResolver->ResolveResource(ResourceAction->ResourceName);
			if (ensure(Resource))
			{
				
				IconWidget = SNew(SResourceDataTypeSelectorHelper, Resource, bInIsReadOnly);
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Variable* VariableAction = static_cast<FOptimusSchemaAction_Variable*>(Action.Get());
			UOptimusVariableDescription* Variable = PathResolver->ResolveVariable(VariableAction->VariableName);
			if (ensure(Variable))
			{
				IconWidget = SNew(SVariableDataTypeSelectorHelper, Variable, bInIsReadOnly);
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(Action.Get());
			const UOptimusComponentSourceBinding* Binding = PathResolver->ResolveComponentBinding(BindingAction->BindingName);
			if (ensure(Binding))
			{
				IconWidget = SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.Image(FSlateIconFinder::FindIconBrushForClass(Binding->GetComponentSource()->GetComponentClass(), TEXT("SCS.Component")));
			}
		}
	}

	if (IconWidget.IsValid())
	{
		return IconWidget.ToSharedRef();
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

TSharedRef<SWidget> SOptimusEditorGraphExplorerItem::CreateTextSlotWidget(
	FCreateWidgetForActionData* const InCreateData, 
	TAttribute<bool> InbIsReadOnly
	)
{
	FOnVerifyTextChanged OnVerifyTextChanged;
	FOnTextCommitted OnTextCommitted;

	if (false /* Check for specific action rename options */)
	{
		
	}
	else
	{
		OnVerifyTextChanged.BindSP(this, &SOptimusEditorGraphExplorerItem::OnNameTextVerifyChanged);
		OnTextCommitted.BindSP(this, &SOptimusEditorGraphExplorerItem::OnNameTextCommitted);
	}

	if (InCreateData->bHandleMouseButtonDown)
	{
		MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;
	}

	// FIXME: Tooltips

	TSharedPtr<SInlineEditableTextBlock> EditableTextElement = SNew(SInlineEditableTextBlock)
	    .Text(this, &SOptimusEditorGraphExplorerItem::GetDisplayText)
	    .HighlightText(InCreateData->HighlightText)
	    // .ToolTip(ToolTipWidget)
	    .OnVerifyTextChanged(OnVerifyTextChanged)
	    .OnTextCommitted(OnTextCommitted)
	    .IsSelected(InCreateData->IsRowSelectedDelegate)
	    .IsReadOnly(InbIsReadOnly);

	InlineRenameWidget = EditableTextElement.ToSharedRef();

	InCreateData->OnRenameRequest->BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	return InlineRenameWidget.ToSharedRef();
}


FText SOptimusEditorGraphExplorerItem::GetDisplayText() const
{
	const UOptimusEditorGraphSchema* Schema = GetDefault<UOptimusEditorGraphSchema>();
	if (MenuDescriptionCache.IsOutOfDate(Schema))
	{
		TSharedPtr< FEdGraphSchemaAction > GraphAction = ActionPtr.Pin();

		MenuDescriptionCache.SetCachedText(ActionPtr.Pin()->GetMenuDescription(), Schema);
	}

	return MenuDescriptionCache;
}


bool SOptimusEditorGraphExplorerItem::OnNameTextVerifyChanged(
	const FText& InNewText, 
	FText& OutErrorMessage
	)
{
	TSharedPtr<FEdGraphSchemaAction> Action = ActionPtr.Pin();
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();

	// FIXME: This whole thing is broken since we're pointing at the incorrect owner.
	if (ensure(Action) && ensure(Editor))
	{
		FString NameStr = InNewText.ToString();

		FName OriginalName;
		const UObject* NamespaceObject = nullptr;
		const UClass* NamespaceClass = nullptr;

		if (Action->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Graph *GraphAction = static_cast<FOptimusSchemaAction_Graph *>(Action.Get());
			UOptimusNodeGraph* NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);
			if (ensure(NodeGraph))
			{
				OriginalName = NodeGraph->GetFName();
				NamespaceObject = Cast<UObject>(NodeGraph->GetCollectionOwner());
				NamespaceClass = UOptimusNodeGraph::StaticClass();
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(Action.Get());
			OriginalName = BindingAction->BindingName;
			NamespaceObject = Editor->GetDeformer();
			NamespaceClass = UOptimusComponentSourceBinding::StaticClass();
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Resource* ResourceAction = static_cast<FOptimusSchemaAction_Resource*>(Action.Get());
			OriginalName = ResourceAction->ResourceName;
			NamespaceObject = Editor->GetDeformer();
			NamespaceClass = UOptimusResourceDescription::StaticClass();
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
		{
			const FOptimusSchemaAction_Variable* VariableAction = static_cast<FOptimusSchemaAction_Variable*>(Action.Get());
			OriginalName = VariableAction->VariableName;
			NamespaceObject = Editor->GetDeformer();
			NamespaceClass = UOptimusVariableDescription::StaticClass();
		}

		TSharedPtr<INameValidatorInterface> NameValidator = MakeShareable(new FOptimusNameValidator(NamespaceObject, NamespaceClass, OriginalName));

		EValidatorResult ValidatorResult = NameValidator->IsValid(NameStr);
		switch (ValidatorResult)
		{
		case EValidatorResult::Ok:
		case EValidatorResult::ExistingName:
			// These are fine, don't need to surface to the user, the rename can 'proceed' even if the name is the existing one
			break;
		default:
			OutErrorMessage = INameValidatorInterface::GetErrorText(NameStr, ValidatorResult);
			break;
		}

		return OutErrorMessage.IsEmpty();
	}
	else
	{
		return false;
	}
}


void SOptimusEditorGraphExplorerItem::OnNameTextCommitted(
	const FText& InNewText, 
	ETextCommit::Type InTextCommit
	)
{
	TSharedPtr<FEdGraphSchemaAction> Action = ActionPtr.Pin();
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	
	if (ensure(Action) && ensure(Editor))
	{
		FString NameStr = InNewText.ToString();

		if (Action->GetTypeId() == FOptimusSchemaAction_Graph::StaticGetTypeId())
		{
			FOptimusSchemaAction_Graph* GraphAction = static_cast<FOptimusSchemaAction_Graph*>(Action.Get());
			UOptimusNodeGraph* NodeGraph = Editor->GetDeformerInterface<IOptimusPathResolver>()->ResolveGraphPath(GraphAction->GraphPath);

			if (ensure(NodeGraph))
			{
				NodeGraph->GetCollectionOwner()->RenameGraph(NodeGraph, NameStr);
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Binding::StaticGetTypeId())
		{
			FOptimusSchemaAction_Binding* BindingAction = static_cast<FOptimusSchemaAction_Binding*>(Action.Get());
			UOptimusComponentSourceBinding* Binding = Editor->GetDeformer()->ResolveComponentBinding(BindingAction->BindingName);
			if (ensure(Binding))
			{
				Editor->GetDeformer()->RenameComponentBinding(Binding, FName(NameStr));
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Resource::StaticGetTypeId())
		{
			FOptimusSchemaAction_Resource* ResourceAction = static_cast<FOptimusSchemaAction_Resource*>(Action.Get());
			UOptimusResourceDescription* Resource = Editor->GetDeformer()->ResolveResource(ResourceAction->ResourceName);
			if (ensure(Resource))
			{
				Editor->GetDeformer()->RenameResource(Resource, FName(NameStr));
			}
		}
		else if (Action->GetTypeId() == FOptimusSchemaAction_Variable::StaticGetTypeId())
		{
			FOptimusSchemaAction_Variable* VariableAction = static_cast<FOptimusSchemaAction_Variable*>(Action.Get());
			UOptimusVariableDescription* Variable = Editor->GetDeformer()->ResolveVariable(VariableAction->VariableName);
			if (ensure(Variable))
			{
				Editor->GetDeformer()->RenameVariable(Variable, FName(NameStr));
			}
		}
	}
}
