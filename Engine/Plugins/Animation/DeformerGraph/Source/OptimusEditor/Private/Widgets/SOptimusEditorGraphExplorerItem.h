// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphPalette.h"
#include "EdGraph/EdGraphNodeUtils.h"


class FOptimusEditor;


class SOptimusEditorGraphExplorerItem : 
	public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SOptimusEditorGraphExplorerItem)
	{}

	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs, 
		FCreateWidgetForActionData* const InCreateData, 
		TWeakPtr<FOptimusEditor> InOptimusEditor);

	TSharedRef<SWidget> CreateIconWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bInIsReadOnly);

	// SWidget overrides
	// virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget Interface

	// SGraphPaletteItem overrides
	TSharedRef<SWidget> CreateTextSlotWidget(FCreateWidgetForActionData* const InCreateData, TAttribute<bool> bIsReadOnly) override;
	FText GetDisplayText() const override;
	bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage) override;
	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit) override;
	// End of SGraphPaletteItem Interface

private:
	FNodeTextCache MenuDescriptionCache;
	TWeakPtr<FOptimusEditor> OptimusEditor;
};
