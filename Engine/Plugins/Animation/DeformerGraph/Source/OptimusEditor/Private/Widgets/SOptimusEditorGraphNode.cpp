// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusEditorGraphNode.h"

#include "OptimusEditorHelpers.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphNode.h"
#include "OptimusEditorStyle.h"

#include "IOptimusNodeAdderPinProvider.h"
#include "OptimusNode.h"
#include "OptimusNodeGraph.h"
#include "OptimusNodePin.h"

#include "GraphEditorSettings.h"
#include "IDocumentation.h"
#include "OptimusComponentSource.h"
#include "SGraphPin.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "SOptimusEditorGraphNode"

static const FName NAME_Pin_Resource_Connected("Node.Pin.Resource_Connected");
static const FName NAME_Pin_Resource_Disconnected("Node.Pin.Resource_Disconnected");
static const FName NAME_Pin_Value_Connected("Node.Pin.Value_Connected");
static const FName NAME_Pin_Value_Disconnected("Node.Pin.Value_Disconnected");
static const FName NAME_Pin_Grouping("Node.Pin.Grouping");

static const FName NAME_PinLabel_TextStyle("Node.PinLabel");
static const FName NAME_GroupLabel_TextStyle("Node.GroupLabel");

static const FName NAME_ToolTipLabel_TextStyle("Node.ToolTipLabel");
static const FName NAME_ToolTipContent_TextStyle("Node.ToolTipContent");

static const FSlateBrush* CachedImg_Pin_Resource_Connected = nullptr;
static const FSlateBrush* CachedImg_Pin_Resource_Disconnected = nullptr;
static const FSlateBrush* CachedImg_Pin_Value_Connected = nullptr;
static const FSlateBrush* CachedImg_Pin_Value_Disconnected = nullptr;
static const FSlateBrush* CachedImg_Pin_Grouping = nullptr;


class SOptimusEditorExpanderArrow : public SExpanderArrow
{
	SLATE_BEGIN_ARGS(SOptimusEditorExpanderArrow) {}

	SLATE_ARGUMENT(bool, LeftAligned)
	SLATE_ARGUMENT(bool, AlwaysVisible)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow)
	{
		bLeftAligned = InArgs._LeftAligned;
		bAlwaysVisible = InArgs._AlwaysVisible;

		SExpanderArrow::Construct(
		    SExpanderArrow::FArguments()
		        .IndentAmount(8.0f),
		    TableRow);

		// override padding
		ChildSlot.Padding(TAttribute<FMargin>(this, &SOptimusEditorExpanderArrow::GetExpanderPadding_Extended));

		// override image
		ExpanderArrow->SetContent(
		    SNew(SImage)
		        .Image(this, &SOptimusEditorExpanderArrow::GetExpanderImage_Extended)
		        .ColorAndOpacity(FSlateColor::UseForeground()));

		// Override visibility so that groups can have an expander arrow despite having no children.
		ExpanderArrow->SetVisibility(
			TAttribute<EVisibility>(this, &SOptimusEditorExpanderArrow::GetExpanderVisibility_Extended));
	}

	FMargin GetExpanderPadding_Extended() const
	{
		const int32 NestingDepth = FMath::Max(0, OwnerRowPtr.Pin()->GetIndentLevel() - GetBaseIndentLevel());
		const float Indent = GetIndentAmount();
		return bLeftAligned ? FMargin(NestingDepth * Indent, 0, 0, 0) : FMargin(0, 0, NestingDepth * Indent, 0);
	}
	
	EVisibility GetExpanderVisibility_Extended() const
	{
		if (bAlwaysVisible)
		{
			return EVisibility::Visible;
		}
		return SExpanderArrow::GetExpanderVisibility();
	}

	const FSlateBrush* GetExpanderImage_Extended() const
	{
		const bool bIsItemExpanded = OwnerRowPtr.Pin()->IsItemExpanded();

		// FIXME: Collapse to a table.
		FName ResourceName;
		if (bIsItemExpanded)
		{
			if (ExpanderArrow->IsHovered())
			{
				static FName ExpandedHoveredLeftName("Node.PinTree.Arrow_Expanded_Hovered_Left");
				static FName ExpandedHoveredRightName("Node.PinTree.Arrow_Expanded_Hovered_Right");
				ResourceName = bLeftAligned ? ExpandedHoveredLeftName : ExpandedHoveredRightName;
			}
			else
			{
				static FName ExpandedLeftName("Node.PinTree.Arrow_Expanded_Left");
				static FName ExpandedRightName("Node.PinTree.Arrow_Expanded_Right");
				ResourceName = bLeftAligned ? ExpandedLeftName : ExpandedRightName;
			}
		}
		else
		{
			if (ExpanderArrow->IsHovered())
			{
				static FName CollapsedHoveredLeftName("Node.PinTree.Arrow_Collapsed_Hovered_Left");
				static FName CollapsedHoveredRightName("Node.PinTree.Arrow_Collapsed_Hovered_Right");
				ResourceName = bLeftAligned ? CollapsedHoveredLeftName : CollapsedHoveredRightName;
			}
			else
			{
				static FName CollapsedLeftName("Node.PinTree.Arrow_Collapsed_Left");
				static FName CollapsedRightName("Node.PinTree.Arrow_Collapsed_Right");
				ResourceName = bLeftAligned ? CollapsedLeftName : CollapsedRightName;
			}
		}

		return FOptimusEditorStyle::Get().GetBrush(ResourceName);
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}

	bool bLeftAligned;
	bool bAlwaysVisible;
};


class SOptimusEditorGraphPinToolTipWidget :
	public SCompoundWidget 
{
public:
	SLATE_BEGIN_ARGS(SOptimusEditorGraphPinToolTipWidget)
		: _ModelPin(nullptr)
	{ }
		SLATE_ARGUMENT(UOptimusNodePin*, ModelPin)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ModelPinPtr = InArgs._ModelPin;
		
		TSharedRef<SGridPanel> GridPanel = SNew(SGridPanel)
			.FillColumn(1, 1.0);

		int32 SlotIndex = 0;
		auto AddEntry = [GridPanel, &SlotIndex](const FText& InLabel, const TAttribute<FText>& InValue)
		{
			GridPanel->AddSlot(0, SlotIndex)
			[
				SNew(STextBlock)
				.TextStyle(FOptimusEditorStyle::Get(), NAME_ToolTipLabel_TextStyle)
				.Text(InLabel)
			];
			GridPanel->AddSlot(1, SlotIndex)
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FOptimusEditorStyle::Get(), NAME_ToolTipContent_TextStyle)
				.Text(InValue)
			];
			SlotIndex++;
		};

		AddEntry(LOCTEXT("PinName", "Pin Name"), {this, &SOptimusEditorGraphPinToolTipWidget::GetPinName});
		AddEntry(LOCTEXT("PinType", "Data Type"), {this, &SOptimusEditorGraphPinToolTipWidget::GetPinDataType});
		AddEntry(LOCTEXT("PinDomain", "Data Domain"), {this, &SOptimusEditorGraphPinToolTipWidget::GetPinDataDomain});
		AddEntry(LOCTEXT("PinComponentSource", "Component Source"), {this, &SOptimusEditorGraphPinToolTipWidget::GetPinComponentSource});
		AddEntry(LOCTEXT("PinMutability", "Mutability"), {this, &SOptimusEditorGraphPinToolTipWidget::GetPinMutability});

		ChildSlot
		[
			GridPanel
		];
	}
private:
	FText GetPinName() const
	{
		if (TObjectPtr<UOptimusNodePin> ModelPin = ModelPinPtr.Get())
		{
			return FText::FromName(ModelPin->GetFName());
		}
		else
		{
			return FText::GetEmpty();
		}
	}

	FText GetPinDataType() const
	{
		if (TObjectPtr<UOptimusNodePin> ModelPin = ModelPinPtr.Get())
		{
			FOptimusDataTypeRef DataType = ModelPin->GetDataType();
			if (DataType.IsValid())
			{
				if (DataType->ShaderValueType.IsValid())
				{
					return FText::Format(LOCTEXT("ShaderTypeWithHLSL", "{0} (HLSL: {1})"), DataType->DisplayName, FText::FromString(DataType->ShaderValueType->ToString()));
				}
				else
				{
					return DataType->DisplayName;
				}
			}
			else
			{
				return LOCTEXT("InvalidDataType", "<Invalid>");
			}
		}
		else
		{
			return FText::GetEmpty();
		}
	}

	FText GetPinDataDomain() const
	{
		if (TObjectPtr<UOptimusNodePin> ModelPin = ModelPinPtr.Get())
		{
			FOptimusDataDomain DataDomain = ModelPin->GetDataDomain();
			switch(DataDomain.Type)
			{
			case EOptimusDataDomainType::Dimensional:
				if (DataDomain.IsSingleton())
				{
					return LOCTEXT("ValueDomain", "Value");
				}
				if (DataDomain.Multiplier > 1)
				{
					return FText::Format(LOCTEXT("DimensionalMultiplierDomain", "{0} x {1} (Dimensional)"), 
						FText::FromString(Optimus::FormatDimensionNames(DataDomain.DimensionNames)),
						FText::AsNumber(DataDomain.Multiplier));
				}
				return FText::Format(LOCTEXT("DimensionalDomain", "{0} (Dimensional)"), 
					FText::FromString(Optimus::FormatDimensionNames(DataDomain.DimensionNames)));
				
			case EOptimusDataDomainType::Expression:
				return FText::Format(LOCTEXT("ExpressionDomain", "\"{0}\" (Expression)"),
					FText::FromString(DataDomain.Expression));
			}
		}
		return FText::GetEmpty();
	}

	FText GetPinComponentSource() const
	{
		if (TObjectPtr<UOptimusNodePin> ModelPin = ModelPinPtr.Get())
		{
			TSet<UOptimusComponentSourceBinding*> ComponentSourceBindings;

			// Make sure it doesn't belong to a node that's just been deleted, since Slate updates are usually a frame
			// behind.
			if (ModelPin->GetPackage() != GetTransientPackage())
			{
				ComponentSourceBindings = ModelPin->GetComponentSourceBindings({});
			}

			if (ComponentSourceBindings.IsEmpty())
			{
				return LOCTEXT("ComponentSourceBindingsEmpty", "<None>");
			}
			else
			{
				TArray<FText> BindingNames;
				for (UOptimusComponentSourceBinding* Binding: ComponentSourceBindings)
				{
					BindingNames.Add(FText::Format(LOCTEXT("ComponentBindingInfo", "{0} ({1})"),
						FText::FromName(Binding->BindingName),
						Binding->GetComponentSource()->GetDisplayName()));
				}
				BindingNames.Sort(FText::FSortPredicate());
				
				return FText::Join(LOCTEXT("Separator", ", "), BindingNames);
			}
		}
		return FText::GetEmpty();
	}

	FText GetPinMutability() const
	{
		if (TObjectPtr<UOptimusNodePin> ModelPin = ModelPinPtr.Get())
		{
			TSet<UOptimusComponentSourceBinding*> ComponentSourceBindings;

			// Make sure it doesn't belong to a node that's just been deleted, since Slate updates are usually a frame
			// behind.
			if (ModelPin->GetPackage() != GetTransientPackage())
			{
				if (ModelPin->IsMutable({}))
				{
					return FText::FromName(TEXT("Mutable"));
				}

				return FText::FromName(TEXT("Immutable"));
			}
		}
		return FText::GetEmpty();
	}
	
	
	TWeakObjectPtr<UOptimusNodePin> ModelPinPtr;
};


class SOptimusEditorGraphPinWidget : public SCompoundWidget
{
public:	
	SLATE_BEGIN_ARGS(SOptimusEditorGraphPinWidget) {}
		/** The text displayed for the pin label*/
		SLATE_ATTRIBUTE(FText, PinLabel)
	SLATE_END_ARGS()
	
	void Construct(
		const FArguments& InArgs,
		TSharedRef<SGraphPin> InPinWidget,
		bool bShowValueWidget,
		TSharedPtr<ITableRow> InOptionalOwnerRow)
	{
		const UEdGraphPin* InGraphPin = InPinWidget->GetPinObj();
		check(InGraphPin);
		const bool bIsLeaf = InGraphPin->SubPins.Num() == 0;
		const bool bIsInput = InGraphPin->Direction == EGPD_Input;
		const bool bLeftAligned = bIsInput;
		const bool bIsGroupPin = InGraphPin->PinType.PinCategory == UOptimusEditorGraphNode::GroupTypeName; 
		const FName TextStyle = bIsGroupPin ? NAME_GroupLabel_TextStyle : NAME_PinLabel_TextStyle;
		
		const TSharedRef<SWidget> LabelWidget = SNew(STextBlock)
			.Text(InArgs._PinLabel)
			.TextStyle(FOptimusEditorStyle::Get(), TextStyle)
			.ColorAndOpacity(FLinearColor::White)
			// .ColorAndOpacity(this, &SOptimusEditorGraphNode::GetPinTextColor, WeakPin)
			;

		TSharedRef<SWidget> LabelContent = LabelWidget;
		const TSharedRef<SWidget> PinContent = InPinWidget;
		
		if (bIsLeaf && bIsInput && bShowValueWidget)
		{
			TSharedPtr<SWidget> InputValueWidget = InPinWidget->GetValueWidget();

			if (InputValueWidget.IsValid())
			{
				TSharedRef<SWidget> LabelAndInputWidget = 
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(2.0f)
					[
						LabelWidget
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f, 2.0f, 18.0f, 2.0f)
					[
						InputValueWidget.IsValid() ? InputValueWidget.ToSharedRef() : SNew(SSpacer)
					];
				LabelContent = LabelAndInputWidget;
			}
		}

		// To allow the label to be a part of the hoverable set of widgets for the pin.
		// HoverWidgetLabels.Add(LabelWidget);
		// HoverWidgetPins.Add(PinWidget.ToSharedRef());

		const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
		FMargin InputPadding = Settings->GetInputPinPadding();
		InputPadding.Top = InputPadding.Bottom = 3.0f;
		InputPadding.Right = 0.0f;

		FMargin OutputPadding = Settings->GetOutputPinPadding();
		OutputPadding.Top = OutputPadding.Bottom = 3.0f;
		OutputPadding.Left = 2.0f;

		SHorizontalBox::FSlot* InnerContentSlotNativePtr = nullptr;

		TSharedPtr<SWidget> ExpanderWidget;
		if (InOptionalOwnerRow.IsValid())
		{
			ExpanderWidget =
				SNew(SOptimusEditorExpanderArrow, InOptionalOwnerRow)
					.LeftAligned(bLeftAligned)
					.AlwaysVisible(false);
		}
		else
		{
			// For pins that are not part of a tree view,
			// add a spacer that is the same size as the expander arrow button
			ExpanderWidget =
				SNew(SSpacer)
					.Size(FVector2d(10.0f, 10.0f));
		}

		TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

		if(bLeftAligned)
		{
			ContentBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(InputPadding)
			[
				SNew(SBox)
				[
					PinContent
				]
			];

			ContentBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				ExpanderWidget.ToSharedRef()
			];

			ContentBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.Expose(InnerContentSlotNativePtr)
			[
				SNew(SBox)
				[
					LabelContent
				]
			];
		}
		else
		{
			ContentBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.0f)
			.Expose(InnerContentSlotNativePtr)
			[
				SNew(SBox)
				[
					LabelContent
				]
			];

			ContentBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				ExpanderWidget.ToSharedRef()
			];

			ContentBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(OutputPadding)
			[
				SNew(SBox)
				[
					PinContent
				]
			];
		}

		this->ChildSlot
		[
			ContentBox
		];

	}
};

class SOptimusEditorGraphPinTreeRow : public STableRow<UOptimusNodePin*>
{
	SLATE_BEGIN_ARGS(SOptimusEditorGraphPinTreeRow) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<UOptimusNodePin*>::FArguments TableRowArgs;
		TableRowArgs
		.Content()
		[
			InArgs._Content.Widget
		];
		STableRow<UOptimusNodePin*>::Construct(TableRowArgs, InOwnerTableView);
	}

	const FSlateBrush* GetBorder() const override
	{
		// We want a transparent background.
		return FCoreStyle::Get().GetBrush("NoBrush");
	}


	void ConstructChildren( ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent ) override
	{
		// ConstructChildren is called from STableRow<UOptimusNodePin*>::Construct(...)
		this->ChildSlot
		[
			InContent
		];
	}
};


static void SetTreeExpansion_Recursive(
	TSharedPtr<STreeView<UOptimusNodePin*>>& InTreeWidget, 
	TArrayView<UOptimusNodePin* const> InItems
	)
{
	for (UOptimusNodePin *Pin: InItems)
	{
		if (Pin->GetIsExpanded())
		{
			InTreeWidget->SetItemExpansion(Pin, true);

			SetTreeExpansion_Recursive(InTreeWidget, Pin->GetSubPins());
		}
	}
}

void SOptimusEditorGraphNode::Construct(const FArguments& InArgs)
{
	if (!CachedImg_Pin_Resource_Connected)
	{
		CachedImg_Pin_Resource_Connected = FOptimusEditorStyle::Get().GetBrush(NAME_Pin_Resource_Connected);
		CachedImg_Pin_Resource_Disconnected = FOptimusEditorStyle::Get().GetBrush(NAME_Pin_Resource_Disconnected);
		CachedImg_Pin_Value_Connected = FOptimusEditorStyle::Get().GetBrush(NAME_Pin_Value_Connected);
		CachedImg_Pin_Value_Disconnected = FOptimusEditorStyle::Get().GetBrush(NAME_Pin_Value_Disconnected);
		CachedImg_Pin_Grouping = FOptimusEditorStyle::Get().GetBrush(NAME_Pin_Grouping);
	}

	GraphNode = InArgs._GraphNode;

	UOptimusEditorGraphNode *EditorGraphNode = InArgs._GraphNode;

	SetCursor( EMouseCursor::CardinalCross );
	UpdateGraphNode();

	TreeScrollBar = SNew(SScrollBar);

	LeftNodeBox->AddSlot()
	    .AutoHeight()
		[
			SAssignNew(InputTree, STreeView<UOptimusNodePin*>)
	        .Visibility(this, &SOptimusEditorGraphNode::GetInputTreeVisibility)
	        .TreeViewStyle(&FOptimusEditorStyle::Get().GetWidgetStyle<FTableViewStyle>("Node.PinTreeView"))
	        .TreeItemsSource(&EditorGraphNode->GetTopLevelInputPins())
	        .SelectionMode(ESelectionMode::None)
	        .OnGenerateRow(this, &SOptimusEditorGraphNode::MakeTableRowWidget)
	        .OnGetChildren(this, &SOptimusEditorGraphNode::HandleGetChildrenForTree)
	        .OnExpansionChanged(this, &SOptimusEditorGraphNode::HandleExpansionChanged)
	        .ExternalScrollbar(TreeScrollBar)
		];

	// Padding for group pin
	if (EditorGraphNode->GetTopLevelInputPins().IsValidIndex(0) &&
		EditorGraphNode->GetTopLevelInputPins()[0]->IsGroupingPin())
	{
		RightNodeBox->AddSlot()
			.MaxHeight(26.0f)
			[
				SNew(SSpacer)
				.Size(FVector2d(1.0f,26.0f))
			];
	}
	
	RightNodeBox->AddSlot()
	    .AutoHeight()
		[
			SAssignNew(OutputTree, STreeView<UOptimusNodePin*>)
			.Visibility(this, &SOptimusEditorGraphNode::GetOutputTreeVisibility)
	        .TreeViewStyle(&FOptimusEditorStyle::Get().GetWidgetStyle<FTableViewStyle>("Node.PinTreeView"))
			.TreeItemsSource(&EditorGraphNode->GetTopLevelOutputPins())
			.SelectionMode(ESelectionMode::None)
			.OnGenerateRow(this, &SOptimusEditorGraphNode::MakeTableRowWidget)
			.OnGetChildren(this, &SOptimusEditorGraphNode::HandleGetChildrenForTree)
			.OnExpansionChanged(this, &SOptimusEditorGraphNode::HandleExpansionChanged)
			.ExternalScrollbar(TreeScrollBar)
		];

	// Add an extra pin for AdderPinProviders to show an adder pin on both input and output sides
	if (IOptimusNodeAdderPinProvider* AdderPinProvider = Cast<IOptimusNodeAdderPinProvider>(GetModelNode()))
	{
		const TArray<EEdGraphPinDirection> Directions = {EGPD_Input, EGPD_Output};
		
		for (const EEdGraphPinDirection& Direction : Directions)
		{
			UEdGraphPin* GraphPin = GraphNode->FindPin(OptimusEditor::GetAdderPinName(Direction), Direction);
			check(GraphPin);

			TSharedPtr<SGraphPin> PinWidget = GetPinWidget(GraphPin);
			TWeakPtr<SGraphPin> WeakPin = PinWidget;
			check(PinWidget.IsValid());

			TSharedPtr<SVerticalBox> NodeBox = Direction == EGPD_Input ? LeftNodeBox : RightNodeBox;
			EHorizontalAlignment Alignment = Direction == EGPD_Input ? HAlign_Left : HAlign_Right;
			
			NodeBox->AddSlot()
				.MaxHeight(26.0f)
				.HAlign(Alignment)
				[
					SNew(SHorizontalBox)
					.ToolTipText(LOCTEXT("OptimusNodeAdderPin_ToolTip", "Connect to add a new pin"))
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SOptimusEditorGraphPinWidget, PinWidget.ToSharedRef(), false, nullptr)
						.PinLabel(this, &SOptimusEditorGraphNode::GetPinLabel, WeakPin)
					]
				];
		}
	}

	UpdatePinExpansionFromGraphPins();

	EditorGraphNode->OnNodeTitleDirtied().BindLambda([this]()
	{
		if (NodeTitle.IsValid())
		{
			NodeTitle->MarkDirty();
		}
	});

	EditorGraphNode->OnNodePinsChanged().BindSP(this, &SOptimusEditorGraphNode::SyncPinWidgetsWithGraphPins);
	EditorGraphNode->OnNodePinExpansionChanged().BindSP(this, &SOptimusEditorGraphNode::UpdatePinExpansionFromGraphPins);
}

EVisibility SOptimusEditorGraphNode::GetTitleVisibility() const
{
	// return UseLowDetailNodeTitles() ? EVisibility::Hidden : EVisibility::Visible;
	return EVisibility::Visible;
}

TSharedRef<SWidget> SOptimusEditorGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle)
{
	NodeTitle = InNodeTitle;

	TSharedRef<SWidget> WidgetRef = SGraphNode::CreateTitleWidget(NodeTitle);
	WidgetRef->SetVisibility(MakeAttributeSP(this, &SOptimusEditorGraphNode::GetTitleVisibility));
	if (NodeTitle.IsValid())
	{
		NodeTitle->SetVisibility(MakeAttributeSP(this, &SOptimusEditorGraphNode::GetTitleVisibility));
	}

	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.0f)
		[
			WidgetRef
		];
}


void SOptimusEditorGraphNode::EndUserInteraction() const
{
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(GraphNode->GetGraph());
	if (ensure(Graph))
	{
		Graph->HandleGraphNodeMoved();
	}
}


void SOptimusEditorGraphNode::CreateStandardPinWidget(UEdGraphPin* CurPin)
{
	const bool bShowPin = ShouldPinBeHidden(CurPin);

	if (bShowPin)
	{
		// Do we have this pin in our list of pins to keep?
		TSharedRef<SGraphPin>* RecycledPin = PinsToKeep.Find(CurPin);
		TSharedPtr<SGraphPin> NewPin;
		if (!RecycledPin)
		{
			NewPin = CreatePinWidget(CurPin);
			check(NewPin.IsValid());
			
			AddPin(NewPin.ToSharedRef());
		}
		else
		{
			UpdatePinIcon(*RecycledPin);
			
			NewPin = *RecycledPin;
		}

		PinWidgetMap.Add(CurPin, NewPin);
		if (CurPin->Direction == EGPD_Input)
		{
			InputPins.Add(NewPin.ToSharedRef());
		}
		else
		{
			OutputPins.Add(NewPin.ToSharedRef());
		}
	}
}


void SOptimusEditorGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	UpdatePinIcon(PinToAdd);
	
	PinToAdd->SetShowLabel(false);

	// Remove value widget from combined pin content
	TSharedPtr<SWrapBox> LabelAndValueWidget = PinToAdd->GetLabelAndValue();
	TSharedPtr<SHorizontalBox> FullPinHorizontalRowWidget = PinToAdd->GetFullPinHorizontalRowWidget().Pin();
	if (LabelAndValueWidget.IsValid() && FullPinHorizontalRowWidget.IsValid())
	{
		FullPinHorizontalRowWidget->RemoveSlot(LabelAndValueWidget.ToSharedRef());
	}

	PinToAdd->SetOwner(SharedThis(this));
}


void SOptimusEditorGraphNode::UpdatePinIcon(
	const TSharedRef<SGraphPin>& InPinToUpdate
	) const
{
	const UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	if (!ensure(EditorGraphNode))
	{
		return;
	}
	const UEdGraphPin* EdPinObj = InPinToUpdate->GetPinObj();
	if (const UOptimusNodePin *ModelPin = EditorGraphNode->FindModelPinFromGraphPin(EdPinObj))
	{
		if (ModelPin->IsGroupingPin())
		{
			InPinToUpdate->SetCustomPinIcon(CachedImg_Pin_Value_Connected, CachedImg_Pin_Value_Disconnected);
		}
		else if (ModelPin->GetDataDomain().IsSingleton())
		{
			InPinToUpdate->SetCustomPinIcon(CachedImg_Pin_Value_Connected, CachedImg_Pin_Value_Disconnected);
		}
		else
		{
			InPinToUpdate->SetCustomPinIcon(CachedImg_Pin_Resource_Connected, CachedImg_Pin_Resource_Disconnected);
		}
	}
	else if (OptimusEditor::IsAdderPin(EdPinObj))
	{
		// TODO: Use a adder pin specific icon
		InPinToUpdate->SetCustomPinIcon(CachedImg_Pin_Value_Connected, CachedImg_Pin_Value_Disconnected);
	}
}

void SOptimusEditorGraphNode::UpdatePinExpansionFromGraphPins()
{
	if (const UOptimusEditorGraphNode *EditorGraphNode = Cast<UOptimusEditorGraphNode>(GraphNode))
	{
		SetTreeExpansion_Recursive(InputTree, EditorGraphNode->GetTopLevelInputPins());
		SetTreeExpansion_Recursive(OutputTree, EditorGraphNode->GetTopLevelOutputPins());
	}
}


TSharedPtr<SGraphPin> SOptimusEditorGraphNode::GetHoveredPin(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	TSharedPtr<SGraphPin> HoveredPin = SGraphNode::GetHoveredPin(MyGeometry, MouseEvent);
#if 0
	if (!HoveredPin.IsValid())
	{
		TArray<TSharedRef<SWidget>> ExtraWidgetArray;
		ExtraWidgetToPinMap.GenerateKeyArray(ExtraWidgetArray);
		TSet<TSharedRef<SWidget>> ExtraWidgets(ExtraWidgetArray);

		TMap<TSharedRef<SWidget>, FArrangedWidget> Result;
		FindChildGeometries(MyGeometry, ExtraWidgets, Result);

		if (Result.Num() > 0)
		{
			FArrangedChildren ArrangedWidgets(EVisibility::Visible);
			Result.GenerateValueArray(ArrangedWidgets.GetInternalArray());
			int32 HoveredWidgetIndex = SWidget::FindChildUnderMouse(ArrangedWidgets, MouseEvent);
			if (HoveredWidgetIndex != INDEX_NONE)
			{
				return *ExtraWidgetToPinMap.Find(ArrangedWidgets[HoveredWidgetIndex].Widget);
			}
		}
	}
#endif
	return HoveredPin;
}


void SOptimusEditorGraphNode::RefreshErrorInfo()
{
	if (GraphNode)
	{
		if (CachedErrorType != GraphNode->ErrorType)
		{
			SGraphNode::RefreshErrorInfo();
			CachedErrorType = GraphNode->ErrorType;
		}
	}
}


void SOptimusEditorGraphNode::Tick(
	const FGeometry& AllottedGeometry,
	const double InCurrentTime,
	const float InDeltaTime
	)
{
	SGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (GraphNode)
	{
		// GraphNode->NodeWidth = (int32)AllottedGeometry.Size.X;
		// GraphNode->NodeHeight = (int32)AllottedGeometry.Size.Y;
		RefreshErrorInfo();

		// These will be deleted on the next tick.
		for (UEdGraphPin *PinToDelete: PinsToDelete)
		{
			PinToDelete->MarkAsGarbage();
		}
		PinsToDelete.Reset();
	}
}

void SOptimusEditorGraphNode::UpdateGraphNode()
{
	if (GraphNode)
	{
		// Assign a tool tip before SGraphNode::UpdateGraphNode because
		// by default SGraphNode adds the raw name of the graph node to
		// the tool tip if the EdGraph is transient, which we don't want
		FText ToolTipText = GraphNode->GetTooltipText();
		TSharedRef<SToolTip> ToolTip = IDocumentation::Get()->CreateToolTip( ToolTipText, NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
		SetToolTip(ToolTip);
	}
	SGraphNode::UpdateGraphNode();
}


UOptimusEditorGraphNode* SOptimusEditorGraphNode::GetEditorGraphNode() const
{
	return Cast<UOptimusEditorGraphNode>(GraphNode);
}


UOptimusNode* SOptimusEditorGraphNode::GetModelNode() const
{
	UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	return EditorGraphNode ? EditorGraphNode->ModelNode : nullptr;
}

TSharedPtr<SGraphPin> SOptimusEditorGraphNode::GetPinWidget(UEdGraphPin* InGraphPin)
{
	TWeakPtr<SGraphPin>* PinWidgetPtr = PinWidgetMap.Find(InGraphPin);
	if (PinWidgetPtr)
	{
		TSharedPtr<SGraphPin> PinWidget = PinWidgetPtr->Pin();
		return PinWidget;
	}

	return nullptr;
}



void SOptimusEditorGraphNode::SyncPinWidgetsWithGraphPins()
{
	// Collect graph pins to delete. We do this here because this widget is the only entity
	// that's aware of the lifetime requirements for the graph pins (SGraphPanel uses Slate 
	// timers to trigger a delete, which makes deleting them from a non-widget setting). 
	TSet<UEdGraphPin *> LocalPinsToDelete;
	for (const TSharedRef<SGraphPin>& GraphPin: InputPins)
	{
		LocalPinsToDelete.Add(GraphPin->GetPinObj());
	}
	for (const TSharedRef<SGraphPin>& GraphPin: OutputPins)
	{
		LocalPinsToDelete.Add(GraphPin->GetPinObj());
	}

	check(PinsToKeep.IsEmpty());

	UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	for (const UEdGraphPin* LivePin: EditorGraphNode->Pins)
	{
		TWeakPtr<SGraphPin>* PinWidgetPtr = PinWidgetMap.Find(LivePin);
		if (PinWidgetPtr)
		{
			TSharedPtr<SGraphPin> PinWidget = PinWidgetPtr->Pin();
			if (PinWidget.IsValid())
			{
				PinsToKeep.Add(LivePin, PinWidget.ToSharedRef());
			}
		}
		LocalPinsToDelete.Remove(LivePin);
	}

	for (const UEdGraphPin* DeletingPin: LocalPinsToDelete)
	{
		TWeakPtr<SGraphPin>* PinWidgetPtr = PinWidgetMap.Find(DeletingPin);
		if (PinWidgetPtr)
		{
			TSharedPtr<SGraphPin> PinWidget = PinWidgetPtr->Pin();
			if (PinWidget.IsValid())
			{
				// Ensure that this pin widget can no longer depend on the soon-to-be-deleted
				// graph pin.
				PinWidget->InvalidateGraphData();
			}
		}
	}
	PinsToDelete.Append(LocalPinsToDelete);

	// Reconstruct the pin widgets. This could be done more surgically but will do for now.
	InputPins.Reset();
	OutputPins.Reset();
	PinWidgetMap.Reset();

	CreatePinWidgets();

	// Nix any pins left in this map. They're most likely hidden sub-pins.
	PinsToKeep.Reset();

	InputTree->RebuildList();
	OutputTree->RebuildList();
}


EVisibility SOptimusEditorGraphNode::GetInputTreeVisibility() const
{
	const UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	
	return EditorGraphNode && !EditorGraphNode->GetTopLevelInputPins().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SOptimusEditorGraphNode::GetOutputTreeVisibility() const
{
	const UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();

	return EditorGraphNode && !EditorGraphNode->GetTopLevelOutputPins().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}


TSharedRef<ITableRow> SOptimusEditorGraphNode::MakeTableRowWidget(
	UOptimusNodePin* InModelPin, 
	const TSharedRef<STableViewBase>& OwnerTable
	)
{
	const bool bShowValueWidget =
		InModelPin->GetDataDomain().IsSingleton() &&
		InModelPin->GetPropertyFromPin() != nullptr &&
		!InModelPin->GetDataType()->IsArrayType();
	
	const UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	TSharedPtr<SGraphPin> PinWidget;
	TWeakPtr<SGraphPin> WeakPin;
	if (ensure(EditorGraphNode))
	{
		UEdGraphPin* GraphPin = EditorGraphNode->FindGraphPinFromModelPin(InModelPin);
		PinWidget = GetPinWidget(GraphPin);
		check(PinWidget);
		WeakPin = PinWidget;
	}

	TSharedPtr<SOptimusEditorGraphPinTreeRow> RowWidget;

	SAssignNew(RowWidget, SOptimusEditorGraphPinTreeRow, OwnerTable);
	if (!InModelPin->IsGroupingPin())
	{
		RowWidget->SetToolTip(MakePinToolTip(InModelPin));
	}
	else
	{
		RowWidget->SetToolTipText(
			LOCTEXT("GroupPinToolTip",
				"Group Pin accepts direct connection from Component Source Node"));
	}

	RowWidget->SetContent(
		SNew(SOptimusEditorGraphPinWidget, PinWidget.ToSharedRef(), bShowValueWidget, RowWidget)
		.PinLabel(this, &SOptimusEditorGraphNode::GetPinLabel, WeakPin)
	);
	
	return RowWidget.ToSharedRef();
}


TSharedPtr<IToolTip> SOptimusEditorGraphNode::MakePinToolTip(
	UOptimusNodePin* InModelPin
	) const
{
	return SNew(SToolTip)
	.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.BrightBackground"))
	.TextMargin(11.0f)
	[
		SNew(SOptimusEditorGraphPinToolTipWidget)
		.ModelPin(InModelPin)
	];
}


void SOptimusEditorGraphNode::HandleGetChildrenForTree(
	UOptimusNodePin* InItem, 
	TArray<UOptimusNodePin*>& OutChildren
	)
{
	OutChildren.Append(InItem->GetSubPins());
}


void SOptimusEditorGraphNode::HandleExpansionChanged(
	UOptimusNodePin* InItem, 
	bool bExpanded
	)
{
	InItem->SetIsExpanded(bExpanded);
}


FText SOptimusEditorGraphNode::GetPinLabel(TWeakPtr<SGraphPin> InWeakGraphPin) const
{
	UOptimusEditorGraphNode* EditorGraphNode = GetEditorGraphNode();
	TSharedPtr<SGraphPin> GraphPin = InWeakGraphPin.Pin();

	// Absence of ModelNode indicates the node has been deleted
	if (GraphPin.IsValid() && EditorGraphNode && EditorGraphNode->ModelNode)
	{
		return EditorGraphNode->GetPinDisplayName(GraphPin->GetPinObj());
	}
	return FText::GetEmpty();
}


#undef LOCTEXT_NAMESPACE
