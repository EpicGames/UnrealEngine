// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class SScrollBar;
class UOptimusEditorGraphNode;
class UOptimusNode;
class UOptimusNodePin;
enum class EOptimusNodePinDirection : uint8;
template<typename ItemType> class STreeView;

class SOptimusEditorGraphNode : 
	public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SOptimusEditorGraphNode)
		: _GraphNode(nullptr)
	{}
		SLATE_ARGUMENT(UOptimusEditorGraphNode*, GraphNode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	// SGraphNode overrides
	void EndUserInteraction() const override;
	void CreateStandardPinWidget(UEdGraphPin* CurPin) override;
	void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	TSharedPtr<SGraphPin> GetHoveredPin(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const override;
	void RefreshErrorInfo() override;
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	void UpdateGraphNode() override;

private:
	UOptimusEditorGraphNode *GetEditorGraphNode() const;
	UOptimusNode* GetModelNode() const;
	TSharedPtr<SGraphPin> GetPinWidget(UEdGraphPin* InGraphPin);
	void UpdatePinIcon(const TSharedRef<SGraphPin>& PinToAdd) const;

	void UpdatePinExpansionFromGraphPins();
	void SyncPinWidgetsWithGraphPins();
	
	EVisibility GetTitleVisibility() const;

	// SGraphNode protected overrides
	TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;
	
	// STreeView helper functions
	EVisibility GetInputTreeVisibility() const;
	EVisibility GetOutputTreeVisibility() const;
	TSharedRef<ITableRow> MakeTableRowWidget(UOptimusNodePin* InItem, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<IToolTip> MakePinToolTip(UOptimusNodePin* InItem) const;
	void HandleGetChildrenForTree(UOptimusNodePin* InItem, TArray<UOptimusNodePin*>& OutChildren);
	void HandleExpansionChanged(UOptimusNodePin* InItem, bool bExpanded);

	FText GetPinLabel(TWeakPtr<SGraphPin> InWeakGraphPin) const;

	TSharedPtr<SNodeTitle> NodeTitle;
	
	// Collapsible input pins
	TSharedPtr<STreeView<UOptimusNodePin*>> InputTree;

	// Collapsible input pins
	TSharedPtr<STreeView<UOptimusNodePin*>> OutputTree;

	TSharedPtr<SScrollBar> TreeScrollBar;

	TMap<const UEdGraphPin*, TWeakPtr<SGraphPin>> PinWidgetMap;

	// A paired list of widgets to map from labels to pin to support labels participating in
	// pin hovering.
	TArray<TSharedRef<SWidget>> HoverWidgetLabels;
	TArray<TSharedRef<SGraphPin>> HoverWidgetPins;

	// Pins to keep after calling SyncPinWidgetsWithGraphPins. We recycle these pins in
	// CreateStandardPinWidget.
	TMap<const UEdGraphPin *, TSharedRef<SGraphPin>> PinsToKeep;

	// Delayed pin deletion. To deal with the fact that pin deletion cannot occur until we
	// have re-generated the pin list. SOptimusEditorGraphNode has already relinquished them
	// but we still have a pointer to them in our pin widget.
	TSet<UEdGraphPin *> PinsToDelete;

	// Cached error type to compare against the UEdGraphNode to see if we need to refresh
	// our error state.
	int32 CachedErrorType = -1;
};
