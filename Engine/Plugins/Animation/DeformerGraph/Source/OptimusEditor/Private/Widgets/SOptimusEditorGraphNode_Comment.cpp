// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusEditorGraphNode_Comment.h"
#include "EdGraphNode_Comment.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorHelpers.h"
#include "OptimusEditorGraphNode_Comment.h"
#include "OptimusNode_Comment.h"

#define LOCTEXT_NAMESPACE "SOptimusEditorGraphNode_Comment"

void SOptimusEditorGraphNode_Comment::Construct(const FArguments& InArgs, UOptimusEditorGraphNode_Comment* InGraphNode)
{
	InGraphNode->GetOnSizeChanged().BindSP(this, &SOptimusEditorGraphNode_Comment::OnSizeChanged);
	InGraphNode->GetOnPositionChanged().BindSP(this, &SOptimusEditorGraphNode_Comment::OnPositionChanged);
	
	SGraphNodeComment::Construct({}, InGraphNode);
}

FReply SOptimusEditorGraphNode_Comment::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SGraphNodeComment::OnMouseButtonUp(MyGeometry, MouseEvent);
	
	UOptimusNode_Comment* CommentNode = Cast<UOptimusNode_Comment>(OptimusEditor::FindModelNodeFromGraphNode(GraphNode));
	if (ensure(CommentNode))
	{
		CommentNode->SetSize(UserSize);
	}
	
	return Reply;
}

void SOptimusEditorGraphNode_Comment::EndUserInteraction() const
{
	
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(GraphNode->GetGraph());
	if (ensure(Graph))
	{
		Graph->HandleGraphNodeMoved();
	}
	
	SGraphNodeComment::EndUserInteraction();
}

void SOptimusEditorGraphNode_Comment::OnSizeChanged()
{
	UserSize = FVector2D(GraphNode->NodeWidth, GraphNode->NodeHeight);
}

void SOptimusEditorGraphNode_Comment::OnPositionChanged()
{
	// Goal here is to use HandleSelection to update the nodes under the comment node after a undo
	
	// SGraphNodeComment::bIsSelected is private, so here goes the workaround to avoid changing the selection state
	bool bIsGraphNodeSelected = false;
	UOptimusEditorGraph* Graph = Cast<UOptimusEditorGraph>(GraphNode->GetGraph());
	if (ensure(Graph))
	{
		 bIsGraphNodeSelected = Graph->GetSelectedNodes().Contains(GraphNode);
	}

	constexpr bool bUpdateNodesUnderComment = true;
	HandleSelection(bIsGraphNodeSelected, bUpdateNodesUnderComment);
}

#undef LOCTEXT_NAMESPACE
