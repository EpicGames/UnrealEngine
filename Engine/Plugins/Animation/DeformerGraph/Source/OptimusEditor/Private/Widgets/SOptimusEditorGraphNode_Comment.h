// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNodeComment.h"

class UOptimusEditorGraphNode_Comment;
class UEdGraphNode_Comment;



class SOptimusEditorGraphNode_Comment : 
	public SGraphNodeComment
{
public:
	SLATE_BEGIN_ARGS(SOptimusEditorGraphNode_Comment) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UOptimusEditorGraphNode_Comment* InGraphNode);

	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void EndUserInteraction() const override;
private:
	void OnSizeChanged();
	void OnPositionChanged();

};
