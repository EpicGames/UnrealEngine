// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusGraphTitleBar.h"

#include "OptimusEditor.h"
#include "OptimusEditorGraph.h"

#include "Styling/AppStyle.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "OptimusNodeGraph.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SOptimusGraphTitleBar"


 SOptimusGraphTitleBar::~SOptimusGraphTitleBar()
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().RemoveAll(this);
	}	
}


void SOptimusGraphTitleBar::Construct(const FArguments& InArgs)
{
	OptimusEditor = InArgs._OptimusEditor;
	OnGraphCrumbClickedEvent = InArgs._OnGraphCrumbClickedEvent;

	// Set-up shared breadcrumb defaults (from SGraphTitleBar::Construct)
	FMargin BreadcrumbTrailPadding = FMargin(4.f, 2.f);
	const FSlateBrush* BreadcrumbButtonImage = FAppStyle::GetBrush("BreadcrumbTrail.Delimiter");

	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().AddRaw(this, &SOptimusGraphTitleBar::Refresh);
	}	

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
		.HAlign(HAlign_Fill)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EventGraphTitleBar")))
		[
			SNew(SHorizontalBox)
			/*
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SGraphTitleBarAddNewBookmark)
				.EditorPtr(Kismet2Ptr)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				InArgs._HistoryNavigationWidget.ToSharedRef()
			]
			*/
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
			// Title text/icon
			+ SHorizontalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10.0f, 5.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(this, &SOptimusGraphTitleBar::GetGraphTypeIcon)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SAssignNew(BreadcrumbTrailScrollBox, SScrollBox)
						.Orientation(Orient_Horizontal)
						.ScrollBarVisibility(EVisibility::Collapsed)

						+ SScrollBox::Slot()
						.Padding(0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SHorizontalBox)
							// show fake 'root' breadcrumb for the title
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(BreadcrumbTrailPadding)
							[
								SNew(STextBlock)
								.Text(this, &SOptimusGraphTitleBar::GetDeformerTitle)
								.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
								.Visibility( this, &SOptimusGraphTitleBar::IsDeformerTitleVisible )
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(BreadcrumbButtonImage)
								.Visibility( this, &SOptimusGraphTitleBar::IsDeformerTitleVisible )
							]

							// New style breadcrumb
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<UOptimusNodeGraph*>)
								.ButtonStyle(FAppStyle::Get(), "GraphBreadcrumbButton")
								.TextStyle(FAppStyle::Get(), "GraphBreadcrumbButtonText")
								.ButtonContentPadding(BreadcrumbTrailPadding)
								.DelimiterImage(BreadcrumbButtonImage)
								.PersistentBreadcrumbs(true)
								.OnCrumbClicked(this, &SOptimusGraphTitleBar::OnBreadcrumbClicked)
							]
#if 0
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
								.ColorAndOpacity(FLinearColor(1, 1, 1, 0.5))
								.Text(this, &SGraphTitleBar::GetTitleExtra)
							]
#endif
						]
					]
				]
			]
		]
	];

	Refresh();
	BreadcrumbTrailScrollBox->ScrollToEnd();
}


void SOptimusGraphTitleBar::Refresh()
{
 	// This doesn't do much until we have nested graphs.
 	BreadcrumbTrail->ClearCrumbs(false);

    if (const TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin())
 	{
	    if (const UOptimusEditorGraph *EditorGraph = Editor->GetGraph(); EditorGraph->GetModelGraph())
 		{
 			BuildBreadcrumbTrail(EditorGraph->GetModelGraph());
 		}
 	}
}


void SOptimusGraphTitleBar::BuildBreadcrumbTrail(UOptimusNodeGraph* InGraph)
{
 	// Traverse upwards first so that parents get pushed onto the breadcrumb trail first.
 	if (InGraph->GetParentGraph())
 	{
 		BuildBreadcrumbTrail(InGraph->GetParentGraph());
 	}
 		
	const auto CrumbName = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&SOptimusGraphTitleBar::GetGraphTitle, (const UOptimusNodeGraph*)InGraph));
	BreadcrumbTrail->PushCrumb(CrumbName, InGraph);
}

const FSlateBrush* SOptimusGraphTitleBar::GetGraphTypeIcon() const
{
	// FIXME: Get this from the editor. See FBlueprintEditor::GetGlyphForGraph for list of 
	// possible icons.
	return FAppStyle::GetBrush(TEXT("GraphEditor.Function_24x"));
}

FText SOptimusGraphTitleBar::GetGraphTitle(const UOptimusNodeGraph* InGraph)
{
 	return FText::FromName(InGraph->GetFName());
}


void SOptimusGraphTitleBar::OnBreadcrumbClicked(UOptimusNodeGraph* const& InModelGraph) const
{
	OnGraphCrumbClickedEvent.ExecuteIfBound(InModelGraph);
}


FText SOptimusGraphTitleBar::GetDeformerTitle() const
{
	TSharedPtr<FOptimusEditor> Editor = OptimusEditor.Pin();
	if (Editor)
	{
		return Editor->GetGraphCollectionRootName();
	}
	else
	{
		return FText::GetEmpty();
	}
}


EVisibility SOptimusGraphTitleBar::IsDeformerTitleVisible() const
{
	return OptimusEditor.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}


#undef LOCTEXT_NAMESPACE
