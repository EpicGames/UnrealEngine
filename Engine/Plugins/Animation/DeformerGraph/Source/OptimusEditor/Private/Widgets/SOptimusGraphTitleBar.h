// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "OptimusEditorDelegates.h"

class FOptimusEditor;
class SScrollBox;
class UOptimusNodeGraph;
template<typename> class SBreadcrumbTrail;

class SOptimusGraphTitleBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOptimusGraphTitleBar)
	{}
		SLATE_ARGUMENT(TWeakPtr<FOptimusEditor>, OptimusEditor)
		SLATE_EVENT(FOptimusOpenGraphEvent, OnGraphCrumbClickedEvent)
	SLATE_END_ARGS()

	~SOptimusGraphTitleBar() override;

	void Construct(const FArguments& InArgs);

	// Forcibly refresh the title bar
	void Refresh();

private:
	void BuildBreadcrumbTrail(UOptimusNodeGraph* InGraph);

	const FSlateBrush* GetGraphTypeIcon() const;

	static FText GetGraphTitle(const UOptimusNodeGraph* InGraph);

	void OnBreadcrumbClicked(UOptimusNodeGraph* const& InModelGraph) const;

	FText GetDeformerTitle() const;
	EVisibility IsDeformerTitleVisible() const;

	// The owning graph editor widget.
	TWeakPtr<FOptimusEditor> OptimusEditor;

	// The scroll box that kicks in if the trail exceeds the widget's visible box.
	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;

	// Breadcrumb trail widget
	TSharedPtr< SBreadcrumbTrail<UOptimusNodeGraph*> > BreadcrumbTrail;

	// Callback for switching graph levels.
	FOptimusOpenGraphEvent OnGraphCrumbClickedEvent;
};
