// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusNodePalette.h"

#include "OptimusDeformer.h"
#include "OptimusEditor.h"
#include "OptimusEditorGraph.h"
#include "OptimusEditorGraphSchema.h"

void SOptimusNodePaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	// Store the action pointer so that the callbacks and various utility functions know what
	// they're operating on.
	ActionPtr = InCreateData->Action;

	// TSharedPtr<const FInputChord> HotkeyChord;
	// FIXME: Hotkey support.


	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	const FSlateBrush* IconBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor IconColor = FSlateColor::UseForeground();
	FText IconToolTip = GraphAction->GetTooltipDescription();
	bool bIsReadOnly = false;

	FSlateFontInfo NameFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	TSharedRef<SWidget> IconWidget = CreateIconWidget(IconToolTip, IconBrush, IconColor);
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly);
#if 0
	TSharedRef<SWidget> HotkeyDisplayWidget = CreateHotkeyDisplayWidget(NameFont, HotkeyChord);
#endif

	// Create the actual widget
	ChildSlot
		[
			SNew(SHorizontalBox)
			// Icon slot
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			IconWidget
		]
	// Name slot
	+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(3, 0)
		[
			NameSlotWidget
		]
	// Hotkey slot
#if 0
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			HotkeyDisplayWidget
		]
#endif
		];

}


TSharedRef<SWidget> SOptimusNodePaletteItem::CreateHotkeyDisplayWidget(const FSlateFontInfo& NameFont, const TSharedPtr<const FInputChord> HotkeyChord)
{
	FText HotkeyText;
	if (HotkeyChord.IsValid())
	{
		HotkeyText = HotkeyChord->GetInputText();
	}
	return SNew(STextBlock)
		.Text(HotkeyText)
		.Font(NameFont);
}


FText SOptimusNodePaletteItem::GetItemTooltip() const
{
	// FIXME: Get from action.
	return FText::FromString(TEXT("Not a tooltip"));
}


SOptimusNodePalette::~SOptimusNodePalette()
{
	const TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().RemoveAll(this);
		
		if (Editor->GetDeformer())
		{
			Editor->GetDeformer()->GetNotifyDelegate().RemoveAll(this);
		}
	}
}


void SOptimusNodePalette::Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InEditor)
{
	OwningEditor = InEditor;

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(2.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(GraphActionMenu, SGraphActionMenu)
			.OnActionDragged(this, &SOptimusNodePalette::OnActionDragged)
			.OnCreateWidgetForAction(this, &SOptimusNodePalette::OnCreateWidgetForAction)
			.OnCollectAllActions(this, &SOptimusNodePalette::CollectAllActions)
			.AutoExpandActionMenu(true)
		]
	];

	const TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin();
	if (ensure(Editor))
	{
		Editor->OnRefresh().AddRaw(this, &SOptimusNodePalette::Refresh);
		
		if (Editor->GetDeformer())
		{
			Editor->GetDeformer()->GetNotifyDelegate().AddSP(this, &SOptimusNodePalette::GraphCollectionNotify);
		}
	}
}

void SOptimusNodePalette::Refresh()
{
	GraphActionMenu->RefreshAllActions(true);
}


TSharedRef<SWidget> SOptimusNodePalette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return	SNew(SOptimusNodePaletteItem, InCreateData);
}


void SOptimusNodePalette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	if (const TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin())
	{
		const UOptimusEditorGraphSchema* Schema = Cast<UOptimusEditorGraphSchema>(Editor->GetGraph()->GetSchema());

		Schema->GetGraphActions(OutAllActions, nullptr, Editor->GetGraph());
	}
}


void SOptimusNodePalette::GraphCollectionNotify(
	EOptimusGlobalNotifyType InType,
	UObject* InObject
	)
{
	if (InType == EOptimusGlobalNotifyType::NodeTypeAdded || InType == EOptimusGlobalNotifyType::NodeTypeRemoved)
	{
		RefreshActionsList(true);
	}
}
