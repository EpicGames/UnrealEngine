// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/InputChord.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphPalette.h"

enum class EOptimusGlobalNotifyType;
class FOptimusEditor;

/** Widget for displaying a single item  */
class SOptimusNodePaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SOptimusNodePaletteItem) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

private:
	/* Create the hotkey display widget */
	TSharedRef<SWidget> CreateHotkeyDisplayWidget(const FSlateFontInfo& NameFont, const TSharedPtr<const FInputChord> HotkeyChord);

	virtual FText GetItemTooltip() const override;
};

//////////////////////////////////////////////////////////////////////////

class SOptimusNodePalette : public SGraphPalette
{
public:
	~SOptimusNodePalette() override;
	
	SLATE_BEGIN_ARGS(SOptimusNodePalette) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InEditor);
	
	// Forcibly refresh
	void Refresh();
	
protected:
	// SGraphPalette overrides
	TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	// End of SGraphPalette Interface

private:
	void GraphCollectionNotify(EOptimusGlobalNotifyType InType, UObject* InObject);
	
	// The owning editor
	TWeakPtr<FOptimusEditor> OwningEditor;

	TArray< TSharedPtr<FString> > CategoryNames;

	/** Combo box used to select category */
	TSharedPtr<STextComboBox> CategoryComboBox;
};
