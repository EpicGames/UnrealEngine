// Copyright Epic Games, Inc. All Rights Reserved.
#include "SOptimusShaderTextDocumentTab.h"

#include "IOptimusShaderTextProvider.h"
#include "OptimusEditor.h"
#include "OptimusHLSLSyntaxHighlighter.h"
#include "SOptimusShaderTextDocumentTextBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "OptimusShaderTextDocumentTab"

SOptimusShaderTextDocumentTab::SOptimusShaderTextDocumentTab()
	: SyntaxHighlighterDeclarations(FOptimusHLSLSyntaxHighlighter::Create())
	, SyntaxHighlighterShaderText(FOptimusHLSLSyntaxHighlighter::Create())
{
}

SOptimusShaderTextDocumentTab::~SOptimusShaderTextDocumentTab()
{
	if (HasValidShaderTextProvider())
	{
		if (TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin())
		{
			Editor->OnDiagnosticsUpdated().RemoveAll(this);
		}
	}
}

void SOptimusShaderTextDocumentTab::Construct(const FArguments& InArgs, UObject* InShaderTextProviderObject, TWeakPtr<FOptimusEditor> InEditor, TSharedRef<SDockTab> InDocumentHostTab)
{
	ShaderTextProviderObject = InShaderTextProviderObject;
	OwningEditor = InEditor;

	check(HasValidShaderTextProvider());

	if (TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin())
	{
		Editor->OnDiagnosticsUpdated().AddSP(this, &SOptimusShaderTextDocumentTab::OnDiagnosticsUpdated);
	}

	const FText DeclarationsTitle =	LOCTEXT("OptimusShaderTextDocumentTab_Declarations_Title", "Declarations (Read-Only)");
	const FText ShaderTextTitle = LOCTEXT("OptimusShaderTextDocumentTab_ShaderText_Title", "Shader Text");

	SExpandableArea::FArguments ExpandableAreaArgs;
	ExpandableAreaArgs.AreaTitleFont(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"));
	
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		.PhysicalSplitterHandleSize(4.0f)
		.HitDetectionSplitterHandleSize(6.0f)
		+ SSplitter::Slot()
		.Value(1.0)
		.SizeRule(this, &SOptimusShaderTextDocumentTab::GetDeclarationsSectionSizeRule)
		[
			SAssignNew(DeclarationsExpandableArea,SExpandableArea) = ExpandableAreaArgs
			.AreaTitle(DeclarationsTitle)
			.InitiallyCollapsed(true)
			.AllowAnimatedTransition(false)
			.BodyContent()
			[
				SAssignNew(DeclarationsTextBox,SOptimusShaderTextDocumentTextBox)
				.Text(this, &SOptimusShaderTextDocumentTab::GetDeclarationsAsText)
				.IsReadOnly(true)
				.Marshaller(SyntaxHighlighterDeclarations)	
			]
		]
		+ SSplitter::Slot()
		.Value(1.0)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				ConstructNonExpandableHeaderWidget(ExpandableAreaArgs.AreaTitle(ShaderTextTitle))
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(ShaderTextTextBox, SOptimusShaderTextDocumentTextBox)
				.Text(this, &SOptimusShaderTextDocumentTab::GetShaderTextAsText)
				.IsReadOnly(this, &SOptimusShaderTextDocumentTab::IsShaderTextReadOnly)
				.Marshaller(SyntaxHighlighterShaderText)
				.OnTextChanged(this, &SOptimusShaderTextDocumentTab::OnShaderTextChanged)	
			]
		]
	];

	// Init any diagnostics.
	OnDiagnosticsUpdated();
}

TSharedRef<SWidget> SOptimusShaderTextDocumentTab::ConstructNonExpandableHeaderWidget(const SExpandableArea::FArguments& InArgs) const
{
	const TAttribute<const FSlateBrush*> TitleBorderImage = FStyleDefaults::GetNoBrush();
	const TAttribute<FSlateColor> TitleBorderBackgroundColor = FLinearColor::Transparent;
	const FVector2d SpacerSize = InArgs._Style->CollapsedImage.GetImageSize();
    					
	return
		SNew(SBorder )
		.BorderImage(TitleBorderImage)
		.BorderBackgroundColor(TitleBorderBackgroundColor)
		.Padding(0.0f)
		[
			SNew( SButton )
			.ButtonStyle(FCoreStyle::Get(), "NoBorder")
			.ContentPadding(InArgs._HeaderPadding)
			.ForegroundColor(FSlateColor::UseForeground())
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(InArgs._AreaTitlePadding)
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
					.Size(SpacerSize)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._AreaTitle)
					.Font(InArgs._AreaTitleFont)
				]
			]
		];
}


SSplitter::ESizeRule SOptimusShaderTextDocumentTab::GetDeclarationsSectionSizeRule() const
{
	SSplitter::ESizeRule SizeRule = SSplitter::ESizeRule::FractionOfParent;
	
	if (ensure(DeclarationsExpandableArea.IsValid()))
	{
		SizeRule = DeclarationsExpandableArea->IsExpanded() ?
			SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
	}
	
	return SizeRule;
}

bool SOptimusShaderTextDocumentTab::HasValidShaderTextProvider() const
{
	return (GetProviderInterface() != nullptr);
}

IOptimusShaderTextProvider* SOptimusShaderTextDocumentTab::GetProviderInterface() const
{
	return Cast<IOptimusShaderTextProvider>(ShaderTextProviderObject);
}

FText SOptimusShaderTextDocumentTab::GetDeclarationsAsText() const
{
	if (HasValidShaderTextProvider())
	{
		return FText::FromString(GetProviderInterface()->GetDeclarations());
	}
	return FText::GetEmpty();
}

FText SOptimusShaderTextDocumentTab::GetShaderTextAsText() const
{
	if (HasValidShaderTextProvider())
	{
		return FText::FromString(GetProviderInterface()->GetShaderText());
	}
	return FText::GetEmpty();
}

void SOptimusShaderTextDocumentTab::OnShaderTextChanged(const FText& InText) const
{
	if (HasValidShaderTextProvider())
	{
		GetProviderInterface()->SetShaderText(InText.ToString());
	}
}

void SOptimusShaderTextDocumentTab::OnDiagnosticsUpdated() const
{
	if (TSharedPtr<FOptimusEditor> Editor = OwningEditor.Pin())
	{
		TArray<FOptimusCompilerDiagnostic> Diagnostics;
		for (FOptimusCompilerDiagnostic const& Diagnostic : Editor->GetCompilationDiagnostics())
		{
			if (Diagnostic.Object == ShaderTextProviderObject)
			{
				Diagnostics.Add(Diagnostic);
			}
		}

		SyntaxHighlighterShaderText->SetCompilerMessages(Diagnostics);
		ShaderTextTextBox->Refresh();
	}
}

bool SOptimusShaderTextDocumentTab::IsShaderTextReadOnly() const
{
	if (IOptimusShaderTextProvider* Provider = Cast<IOptimusShaderTextProvider>(ShaderTextProviderObject))
	{
		return Provider->IsShaderTextReadOnly();
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
