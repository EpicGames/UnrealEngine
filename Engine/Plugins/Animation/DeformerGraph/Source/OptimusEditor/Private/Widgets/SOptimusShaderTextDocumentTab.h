// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSplitter.h"

class IOptimusShaderTextProvider;
class FOptimusHLSLSyntaxHighlighter;

class SVerticalBox;
class FOptimusEditor;
class FSpawnTabArgs;
class FTabManager;
class SOptimusShaderTextDocumentTextBox;
class SDockTab;


class SOptimusShaderTextDocumentTab : public SCompoundWidget
{
public:
	SOptimusShaderTextDocumentTab();
	virtual ~SOptimusShaderTextDocumentTab() override;

	SLATE_BEGIN_ARGS(SOptimusShaderTextDocumentTab) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UObject* InShaderTextProviderObject, TWeakPtr<FOptimusEditor> InEditor, TSharedRef<SDockTab> InDocumentHostTab);
	
private:
	TSharedRef<SWidget> ConstructNonExpandableHeaderWidget(const SExpandableArea::FArguments& InArgs) const;
	SSplitter::ESizeRule GetDeclarationsSectionSizeRule() const;
	
	bool HasValidShaderTextProvider() const;
	IOptimusShaderTextProvider* GetProviderInterface() const;
	FText GetDeclarationsAsText() const;
	FText GetShaderTextAsText() const;
	void OnShaderTextChanged(const FText& InText) const;
	void OnDiagnosticsUpdated() const;
	bool IsShaderTextReadOnly() const;

	TSharedRef<FOptimusHLSLSyntaxHighlighter> SyntaxHighlighterDeclarations;
	TSharedRef<FOptimusHLSLSyntaxHighlighter> SyntaxHighlighterShaderText;

	TSharedPtr<FTabManager> TabManager;
	
	TWeakObjectPtr<UObject> ShaderTextProviderObject;
	TWeakPtr<FOptimusEditor> OwningEditor;

	TSharedPtr<SExpandableArea> DeclarationsExpandableArea;
	// ptr needed for text search
	TSharedPtr<SOptimusShaderTextDocumentTextBox> DeclarationsTextBox;
	TSharedPtr<SOptimusShaderTextDocumentTextBox> ShaderTextTextBox;
};

