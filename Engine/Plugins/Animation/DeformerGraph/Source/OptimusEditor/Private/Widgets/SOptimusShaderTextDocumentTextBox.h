// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"
#include "Widgets/Input/SSearchBox.h"

class SMultiLineEditableText;
class ITextLayoutMarshaller;
class SExpandableArea;
class SOptimusShaderTextSearchWidget;
class SDockTab;
class SVerticalBox;

class FOptimusShaderTextEditorDocumentTextBoxCommands :
	public TCommands<FOptimusShaderTextEditorDocumentTextBoxCommands>
{
public:
	FOptimusShaderTextEditorDocumentTextBoxCommands();
	
	// TCommands<> overrides
	virtual void RegisterCommands() override;
	
	TSharedPtr<FUICommandInfo> Search;
	TSharedPtr<FUICommandInfo> NextOccurrence;
	TSharedPtr<FUICommandInfo> PreviousOccurrence;
	
	TSharedPtr<FUICommandInfo> ToggleComment;
};

class SOptimusShaderTextDocumentTextBox : public SCompoundWidget
{
public:
	
	SOptimusShaderTextDocumentTextBox();
	virtual ~SOptimusShaderTextDocumentTextBox() override;

	SLATE_BEGIN_ARGS(SOptimusShaderTextDocumentTextBox) {};
		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Text to search for (a new search is triggered whenever this text changes) */
		SLATE_ATTRIBUTE(FText, SearchText)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr< ITextLayoutMarshaller >, Marshaller)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Called whenever the text is changed programmatically or interactively by the user */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void Refresh() const;

private:
	
	/** SWidget interface */
	// use Preview Key Down so that the commands related to this tab is given the priority
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void RegisterCommands();
	
	bool HandleEscape();
	
	void ShowSearchBar();
	bool HideSearchBar();
	
	void OnTriggerSearch();
	void OnSearchTextChanged(const FText& InTextToSearch);
	void OnSearchTextCommitted(const FText& InTextToSearch, ETextCommit::Type InCommitType);
	TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;
	void OnSearchResultNavigationButtonClicked(SSearchBox::SearchDirection InDirection);
	void OnGoToNextOccurrence();
	void OnGoToPreviousOccurrence();

	FReply OnTextKeyDown(const FGeometry& MyGeometry,
		const FKeyEvent& InCharacterEvent) const;

	void OnToggleComment() const;
	
	FReply OnTextKeyChar(const FGeometry& MyGeometry,
		const FCharacterEvent& InCharacterEvent) const;
	
	// Figure out if we need to auto-indent.
	void HandleAutoIndent() const;

	TSharedPtr<SVerticalBox> TabBody;
	
	TSharedPtr<SMultiLineEditableText> Text;

	bool bIsSearchBarHidden;
	TSharedPtr<SOptimusShaderTextSearchWidget> SearchBar;

	// Top level command list is processed during preview key down
	// such that its commands are given the priority
	TSharedRef<FUICommandList> TopLevelCommandList;
	
	TSharedRef<FUICommandList> TextCommandList;
};