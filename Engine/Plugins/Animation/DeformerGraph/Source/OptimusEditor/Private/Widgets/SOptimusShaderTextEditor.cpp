// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOptimusShaderTextEditor.h"
#include "SOptimusShaderTextDocumentTab.h"
#include "OptimusEditor.h"
#include "OptimusNode.h"

#include "IOptimusShaderTextProvider.h"

#include "Framework/Application/SlateApplication.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "OptimusShaderTextEditor"


// Tab identifiers, DocumentTracker seems to require it to be "Document"
const FName SOptimusShaderTextEditor::DocumentTabId = TEXT("Document");

struct FOptimusShaderTextEditorDocumentTabSummoner : public FDocumentTabFactory
{
public:
	FOptimusShaderTextEditorDocumentTabSummoner(TWeakPtr<FOptimusEditor> InEditor)
		: FDocumentTabFactory(SOptimusShaderTextEditor::DocumentTabId, InEditor.Pin())
		, EditorPtr(InEditor)
	{
	}	

	// Does this tab support the specified payload
	virtual bool IsPayloadSupported(TSharedRef<FTabPayload> Payload) const override
	{
		if (Payload->PayloadType == NAME_Object && Payload->IsValid())
		{
			UObject* DocumentID = FTabPayload_UObject::CastChecked<UObject>(Payload);
			if (IOptimusShaderTextProvider* Provider = Cast<IOptimusShaderTextProvider>(DocumentID))
			{
				return true;
			}
		}
		return false;
	}

protected:

	// Creates the label for the tab
	virtual TAttribute<FText> ConstructTabName(const FWorkflowTabSpawnInfo& Info) const override
	{
		check(Info.TabInfo.IsValid());

		const IOptimusShaderTextProvider* Provider = FTabPayload_UObject::CastChecked<IOptimusShaderTextProvider>(Info.Payload);

		return FText::FromString(Provider->GetNameForShaderTextEditor());
	}
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		check(Info.TabInfo.IsValid());

		UObject* ProviderObject = FTabPayload_UObject::CastChecked<UObject>(Info.Payload);

		TSharedRef<SDockTab> DocumentHostTab = Info.TabInfo->GetTab().Pin().ToSharedRef();
		
		return SNew(SOptimusShaderTextDocumentTab, ProviderObject, EditorPtr, DocumentHostTab);
	}

private:
	TWeakPtr<FOptimusEditor> EditorPtr;
};


TArray<FName> SOptimusShaderTextEditor::GetAllTabIds()
{
	TArray<FName> TabIds;
	TabIds.AddUnique(DocumentTabId);
	return TabIds;
}

SOptimusShaderTextEditor::~SOptimusShaderTextEditor()
{
	if (OwningEditor.Pin())
	{
		OwningEditor.Pin()->UnsubscribeToGraphNotifies(OnGraphNotifyDelegate.GetHandle());
	}	
}

void SOptimusShaderTextEditor::Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InEditor, const TSharedRef<SDockTab>& InShaderTextEditorHostTab)
{
	OwningEditor = InEditor;
	TabManager = FGlobalTabmanager::Get()->NewTabManager(InShaderTextEditorHostTab);
	InShaderTextEditorHostTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(&SOptimusShaderTextEditor::OnHostTabClosed));

	DocumentTracker = MakeShareable(new FDocumentTracker);
	// DocumentTracker holds a weak ptr to the editor
	DocumentTracker->Initialize(OwningEditor.Pin());
	const TSharedRef<FDocumentTabFactory> ShaderTextDocumentTabFactory = MakeShareable(new FOptimusShaderTextEditorDocumentTabSummoner(OwningEditor));
	DocumentTracker->RegisterDocumentFactory(ShaderTextDocumentTabFactory);
	DocumentTracker->SetTabManager(TabManager.ToSharedRef());

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("SOptimusShaderTextEditor1.0") 
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->Split
		(
			FTabManager::NewStack()
			->SetHideTabWell(false)
			->AddTab(SOptimusShaderTextEditor::DocumentTabId, ETabState::ClosedTab)
		)
	);
	
	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow( InShaderTextEditorHostTab );
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			TabManager->RestoreFrom(Layout, ParentWindow).ToSharedRef()
		]
	];

	if (ensure(OwningEditor.Pin()))
	{
		OwningEditor.Pin()->OnSelectedNodesChanged().AddSP(this, &SOptimusShaderTextEditor::HandleSelectedNodesChanged);
	}
	
	OnGraphNotifyDelegate.BindSP(this, &SOptimusShaderTextEditor::HandleGraphNotified);
}

void SOptimusShaderTextEditor::OnHostTabClosed(TSharedRef<SDockTab> InShaderTextEditorHostTab)
{
	const TSharedPtr<FTabManager> SubTabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(InShaderTextEditorHostTab);

	const TArray<FName> TabIdToFind = GetAllTabIds();
	for (const FName TabId : TabIdToFind)
	{
		while(true)
		{
			TSharedPtr<SDockTab> DocumentTab = SubTabManager->FindExistingLiveTab(TabId);
			if (DocumentTab.IsValid())
			{
				DocumentTab->RequestCloseTab();
			}
			else
			{
				break;
			}
		}
	}
}

void SOptimusShaderTextEditor::HandleSelectedNodesChanged(const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects) const
{
	for (const TWeakObjectPtr<UObject>& Object : InSelectedObjects)
	{
		TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(Object.Get());

		if (TSharedPtr<SDockTab> OpenedTab = DocumentTracker->OpenDocument(Payload, FDocumentTracker::OpenNewDocument))
		{
			if (UOptimusNode* Node = Cast<UOptimusNode>(Object))
			{
				if (ensure(OwningEditor.Pin()))
				{
					// Register our handler with the owning graph of each kernel node
					// OwningEditor should detect duplicated registration 
					OwningEditor.Pin()->SubscribeToGraphNotifies(Node->GetOwningGraph(), OnGraphNotifyDelegate);
				}
			}
		}
	}
}

void SOptimusShaderTextEditor::HandleGraphNotified(EOptimusGraphNotifyType InNotifyType, UOptimusNodeGraph* InGraph, UObject* InSubject) const
{
	switch (InNotifyType)
	{
		case EOptimusGraphNotifyType::NodeRemoved:
		{
			TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(InSubject);
			DocumentTracker->CloseTab(Payload);
			break;
		}
		case EOptimusGraphNotifyType::NodeDisplayNameChanged:
		{
			if (IOptimusShaderTextProvider* ShaderTextProvider = Cast<IOptimusShaderTextProvider>(InSubject))
			{
				TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(InSubject);
				TArray<TSharedPtr<SDockTab>> Results;
				DocumentTracker->FindMatchingTabs(Payload,Results);
				for (TSharedPtr<SDockTab> Tab : Results)
				{
					Tab->SetLabel(FText::FromString(ShaderTextProvider->GetNameForShaderTextEditor()));
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE