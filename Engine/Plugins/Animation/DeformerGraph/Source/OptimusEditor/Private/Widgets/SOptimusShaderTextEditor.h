// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "OptimusEditor.h"

class FOptimusEditor;
class FDocumentTracker;
class FTabManager;
class SDockTab;
class UOptimusNodeGraph;
enum class EOptimusGraphNotifyType;


class SOptimusShaderTextEditor : public SCompoundWidget
{
public:
	static const FName DocumentTabId;
	static TArray<FName> GetAllTabIds();
	static void OnHostTabClosed(TSharedRef<SDockTab> InShaderTextEditorHostTab);
	
	virtual ~SOptimusShaderTextEditor() override;

	SLATE_BEGIN_ARGS(SOptimusShaderTextEditor) {};
	SLATE_END_ARGS()

	// similar to SRewindDebugger, construct the widget under a tab so that
	// we can use it to create a tab manager to manage tabs under it
	void Construct(const FArguments& InArgs, TWeakPtr<FOptimusEditor> InEditor, const TSharedRef<SDockTab>& InShaderTextEditorHostTab);

	// The owning editor
	TWeakPtr<FOptimusEditor> OwningEditor;

private:
	void HandleSelectedNodesChanged(const TArray<TWeakObjectPtr<UObject>>& InSelectedObjects) const;
	void HandleGraphNotified(EOptimusGraphNotifyType InNotifyType, UOptimusNodeGraph* InGraph, UObject* InSubject) const;

	TSharedPtr<FTabManager> TabManager;

	TSharedPtr<FDocumentTracker> DocumentTracker;

	FOptimusEditor::FOnGraphNotified OnGraphNotifyDelegate;
};



