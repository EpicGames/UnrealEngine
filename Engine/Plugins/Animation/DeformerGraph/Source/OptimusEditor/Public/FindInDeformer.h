// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FindInGraph.h"

class FAssetEditorToolkit;
class SSearchBox;

/** Item that matched the search results */
class FFindInDeformerResult : public FFindInGraphResult
{
public:

	FFindInDeformerResult(const FFindInGraphResult::FCreateParams& InCreateParams);

	//~ Begin FFindInGraphResult Interface
	virtual void JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const override;
	//~ End FFindInGraphResult Interface
};


/** Widget for searching for items that are part of a Deformer Graph */
class SFindInDeformer : public SFindInGraph
{

protected:

	//~ Begin SFindInGraph Interface
	virtual TSharedPtr<FFindInGraphResult> MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams) override;
	virtual const UEdGraph* GetGraph() override;
	virtual bool MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens) override;
	//~ End SFindInGraph Interface
};
