// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaAssetEditorToolkit.h"
#include "IHasPersonaToolkit.h"

class IOptimusEditor :
	public FPersonaAssetEditorToolkit,
	public IHasPersonaToolkit
//	, public IHasMenuExtensibility
//	, public IHasToolBarExtensibility
{
public:
	virtual ~IOptimusEditor() override = default;
};
