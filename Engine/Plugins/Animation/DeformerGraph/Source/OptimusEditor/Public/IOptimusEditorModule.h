// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkit.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOptimusEditor, Log, All);

class FOptimusEditorClipboard;
class IOptimusEditor;
class UOptimusDeformer;

class IOptimusEditorModule
	: public IModuleInterface
//	, public IHasMenuExtensibility
//	, public IHasToolBarExtensibility,
	, public FStructureEditorUtils::INotifyOnStructChanged
{
public:
	static IOptimusEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IOptimusEditorModule >(TEXT("OptimusEditor"));
	}

	/// Creates an instance of a Control Rig editor.
	/// @param Mode				Mode that this editor should operate in
	/// @param InitToolkitHost	When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	/// @param DeformerObject	The deformer object to start editing.
	///	@return Interface to the new Optimus Deformer editor
	virtual TSharedRef<IOptimusEditor> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UOptimusDeformer* DeformerObject) = 0;

	/// Returns the shared clipboard for the deformer graph.
	virtual FOptimusEditorClipboard& GetClipboard() const = 0; 
};
