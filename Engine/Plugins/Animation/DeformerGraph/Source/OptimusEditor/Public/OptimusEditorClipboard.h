// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"

#define UE_API OPTIMUSEDITOR_API


class UClass;
class UObject;
class UOptimusNodeGraph;
class UOptimusNode;


class FOptimusEditorClipboard
{
public:
	static UE_API void SetClipboardFromNodes(const TArray<UOptimusNode*>& InNodes);

	// Creates a self-contained 
	static UE_API UOptimusNodeGraph* GetGraphFromClipboardContent(
		UPackage* InTargetPackage
		);

	static UE_API bool HasValidClipboardContent();

private:
	static UE_API bool CanCreateObjectsFromText(const TCHAR* InBuffer);

	static UE_API bool ProcessObjectBuffer(UPackage* InPackage, UObject* InRootOuter, const TCHAR* InBuffer);
	static UE_API bool CanCreateClass(const UClass* ObjectClass);
	static UE_API bool ProcessPostCreateObject(UObject* InRootOuter, UObject* InNewObject);
};

#undef UE_API
