// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OptimusSettings : ModuleRules
	{
		public OptimusSettings(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ComputeFramework",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"RenderCore",
					"RHI",
				}
			);
		}
	}
}
