// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusSettingsModule.h"

#include "Animation/MeshDeformer.h"
#include "ComputeFramework/ComputeFramework.h"
#include "Features/IModularFeatures.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "OptimusSettings.h"
#include "ShaderPlatformCachedIniValue.h"


namespace Optimus
{
	/** ReadOnly CVar intended for enabling/disabling of DeformerGraph support per platform. */
	static TAutoConsoleVariable<int32> CVarDeformerGraphEnable(
		TEXT("r.DeformerGraph.Enable"),
		1,
		TEXT("Set to 0 to disable DeformerGraph support on a platform.\n"),
		ECVF_ReadOnly);

	bool IsSupported(EShaderPlatform Platform)
	{
		static FShaderPlatformCachedIniValue<int32> PerPlatformCVar(TEXT("r.DeformerGraph.Enable"));
		return PerPlatformCVar.Get(Platform) != 0 && ComputeFramework::IsSupported(Platform);
	}

	bool IsEnabled()
	{
		return ComputeFramework::IsEnabled();
	}
	
	static const TCHAR* DeformerGraphEnableAssetValidationCVarName = TEXT("r.DeformerGraph.EnableAssetValidation");
	static TAutoConsoleVariable<int32> CVarDeformerGraphEnableAssetValidation(
	DeformerGraphEnableAssetValidationCVarName,
	1,
	TEXT("Set to 0 to disable Deformer Graph validation that may generate "
	  "extra warnings when Deformer Graph cannot run optimally on certain assets"));

	bool IsAssetValidationEnabled()
	{
		return CVarDeformerGraphEnableAssetValidation.GetValueOnAnyThread() != 0;
	}

	FName GetEnableAssetValidationCVarName()
	{
		return DeformerGraphEnableAssetValidationCVarName;
	}
}


UOptimusSettings::UOptimusSettings(const FObjectInitializer& ObjectInitlaizer)
	: UDeveloperSettings(ObjectInitlaizer)
{
}

bool FOptimusSettingsModule::IsSupported(EShaderPlatform Platform) const
{
	return Optimus::IsSupported(Platform);
}

#if WITH_EDITOR

UOptimusSettings::FOnUpdateSettings UOptimusSettings::OnSettingsChange;

void UOptimusSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	OnSettingsChange.Broadcast(this);
}

#endif // WITH_EDITOR


void FOptimusSettingsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IMeshDeformerProvider::ModularFeatureName, this);

#if WITH_EDITOR
	UOptimusSettings::OnSettingsChange.AddRaw(this, &FOptimusSettingsModule::CacheDefaultMeshDeformers);
#endif

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FOptimusSettingsModule::CacheDefaultMeshDeformers);
}

void FOptimusSettingsModule::ShutdownModule()
{
#if WITH_EDITOR
	UOptimusSettings::OnSettingsChange.RemoveAll(this);
#endif

	IModularFeatures::Get().UnregisterModularFeature(IMeshDeformerProvider::ModularFeatureName, this);
}

void FOptimusSettingsModule::CacheDefaultMeshDeformers()
{
	CacheDefaultMeshDeformers(GetDefault<UOptimusSettings>());
}

void FOptimusSettingsModule::CacheDefaultMeshDeformers(UOptimusSettings const* InSettings)
{
	const bool bLoadDeformers = InSettings != nullptr && InSettings->DefaultMode != EOptimusDefaultDeformerMode::Never;
	DefaultDeformer = TStrongObjectPtr<UMeshDeformer>(bLoadDeformers && !InSettings->DefaultDeformer.IsNull() ? InSettings->DefaultDeformer.LoadSynchronous() : nullptr);
	DefaultRecomputeTangentDeformer = TStrongObjectPtr<UMeshDeformer>(bLoadDeformers && !InSettings->DefaultRecomputeTangentDeformer.IsNull() ? InSettings->DefaultRecomputeTangentDeformer.LoadSynchronous() : nullptr);
}

TObjectPtr<UMeshDeformer> FOptimusSettingsModule::GetDefaultMeshDeformer(FDefaultMeshDeformerSetup const& Setup)
{
	const UOptimusSettings* Settings = GetDefault<UOptimusSettings>();
	if (Settings != nullptr && 
		(Settings->DefaultMode == EOptimusDefaultDeformerMode::Always ||
		(Settings->DefaultMode == EOptimusDefaultDeformerMode::OptIn && Setup.bIsRequestingDeformer)))
	{
		if (Setup.bIsRequestingRecomputeTangent && DefaultRecomputeTangentDeformer)
		{
			return DefaultRecomputeTangentDeformer.Get();
		}
		if (DefaultDeformer)
		{ 
			return DefaultDeformer.Get();
		}
	}
	return nullptr;
}

IMPLEMENT_MODULE(FOptimusSettingsModule, OptimusSettings)
