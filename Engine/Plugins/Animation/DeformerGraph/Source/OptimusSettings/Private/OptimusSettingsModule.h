// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MeshDeformerProvider.h"
#include "Modules/ModuleInterface.h"
#include "UObject/StrongObjectPtr.h"

class UOptimusSettings;

class FOptimusSettingsModule : public IModuleInterface, public IMeshDeformerProvider
{
	/** IModuleInterface implementation */
	void StartupModule() override;
	void ShutdownModule() override;

	/** IMeshDeformerProvider implementation */
	bool IsSupported(EShaderPlatform Platform) const override;
	TObjectPtr<UMeshDeformer> GetDefaultMeshDeformer(FDefaultMeshDeformerSetup const& InSetup) override;

private:
	void CacheDefaultMeshDeformers();
	void CacheDefaultMeshDeformers(UOptimusSettings const* InSettings);

	TStrongObjectPtr<UMeshDeformer> DefaultDeformer;
	TStrongObjectPtr<UMeshDeformer> DefaultRecomputeTangentDeformer;
};
