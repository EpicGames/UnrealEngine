// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "OptimusSettings.generated.h"

#define UE_API OPTIMUSSETTINGS_API

class UMeshDeformer;
enum EShaderPlatform : uint16;

UENUM()
enum class EOptimusDefaultDeformerMode : uint8
{
	/** Never apply the default deformers. */
	Never,
	/** Only apply default deformers if requested. */
	OptIn,
	/** Always apply the default deformers. */
	Always,
};

UCLASS(MinimalAPI, config = DeformerGraph, defaultconfig, meta = (DisplayName = "DeformerGraph"))
class UOptimusSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	/** Set when skinned meshes should have a default deformer applied. */
	UPROPERTY(config, EditAnywhere, Category = DeformerGraph)
	EOptimusDefaultDeformerMode DefaultMode = EOptimusDefaultDeformerMode::OptIn;

	/** A default deformer that will be used on a skinned mesh if no other deformer has been set. */
	UPROPERTY(config, EditAnywhere, Category = DeformerGraph, meta = (AllowedClasses = "/Script/OptimusCore.OptimusDeformer", EditCondition = "DefaultMode != EOptimusDefaultDeformerMode::Never"))
	TSoftObjectPtr<UMeshDeformer> DefaultDeformer;

	/** A default deformer that will be used on a skinned mesh if no other deformer has been set, and if the mesh has requested to recompute tangets. */
	UPROPERTY(config, EditAnywhere, Category = DeformerGraph, meta = (AllowedClasses = "/Script/OptimusCore.OptimusDeformer", EditCondition = "DefaultMode != EOptimusDefaultDeformerMode::Never"))
	TSoftObjectPtr<UMeshDeformer> DefaultRecomputeTangentDeformer;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateSettings, UOptimusSettings const*);
	static UE_API FOnUpdateSettings OnSettingsChange;

	UE_API void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
};

namespace Optimus
{
	/** Returns true if DeformerGraph is supported on a platform. */
	OPTIMUSSETTINGS_API bool IsSupported(EShaderPlatform Platform);

	/** Returns true if DeformerGraph is currently enabled. */
	OPTIMUSSETTINGS_API bool IsEnabled();
	
	/** Returns true if DeformerGraph should enable asset validation. */
	OPTIMUSSETTINGS_API bool IsAssetValidationEnabled();
	
	/** Returns the name of the CVar that toggles asset validation, used by warning messages. */
	OPTIMUSSETTINGS_API FName GetEnableAssetValidationCVarName();
}

#undef UE_API
